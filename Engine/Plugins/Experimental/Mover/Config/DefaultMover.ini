[CoreRedirects]
+ClassRedirects=(OldName="/Script/Mover.KinematicMoverComponent",NewName="/Script/Mover.MoverComponent")
+StructRedirects=(OldName="/Script/Mover.KinematicMoverInputCmdContext",NewName="/Script/Mover.MoverInputCmdContext")
+StructRedirects=(OldName="/Script/Mover.KinematicMoverSyncState",NewName="/Script/Mover.MoverSyncState")
+StructRedirects=(OldName="/Script/Mover.KinematicMoverAuxStateContext",NewName="/Script/Mover.MoverAuxStateContext")
+StructRedirects=(OldName="/Script/Mover.KinematicMoverTickStartData",NewName="/Script/Mover.MoverTickStartData")
+StructRedirects=(OldName="/Script/Mover.KinematicMoverTickEndData",NewName="/Script/Mover.MoverTickEndData")
+StructRedirects=(OldName="/Script/Mover.KinematicDefaultInputs",NewName="/Script/Mover.CharacterDefaultInputs")
+StructRedirects=(OldName="/Script/Mover.LayeredMove_JumpImpulse",NewName="/Script/Mover.LayeredMove_JumpImpulseOverDuration")

+FunctionRedirects=(OldName="/Script/Mover.BaseMovementModeTransition.K2_OnTrigger",NewName="Trigger")
+FunctionRedirects=(OldName="/Script/Mover.BaseMovementModeTransition.K2_OnEvaluate",NewName="Evaluate")
+FunctionRedirects=(OldName="/Script/Mover.BaseMovementModeTransition.GetMoverComponent",NewName="K2_GetMoverComponent")
+FunctionRedirects=(OldName="/Script/Mover.BaseMovementMode.K2_OnGenerateMove",NewName="GenerateMove")
+FunctionRedirects=(OldName="/Script/Mover.BaseMovementMode.K2_OnSimulationTick",NewName="SimulationTick")
+FunctionRedirects=(OldName="/Script/Mover.BaseMovementMode.K2_OnActivate",NewName="K2_OnActivated")
+FunctionRedirects=(OldName="/Script/Mover.BaseMovementMode.Activate",NewName="K2_OnActivated")
+FunctionRedirects=(OldName="/Script/Mover.BaseMovementMode.K2_OnDeactivate",NewName="K2_OnDeactivated")
+FunctionRedirects=(OldName="/Script/Mover.BaseMovementMode.Deactivate",NewName="K2_OnDeactivated")
+FunctionRedirects=(OldName="/Script/Mover.BaseMovementMode.GetMoverComponent",NewName="K2_GetMoverComponent")