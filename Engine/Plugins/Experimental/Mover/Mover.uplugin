{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Mover",
	"Description": "Mover is an Unreal Engine plugin to support movement of actors with rollback networking.\nPlease refer to the README document for information about getting started, an overview of concepts, and known issues.",
	"Category": "Gameplay",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "Mover",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "MoverCVDData",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"TargetConfigurationDenyList": [ "Shipping" ]
		},
		{
			"Name": "MoverEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "MoverCVDEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "NetworkPrediction",
			"Enabled": true
		},
		{
			"Name": "MotionWarping",
			"Enabled": true
		},
		{
			"Name": "PoseSearch",
			"Enabled": true
		},
		{
			"Name": "Water",
			"Enabled": true
		},
		{
			"Name": "ChaosVD",
			"Enabled": true
		}
	]
}