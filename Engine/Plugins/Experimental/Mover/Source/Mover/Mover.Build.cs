// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class Mover : ModuleRules
{
	public Mover(ReadOnlyTargetRules Target) : base(Target)
	{

		// TODO: find a better way to manage optional dependencies, such as Water and PoseSearch. This includes module dependencies here, as well as .uplugin dependencies.

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"NetCore",
				"InputCore",
				"NetworkPrediction",
				"AnimGraphRuntime",
				"MotionWarping",
				"Water",
				"GameplayTags",
				"NavigationSystem"
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Chaos",
				"CoreUObject",
				"Engine",
				"PhysicsCore",
				"DeveloperSettings",
				"PoseSearch"
			}
			);

		if (IsChaosVisualDebuggerSupported(Target))
		{
			PublicDependencyModuleNames.Add("MoverCVDData");
		}
		SetupModuleChaosVisualDebuggerSupport(Target);

		SetupGameplayDebuggerSupport(Target);

		SetupModulePhysicsSupport(Target);
	}
}
