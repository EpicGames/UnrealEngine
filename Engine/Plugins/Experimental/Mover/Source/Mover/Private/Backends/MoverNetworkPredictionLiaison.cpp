// Copyright Epic Games, Inc. All Rights Reserved.

#include "Backends/MoverNetworkPredictionLiaison.h"
#include "NetworkPredictionModelDefRegistry.h"
#include "NetworkPredictionProxyInit.h"
#include "NetworkPredictionProxyWrite.h"
#include "GameFramework/Actor.h"
#include "InputContainerStruct.h"
#include "KinematicActorSimulation.h"
#include "MoverComponent.h"

#if WITH_EDITOR
#include "Misc/DataValidation.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoverNetworkPredictionLiaison)


#define LOCTEXT_NAMESPACE "Mover"

// ----------------------------------------------------------------------------------------------------------
//	FMoverActorModelDef: the piece that ties everything together that we use to register with the NP system.
// ----------------------------------------------------------------------------------------------------------

class FMoverActorModelDef : public FNetworkPredictionModelDef
{
public:

	NP_MODEL_BODY();

	using Simulation = UMoverNetworkPredictionLiaisonComponent;
	using StateTypes = KinematicMoverStateTypes;
	using Driver = UMoverNetworkPredictionLiaisonComponent;

	static const TCHAR* GetName() { return TEXT("MoverActor"); }
	static constexpr int32 GetSortPriority() { return (int32)ENetworkPredictionSortPriority::PreKinematicMovers; }
};

NP_MODEL_REGISTER(FMoverActorModelDef);



UMoverNetworkPredictionLiaisonComponent::UMoverNetworkPredictionLiaisonComponent()
{
	PrimaryComponentTick.TickGroup = TG_PrePhysics;
	PrimaryComponentTick.bCanEverTick = true;

	bWantsInitializeComponent = true;
	bAutoActivate = true;
	SetIsReplicatedByDefault(true);

	SimulationClass = UKinematicActorSimulation::StaticClass();
}

void UMoverNetworkPredictionLiaisonComponent::ProduceInput(const int32 DeltaTimeMS, FMoverInputCmdContext* Cmd)
{
	check(MoverComp);
	MoverComp->ProduceInput(DeltaTimeMS, Cmd);
}

bool UMoverNetworkPredictionLiaisonComponent::ShouldReconcile(const FMoverSyncState* PredictedState, const FMoverSyncState* AuthorityState) const
{
	return PredictedState->ShouldReconcile(*AuthorityState);
}

bool UMoverNetworkPredictionLiaisonComponent::ShouldReconcile(const FMoverAuxStateContext* PredictedState, const FMoverAuxStateContext* AuthorityState) const
{
	return PredictedState->ShouldReconcile(*AuthorityState);
}

void UMoverNetworkPredictionLiaisonComponent::RestoreFrame(const FMoverSyncState* SyncState, const FMoverAuxStateContext* AuxState)
{
	check(MoverComp);
	check(Simulation);

	int32 NewBaseSimTimeMs = 0;
	int32 NextFrameNum = 0;

	switch (UNetworkPredictionWorldManager::ActiveInstance->PreferredDefaultTickingPolicy())
	{
		default:	// fall through
		case ENetworkPredictionTickingPolicy::Fixed:
		{
			const FFixedTickState& FixedTickState = UNetworkPredictionWorldManager::ActiveInstance->GetFixedTickState();
			FNetSimTimeStep TimeStep = FixedTickState.GetNextTimeStep();
			NewBaseSimTimeMs = TimeStep.TotalSimulationTime;
			NextFrameNum = TimeStep.Frame;
		}
		break; 

		case ENetworkPredictionTickingPolicy::Independent:
		{
			const FVariableTickState& VariableTickState = UNetworkPredictionWorldManager::ActiveInstance->GetVariableTickState();
			const FNetSimTimeStep NextVariableTimeStep = VariableTickState.GetNextTimeStep(VariableTickState.Frames[VariableTickState.ConfirmedFrame]);
			NewBaseSimTimeMs = NextVariableTimeStep.TotalSimulationTime;
			NextFrameNum = NextVariableTimeStep.Frame;

		}
		break;
	}

	FMoverTimeStep MoverTimeStep;

	MoverTimeStep.ServerFrame = NextFrameNum;
	MoverTimeStep.BaseSimTimeMs = NewBaseSimTimeMs;
	MoverTimeStep.StepMs = 0;

	Simulation->RollbackToPriorState(MoverTimeStep, *SyncState, *AuxState);

	MoverComp->OnSimulationRollback(SyncState, AuxState, MoverTimeStep);

}

void UMoverNetworkPredictionLiaisonComponent::FinalizeFrame(const FMoverSyncState* SyncState, const FMoverAuxStateContext* AuxState)
{
	ensureMsgf(IsInGameThread(), TEXT("%hs must only be called on the game thread"), __FUNCTION__);

	check(MoverComp);

	const FNetworkPredictionSettings NetworkPredictionSettings = UNetworkPredictionWorldManager::ActiveInstance->GetSettings();
	if (MoverComp->GetOwnerRole() == ROLE_SimulatedProxy && NetworkPredictionSettings.SimulatedProxyNetworkLOD == ENetworkLOD::Interpolated)
	{
		FMoverInputCmdContext InputCmd;
		MoverComp->TickInterpolatedSimProxy(MoverComp->GetLastTimeStep(), InputCmd, MoverComp, MoverComp->GetSyncState(), *SyncState, *AuxState);
	}
	
	MoverComp->FinalizeFrame(SyncState, AuxState);
}

void UMoverNetworkPredictionLiaisonComponent::FinalizeSmoothingFrame(const FMoverSyncState* SyncState, const FMoverAuxStateContext* AuxState)
{
	check(MoverComp);
	MoverComp->FinalizeSmoothingFrame(SyncState, AuxState);
}

void UMoverNetworkPredictionLiaisonComponent::InitializeSimulationState(FMoverSyncState* OutSync, FMoverAuxStateContext* OutAux)
{
	check(MoverComp);
	check(Simulation);
	StartingOutSync = OutSync;
	StartingOutAux = OutAux;

	Simulation->InitializeSimulationState(*StartingOutSync, *StartingOutAux);

	MoverComp->OnSimulationStateInitialized(*OutSync, *OutAux);

}

void UMoverNetworkPredictionLiaisonComponent::SimulationTick(const FNetSimTimeStep& TimeStep, const TNetSimInput<KinematicMoverStateTypes>& SimInput, const TNetSimOutput<KinematicMoverStateTypes>& SimOutput)
{
	check(MoverComp);
	check(Simulation);

	if (!HasBegunPlay())	// If we're not ready to run yet, maintain the same simulation state
	{
		*SimOutput.Sync = *SimInput.Sync;
		*SimOutput.Aux.Get() = *SimInput.Aux;
		return;
	}

	FMoverSimContext SimContext;
	SimContext.Simulation = Simulation;
	SimContext.Blackboard = Simulation->GetRollbackBlackboardSimWrapper();
	SimContext.SimulationOwner = MoverComp;

	FMoverTickStartData StartData;
	FMoverTickEndData EndData;

	StartData.InputCmd  = *SimInput.Cmd;
	StartData.SyncState = *SimInput.Sync;
	StartData.AuxState  = *SimInput.Aux;

	// Ensure persistent SyncStates are present in the start state for a SimTick.
	// TODO: Consider moving this functionality to a common utility in the Simulation
	for (const FMoverDataPersistence& PersistentSyncEntry : MoverComp->PersistentSyncStateDataTypes)
	{
		StartData.SyncState.SyncStateCollection.FindOrAddDataByType(PersistentSyncEntry.RequiredType);
	}
	
	FMoverTimeStep MoverTimeStep;

	MoverTimeStep.ServerFrame	= TimeStep.Frame;
	MoverTimeStep.BaseSimTimeMs = TimeStep.TotalSimulationTime;
	MoverTimeStep.StepMs		= TimeStep.StepMS;

	MoverComp->OnPreSimulate(MoverTimeStep, StartData);

	Simulation->DoSimulationTick(MoverTimeStep, StartData, OUT EndData);

	MoverComp->OnPostSimulate(MoverTimeStep, StartData, EndData);

	*SimOutput.Sync = EndData.SyncState;
    *SimOutput.Aux.Get() = EndData.AuxState;
}


double UMoverNetworkPredictionLiaisonComponent::GetCurrentSimTimeMs()
{
	return NetworkPredictionProxy.GetTotalSimTimeMS();
}

int32 UMoverNetworkPredictionLiaisonComponent::GetCurrentSimFrame()
{
	return NetworkPredictionProxy.GetPendingFrame();
}


bool UMoverNetworkPredictionLiaisonComponent::ReadPendingSyncState(OUT FMoverSyncState& OutSyncState)
{
	if (const FMoverSyncState* PendingSyncState = NetworkPredictionProxy.ReadSyncState<FMoverSyncState>(ENetworkPredictionStateRead::Simulation))
	{
		OutSyncState = *PendingSyncState;
		return true;
	}

	return false;
}

bool UMoverNetworkPredictionLiaisonComponent::WritePendingSyncState(const FMoverSyncState& SyncStateToWrite)
{
	bool bDidWriteSucceed = NetworkPredictionProxy.WriteSyncState<FMoverSyncState>([&SyncStateToWrite](FMoverSyncState& PendingSyncStateRef)
		{
			PendingSyncStateRef = SyncStateToWrite;
		}) != nullptr;

	return bDidWriteSucceed;
}


bool UMoverNetworkPredictionLiaisonComponent::ReadPresentationSyncState(OUT FMoverSyncState& OutSyncState)
{
	if (const FMoverSyncState* PendingSyncState = NetworkPredictionProxy.ReadSyncState<FMoverSyncState>(ENetworkPredictionStateRead::Presentation))
	{
		OutSyncState = *PendingSyncState;
		return true;
	}

	return false;
}


bool UMoverNetworkPredictionLiaisonComponent::WritePresentationSyncState(const FMoverSyncState& SyncStateToWrite)
{
	bool bDidWriteSucceed = NetworkPredictionProxy.WritePresentationSyncState<FMoverSyncState>([&SyncStateToWrite](FMoverSyncState& PresentationSyncStateRef)
		{
			PresentationSyncStateRef = SyncStateToWrite;
		}) != nullptr;

	return bDidWriteSucceed;
}


bool UMoverNetworkPredictionLiaisonComponent::ReadPrevPresentationSyncState(FMoverSyncState& OutSyncState)
{
	if (const FMoverSyncState* PrevPresentationSyncState = NetworkPredictionProxy.ReadPrevPresentationSyncState<FMoverSyncState>())
	{
		OutSyncState = *PrevPresentationSyncState;
		return true;
	}

	return false;
}


bool UMoverNetworkPredictionLiaisonComponent::WritePrevPresentationSyncState(const FMoverSyncState& SyncStateToWrite)
{
	bool bDidWriteSucceed = NetworkPredictionProxy.WritePrevPresentationSyncState<FMoverSyncState>([&SyncStateToWrite](FMoverSyncState& PresentationSyncStateRef)
		{
			PresentationSyncStateRef = SyncStateToWrite;
		}) != nullptr;

	return bDidWriteSucceed;
}


#if WITH_EDITOR
EDataValidationResult UMoverNetworkPredictionLiaisonComponent::ValidateData(FDataValidationContext& Context, const UMoverComponent& ValidationMoverComp) const
{
	if (const AActor* OwnerActor = ValidationMoverComp.GetOwner())
	{
		if (OwnerActor->IsReplicatingMovement())
		{
			Context.AddError(FText::Format(LOCTEXT("ConflictingReplicateMovementProperty", "The owning actor ({0}) has the ReplicateMovement property enabled. This will conflict with Network Prediction and cause poor quality movement. Please disable it."),
				FText::FromString(GetNameSafe(OwnerActor))));

			return EDataValidationResult::Invalid;
		}
	}

	return EDataValidationResult::Valid;
}
#endif // WITH_EDITOR

void UMoverNetworkPredictionLiaisonComponent::BeginPlay()
{
	Super::BeginPlay();

	if (StartingOutSync && StartingOutAux)
	{
		if (FMoverDefaultSyncState* StartingSyncState = StartingOutSync->SyncStateCollection.FindMutableDataByType<FMoverDefaultSyncState>())
		{
			const FTransform UpdatedComponentTransform = MoverComp->GetUpdatedComponentTransform();
			// if our location has changed between initialization and begin play (ex: Actors sharing an exact start location and one gets "pushed" to make them fit) lets write the new location to avoid any disagreements
			if (!UpdatedComponentTransform.GetLocation().Equals(StartingSyncState->GetLocation_WorldSpace()))
			{
				StartingSyncState->SetTransforms_WorldSpace(UpdatedComponentTransform.GetLocation(),
													 UpdatedComponentTransform.GetRotation().Rotator(),
													 FVector::ZeroVector,
													 FVector::ZeroVector);	// no initial velocity
			}
		}
	}
}


void UMoverNetworkPredictionLiaisonComponent::EndPlay(const EEndPlayReason::Type Reason)
{
	if (Simulation)
	{
		Simulation->Uninit();
		Simulation = nullptr;
	}

	MoverComp = nullptr;

	Super::EndPlay(Reason);
}

void UMoverNetworkPredictionLiaisonComponent::InitializeComponent()
{
	SetupSimulation();

	Super::InitializeComponent();
}


void UMoverNetworkPredictionLiaisonComponent::UninitializeComponent()
{
	NetworkPredictionProxy.EndPlay();

	if (Simulation)
	{
		Simulation->Uninit();
		Simulation = nullptr;
	}

	Super::UninitializeComponent();
}


void UMoverNetworkPredictionLiaisonComponent::RegisterComponentTickFunctions(bool bRegister)
{
	Super::RegisterComponentTickFunctions(bRegister);
}

void UMoverNetworkPredictionLiaisonComponent::InitializeNetworkPredictionProxy()
{
	MoverComp = GetOwner()->FindComponentByClass<UMoverComponent>();


	if (ensureAlwaysMsgf(MoverComp, TEXT("UMoverNetworkPredictionLiaisonComponent on actor %s failed to find associated Mover component. This actor's movement will not be simulated. Verify its setup."), *GetNameSafe(GetOwner())))
	{
		NetworkPredictionProxy.Init<FMoverActorModelDef>(GetWorld(), GetReplicationProxies(), this, this);
	}
}

void UMoverNetworkPredictionLiaisonComponent::ResetSimulation()
{
	SetupSimulation();
}

void UMoverNetworkPredictionLiaisonComponent::SetupSimulation()
{
	MoverComp = GetOwner()->FindComponentByClass<UMoverComponent>();

	if (Simulation)
	{
		Simulation->Uninit();
		Simulation = nullptr;
	}

	if (ensureMsgf(MoverComp, TEXT("Use of UMoverNetworkPredictionLiaisonComponent requires a sibling UMoverComponent. No movement will occur.")))
	{
		Simulation = NewObject<UKinematicActorSimulation>(MoverComp, SimulationClass, *SimulationClass->GetName(), RF_Transient);

		UKinematicActorSimulation::FInitParams SimInitParams;

		if (MoverComp)
		{
			MoverComp->SetSimulation(*Simulation);

			SimInitParams.StartingMovementMode = MoverComp->StartingMovementMode;
			SimInitParams.World = GetWorld();
			SimInitParams.MoverComp = MoverComp;
			SimInitParams.UpdatedComp = MoverComp->GetUpdatedComponent();

			for (const TPair<FName, TObjectPtr<UBaseMovementMode>>& Element : MoverComp->MovementModes)
			{
				if (Element.Key != NAME_None && Element.Value.Get() != nullptr)
				{
					SimInitParams.ModesToRegister.Add(Element.Key, TWeakObjectPtr<UBaseMovementMode>(Element.Value.Get()));
				}
			}

			for (const TObjectPtr<UBaseMovementModeTransition>& Transition : MoverComp->Transitions)
			{
				if (Transition != nullptr)
				{
					SimInitParams.TransitionsToRegister.Add(TWeakObjectPtr<UBaseMovementModeTransition>(Transition.Get()));
				}
			}
		}

		Simulation->Init(SimInitParams);
	}

}


void UMoverNetworkPredictionLiaisonComponent::OnRegister()
{
	Super::OnRegister();
}


#undef LOCTEXT_NAMESPACE
