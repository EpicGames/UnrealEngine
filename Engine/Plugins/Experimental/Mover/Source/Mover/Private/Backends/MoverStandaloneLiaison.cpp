// Copyright Epic Games, Inc. All Rights Reserved.

#include "Backends/MoverStandaloneLiaison.h"
#include "Engine/World.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/Controller.h"
#include "MoverComponent.h"
#include "KinematicActorSimulation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoverStandaloneLiaison)

namespace MoverStandaloneBackendCVars
{
	static void OnRunProduceInputOnAnyThreadChanged(IConsoleVariable* Var)
	{
		// Force a refresh of all liaisons when the global cvar changes to reflect new settings
		ForEachObjectOfClass(UMoverStandaloneLiaisonComponent::StaticClass(), [&](UObject* AsObj)
		{
			if (UMoverStandaloneLiaisonComponent* StandaloneLiaison = CastChecked<UMoverStandaloneLiaisonComponent>(AsObj, ECastCheckedType::NullAllowed))
			{
				StandaloneLiaison->SetUseAsyncProduceInput(StandaloneLiaison->GetUseAsyncProduceInput());
			}
		});
	}
	
	static void OnRunMovementSimOnAnyThreadChanged(IConsoleVariable* Var)
	{
		// Force a refresh of all liaisons when the global cvar changes to reflect new settings
		ForEachObjectOfClass(UMoverStandaloneLiaisonComponent::StaticClass(), [&](UObject* AsObj)
		{
			if (UMoverStandaloneLiaisonComponent* StandaloneLiaison = CastChecked<UMoverStandaloneLiaisonComponent>(AsObj, ECastCheckedType::NullAllowed))
			{
				StandaloneLiaison->SetUseAsyncMovementSimulationTick(StandaloneLiaison->GetUseAsyncMovementSimulationTick());
			}
		});
	}

	// Whether to allow produce input calls on any thread
	static int32 RunProduceInputOnAnyThread = 0;
	FAutoConsoleVariableRef CVarMoverStandaloneProduceInputOnAnyThread(
		TEXT("mover.standalone.RunProduceInputOnAnyThread"),
		RunProduceInputOnAnyThread,
		TEXT("Whether to allow produce input to run on any thread.\nIndividuals must also have their UseAsyncProduceInput option enabled.\n0: Game thread only, 1: Any thread"),
		FConsoleVariableDelegate::CreateStatic(&OnRunProduceInputOnAnyThreadChanged),
		ECVF_Default);
	
	// Whether to allow movement simulation ticks on any thread
	static int32 RunMovementSimOnAnyThread = 0;
	FAutoConsoleVariableRef CVarMoverStandaloneRunSimOnAnyThread(
		TEXT("mover.standalone.RunMovementSimOnAnyThread"),
		RunMovementSimOnAnyThread,
		TEXT("Whether to allow Mover simulation ticks to run on any thread. Requires use of threadsafe movement modes that do not modify scene components.\nIndividuals must also have their UseAsyncMovementSimulationTick option enabled.\n0: Game thread only, 1: Any thread"),
		FConsoleVariableDelegate::CreateStatic(&OnRunMovementSimOnAnyThreadChanged),
		ECVF_Default);

}	// end MoverStandaloneBackendCVars

UMoverStandaloneLiaisonComponent::UMoverStandaloneLiaisonComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.TickGroup = TG_PrePhysics;
	PrimaryComponentTick.bCanEverTick = false;

	ProduceInputTickFunction.bCanEverTick = true;
	ProduceInputTickFunction.bStartWithTickEnabled = true;
	ProduceInputTickFunction.SetTickFunctionEnable(true);
	ProduceInputTickFunction.TickGroup = TG_PrePhysics;
	ProduceInputTickFunction.bAllowTickBatching = true;
	ProduceInputTickFunction.bRunOnAnyThread = bUseAsyncProduceInput && MoverStandaloneBackendCVars::RunProduceInputOnAnyThread;

	SimulateMovementTickFunction.bCanEverTick = true;
	SimulateMovementTickFunction.bStartWithTickEnabled = true;
	SimulateMovementTickFunction.SetTickFunctionEnable(true);
	SimulateMovementTickFunction.TickGroup = TG_PrePhysics;
	SimulateMovementTickFunction.bRunOnAnyThread = bUseAsyncMovementSimulationTick && MoverStandaloneBackendCVars::RunMovementSimOnAnyThread;
	SimulateMovementTickFunction.bHighPriority = true;

	ApplyStateTickFunction.bCanEverTick = true;
	ApplyStateTickFunction.bStartWithTickEnabled = true;
	ApplyStateTickFunction.SetTickFunctionEnable(true);
	ApplyStateTickFunction.TickGroup = TG_PrePhysics;
	ApplyStateTickFunction.bHighPriority = true;	

	bWantsInitializeComponent = true;
	bAutoActivate = true;
	SetIsReplicatedByDefault(false);

	SimulationClass = UKinematicActorSimulation::StaticClass();

	CurrentSimTimeMs = 0.0;
	CurrentSimFrame = 0;

}

double UMoverStandaloneLiaisonComponent::GetCurrentSimTimeMs()
{
	return CurrentSimTimeMs;
}

int32 UMoverStandaloneLiaisonComponent::GetCurrentSimFrame()
{
	return CurrentSimFrame;
}


bool UMoverStandaloneLiaisonComponent::ReadPendingSyncState(OUT FMoverSyncState& OutSyncState)
{
	{
		FRWScopeLock Lock(StateDataLock, SLT_ReadOnly);
		OutSyncState = CachedLastSyncState;
	}

	return true;
}

bool UMoverStandaloneLiaisonComponent::WritePendingSyncState(const FMoverSyncState& SyncStateToWrite)
{
	if (bIsInApplySimulationState)
	{
		UE_LOGF(LogMover, Log, "Attempted to WritePendingSyncState during ApplySimulationState. Skipping. (%ls)", *GetNameSafe(GetOwner()));
		return false;	// can't write to the sync state while it's being applied. Any changes from this will be overwritten anyway.
	}

	{
		FRWScopeLock Lock(StateDataLock, SLT_Write);
		CachedLastSyncState = SyncStateToWrite;
		bIsCachedStateDirty = true;
	}

	return true;
}

void UMoverStandaloneLiaisonComponent::InitializeComponent()
{
	MoverComp = GetOwner()->FindComponentByClass<UMoverComponent>();

	if (ensureMsgf(MoverComp, TEXT("Use of UMoverStandaloneLiaisonComponent requires a sibling UMoverComponent. No movement will occur.")))
	{
		Simulation = NewObject<UKinematicActorSimulation>(MoverComp, SimulationClass, *SimulationClass->GetName(), RF_Transient);

		UKinematicActorSimulation::FInitParams SimInitParams;

		if (MoverComp)
		{
			MoverComp->SetSimulation(*Simulation);

			SimInitParams.StartingMovementMode = MoverComp->StartingMovementMode;
			SimInitParams.World = GetWorld();
			SimInitParams.MoverComp = MoverComp;
			SimInitParams.UpdatedComp = MoverComp->GetUpdatedComponent();

			for (const TPair<FName, TObjectPtr<UBaseMovementMode>>& Element : MoverComp->MovementModes)
			{
				if (Element.Key != NAME_None && Element.Value.Get() != nullptr)
				{
					SimInitParams.ModesToRegister.Add(Element.Key, TWeakObjectPtr<UBaseMovementMode>(Element.Value.Get()));
				}
			}

			for (const TObjectPtr<UBaseMovementModeTransition>& Transition : MoverComp->Transitions)
			{
				if (Transition != nullptr)
				{
					SimInitParams.TransitionsToRegister.Add(TWeakObjectPtr<UBaseMovementModeTransition>(Transition.Get()));
				}
			}
		}

		Simulation->Init(SimInitParams);
	}
	
	Super::InitializeComponent();
}

void UMoverStandaloneLiaisonComponent::UninitializeComponent()
{
	if (Simulation)
	{
		Simulation->Uninit();
		Simulation = nullptr;
	}
	
	MoverComp = nullptr;
	
	Super::UninitializeComponent();
}


void UMoverStandaloneLiaisonComponent::BeginPlay()
{
	Super::BeginPlay();

	UpdateSimulationTime();

	if (const AActor* OwnerActor = GetOwner())
	{
		ensureMsgf(OwnerActor->GetNetMode() == NM_Standalone, TEXT("UMoverStandaloneLiaisonComponent is only valid for use in Standalone projects. Movement will not work properly in networked play."));

		if (UMoverComponent* FoundMoverComp = OwnerActor->FindComponentByClass<UMoverComponent>())
		{
			MoverComp = FoundMoverComp;

			FRWScopeLock Lock(StateDataLock, SLT_Write);
			check(Simulation);
			Simulation->InitializeSimulationState(OUT CachedLastSyncState, OUT CachedLastAuxState);

			MoverComp->OnSimulationStateInitialized(CachedLastSyncState, CachedLastAuxState);

			bIsCachedStateDirty = true;
		}
		else
		{
			ensureMsgf(MoverComp, TEXT("Owning actor %s does not have a MoverComponent."), *GetNameSafe(GetOwner()));

			// Disable tick if we don't have mover for some reason..
			SetComponentTickEnabled(false);
			ProduceInputTickFunction.SetTickFunctionEnable(false);
			SimulateMovementTickFunction.SetTickFunctionEnable(false);
			ApplyStateTickFunction.SetTickFunctionEnable(false);
		}
	}
}

void UMoverStandaloneLiaisonComponent::EndPlay(const EEndPlayReason::Type Reason)
{
	if (Simulation)
	{
		Simulation->Uninit();
		Simulation = nullptr;
	}

	MoverComp = nullptr;

	Super::EndPlay(Reason);
}


FTickFunction* UMoverStandaloneLiaisonComponent::FindTickFunction(EMoverTickPhase MoverTickPhase)
{
	FTickFunction* FoundFunction = nullptr;

	switch (MoverTickPhase)
	{
		case EMoverTickPhase::ProduceInput:
			FoundFunction = &ProduceInputTickFunction;
			break;

		case EMoverTickPhase::SimulateMovement:
			FoundFunction = &SimulateMovementTickFunction;
			break;

		case EMoverTickPhase::ApplyState:
			FoundFunction = &ApplyStateTickFunction;
			break;
			
		default:
			break;
	}

	return FoundFunction;
}

void UMoverStandaloneLiaisonComponent::AddTickDependency(UActorComponent* OtherComponent, EMoverTickDependencyOrder TickOrder, EMoverTickPhase TickPhase)
{
	ensure(OtherComponent);
	FTickFunction* TickFunction = FindTickFunction(TickPhase);
	if (OtherComponent && TickFunction)
	{
		switch (TickOrder)
		{
			case EMoverTickDependencyOrder::After:
				TickFunction->RemovePrerequisite(OtherComponent, OtherComponent->PrimaryComponentTick);
				OtherComponent->PrimaryComponentTick.AddPrerequisite(this, *TickFunction);
				break;
			case EMoverTickDependencyOrder::Before:
				OtherComponent->PrimaryComponentTick.RemovePrerequisite(this, *TickFunction);
				TickFunction->AddPrerequisite(OtherComponent, OtherComponent->PrimaryComponentTick);
				break;
			
			default:
				break;
		}
	}
}

void UMoverStandaloneLiaisonComponent::SetUseAsyncProduceInput(bool bUseAsyncInputProduction)
{
	bUseAsyncProduceInput = bUseAsyncInputProduction;
	ProduceInputTickFunction.bRunOnAnyThread = bUseAsyncProduceInput && MoverStandaloneBackendCVars::RunProduceInputOnAnyThread;
}

bool UMoverStandaloneLiaisonComponent::GetUseAsyncProduceInput() const
{
	return bUseAsyncProduceInput;
}

void UMoverStandaloneLiaisonComponent::SetEnableProduceInput(bool bEnableInputProduction)
{
	ProduceInputTickFunction.SetTickFunctionEnable(bEnableInputProduction);
	ProduceInputTickFunction.bStartWithTickEnabled = bEnableInputProduction;

	if (HasBegunPlay())
	{
		if (!bEnableInputProduction && ProduceInputTickFunction.IsTickFunctionRegistered())
		{
			ProduceInputTickFunction.UnRegisterTickFunction();
		}
		else if (bEnableInputProduction && !ProduceInputTickFunction.IsTickFunctionRegistered())
		{
			const AActor* Owner = GetOwner();
			ULevel* ComponentLevel = (Owner ? Owner->GetLevel() : ToRawPtr(GetWorld()->PersistentLevel));
			ProduceInputTickFunction.RegisterTickFunction(ComponentLevel);
		}
	}
}

bool UMoverStandaloneLiaisonComponent::GetEnableProduceInput() const
{
	return ProduceInputTickFunction.IsTickFunctionEnabled();
}

void UMoverStandaloneLiaisonComponent::SetUseAsyncMovementSimulationTick(bool bUseAsyncMovementSim)
{
	bUseAsyncMovementSimulationTick = bUseAsyncMovementSim;
	SimulateMovementTickFunction.bRunOnAnyThread = bUseAsyncMovementSimulationTick && MoverStandaloneBackendCVars::RunMovementSimOnAnyThread;
}

bool UMoverStandaloneLiaisonComponent::GetUseAsyncMovementSimulationTick() const
{
	return bUseAsyncMovementSimulationTick;
}


void UMoverStandaloneLiaisonComponent::RegisterComponentTickFunctions(bool bRegister)
{
	Super::RegisterComponentTickFunctions(bRegister);

	if (bRegister)
	{
		if (SetupActorComponentTickFunction(&ProduceInputTickFunction))
		{
			// Set up tick dependencies so this order always holds true:
			//   1) Controller update, which triggers input events
			//   2) Mover input production for the next movement simulation step
			//   3) Perform simulation step to generate new state
			//   4) Apply new state to actor/components

			ProduceInputTickFunction.Target = this;

			// Input production should always wait for the controller update, and we will watch for controller changes
			if (APawn* OwnerPawn = Cast<APawn>(GetOwner()))
			{
				if (AController* OwnerController = OwnerPawn->GetController())
				{
					ProduceInputTickFunction.AddPrerequisite(OwnerController, OwnerController->PrimaryActorTick);
				}

				OwnerPawn->ReceiveControllerChangedDelegate.AddDynamic(this, &UMoverStandaloneLiaisonComponent::OnControllerChanged);
			}

			if (SetupActorComponentTickFunction(&SimulateMovementTickFunction))
			{
				SimulateMovementTickFunction.Target = this;
				SimulateMovementTickFunction.AddPrerequisite(this, this->ProduceInputTickFunction);

				if (SetupActorComponentTickFunction(&ApplyStateTickFunction))
				{
					ApplyStateTickFunction.Target = this;
					ApplyStateTickFunction.AddPrerequisite(this, this->SimulateMovementTickFunction);
				}
			}
		}
	}
	else
	{
		if (APawn* OwnerPawn = Cast<APawn>(GetOwner()))
		{
			if (AController* OwnerController = OwnerPawn->GetController())
			{
				ProduceInputTickFunction.RemovePrerequisite(OwnerController, OwnerController->PrimaryActorTick);
			}

			OwnerPawn->ReceiveControllerChangedDelegate.RemoveDynamic(this, &UMoverStandaloneLiaisonComponent::OnControllerChanged);
		}

		if (ProduceInputTickFunction.IsTickFunctionRegistered())
		{
			ProduceInputTickFunction.UnRegisterTickFunction();
		}

		if (SimulateMovementTickFunction.IsTickFunctionRegistered())
		{
			SimulateMovementTickFunction.UnRegisterTickFunction();
		}

		if (ApplyStateTickFunction.IsTickFunctionRegistered())
		{
			ApplyStateTickFunction.UnRegisterTickFunction();
		}
	}
}


void UMoverStandaloneLiaisonComponent::UpdateSimulationTime()
{
	CurrentSimTimeMs = GetWorld()->GetTimeSeconds() * 1000.0;
	CurrentSimFrame = GFrameCounter;
}

void UMoverStandaloneLiaisonComponent::TickInputProduction(float DeltaSeconds)
{
	check(MoverComp);
	UpdateSimulationTime();

	const int32 DeltaTimeMs = DeltaSeconds * 1000.f;

	FMoverInputCmdContext InputCmd;
	{
		SCOPED_NAMED_EVENT(StandaloneMoverBackendLiaison_Tick_ProduceInput, FColor::Yellow);
		MoverComp->ProduceInput(DeltaTimeMs, OUT & LastProducedInputCmd);
	}
}

void UMoverStandaloneLiaisonComponent::TickMovementSimulation(float DeltaSeconds)
{
	check(Simulation);
	check(MoverComp);

	SCOPED_NAMED_EVENT(StandaloneMoverBackendLiaison_Tick_MovementSimulation, FColor::Blue);

	if (!GetEnableProduceInput())
	{
		// If input production is turned off, we need to update current time ourselves
		UpdateSimulationTime();
	}

	FMoverTimeStep TimeStep;
	TimeStep.ServerFrame = CurrentSimFrame;
	TimeStep.BaseSimTimeMs = CurrentSimTimeMs;
	TimeStep.StepMs = DeltaSeconds * 1000.f;

	FMoverSimContext SimContext;
	SimContext.Simulation = Simulation;
	SimContext.Blackboard = Simulation->GetRollbackBlackboardSimWrapper();
	SimContext.SimulationOwner = MoverComp;

	WorkingStartData.InputCmd = LastProducedInputCmd;

	WorkingEndData.InitForNewFrame();

	{
		FRWScopeLock Lock(StateDataLock, SLT_ReadOnly);
		WorkingStartData.SyncState = CachedLastSyncState;
		WorkingStartData.AuxState = CachedLastAuxState;
	}

	{
		SCOPED_NAMED_EVENT(StandaloneMoverBackendLiaison_Tick_SimulationOnly, FColor::Blue);
		MoverComp->OnPreSimulate(TimeStep, WorkingStartData);

		Simulation->DoSimulationTick(TimeStep, WorkingStartData, OUT WorkingEndData);
		
		MoverComp->OnPostSimulate(TimeStep, WorkingStartData, WorkingEndData);
	}

	{
		if (!CachedLastSyncState.HasSameContents(WorkingEndData.SyncState) ||
			!WorkingEndData.MovementEndState.bEndedWithNoChanges)
		{
			FRWScopeLock Lock(StateDataLock, SLT_Write);
			CachedLastSyncState = WorkingEndData.SyncState;
			CachedLastAuxState = WorkingEndData.AuxState;
			bIsCachedStateDirty = true;
		}
	}

}



void UMoverStandaloneLiaisonComponent::TickApplySimulationState(float DeltaSeconds)
{
	TGuardValue<bool> InApplySimulationState(bIsInApplySimulationState, true);

	{
		SCOPED_NAMED_EVENT(StandaloneMoverBackendLiaison_Tick_FinalizeFrame, FColor::Green);

		bool bNeedsFinalizeUnchanged = true;

		{
			FRWScopeLock Lock(StateDataLock, SLT_ReadOnly);
			if (bIsCachedStateDirty)
			{
				MoverComp->FinalizeFrame(&CachedLastSyncState, &CachedLastAuxState);

				bIsCachedStateDirty = false;
				bNeedsFinalizeUnchanged = false;
			}
		}

		if (bNeedsFinalizeUnchanged)
		{
			MoverComp->FinalizeUnchangedFrame();
		}
	}
}

void UMoverStandaloneLiaisonComponent::OnControllerChanged(APawn* Pawn, AController* OldController, AController* NewController)
{
	if (OldController)
	{
		ProduceInputTickFunction.RemovePrerequisite(OldController, OldController->PrimaryActorTick);
	}

	if (NewController)
	{
		ProduceInputTickFunction.AddPrerequisite(NewController, NewController->PrimaryActorTick);
	}
}


// FMoverStandaloneProduceInputTickFunction ---------------------

void FMoverStandaloneProduceInputTickFunction::ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	// ExecuteTickHelper does the null check
	UMoverStandaloneLiaisonComponent* TargetComp = Target.Get();
	FActorComponentTickFunction::ExecuteTickHelper(TargetComp, /*bTickInEditor=*/ false, DeltaTime, TickType, [TargetComp](float DilatedTime)
		{
			TargetComp->TickInputProduction(DilatedTime);
		});
}

FString FMoverStandaloneProduceInputTickFunction::DiagnosticMessage()
{
	return Target->GetFullName() + TEXT("[UMoverStandaloneLiaisonComponent::ProduceInputTick]");
}

FName FMoverStandaloneProduceInputTickFunction::DiagnosticContext(bool bDetailed)
{
	if (bDetailed)
	{
		return FName(*FString::Printf(TEXT("MoverStandaloneProduceInputTickFunction/%s"), *GetFullNameSafe(Target.Get())));
	}

	static const FName MoverStandaloneProduceInputTickFunctionFName(TEXT("MoverStandaloneProduceInputTickFunction"));
	return MoverStandaloneProduceInputTickFunctionFName;
}

// FMoverStandaloneSimulateMovementTickFunction ---------------------

void FMoverStandaloneSimulateMovementTickFunction::ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	UMoverStandaloneLiaisonComponent* TargetComp = Target.Get();
	FActorComponentTickFunction::ExecuteTickHelper(TargetComp, /*bTickInEditor=*/ false, DeltaTime, TickType, [TargetComp](float DilatedTime)
		{
			TargetComp->TickMovementSimulation(DilatedTime);
		});
}

FString FMoverStandaloneSimulateMovementTickFunction::DiagnosticMessage()
{
	return Target->GetFullName() + TEXT("[UMoverStandaloneLiaisonComponent::SimulateMovement]");
}

FName FMoverStandaloneSimulateMovementTickFunction::DiagnosticContext(bool bDetailed)
{
	if (bDetailed)
	{
		return FName(*FString::Printf(TEXT("MoverStandaloneSimulateMovementTickFunction/%s"), *GetFullNameSafe(Target.Get())));
	}

	static const FName MoverStandaloneSimulateMovementTickFunctionFName(TEXT("MoverStandaloneSimulateMovementTickFunction"));
	return MoverStandaloneSimulateMovementTickFunctionFName;
}


// FMoverStandaloneApplyStateTickFunction ---------------------

void FMoverStandaloneApplyStateTickFunction::ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	UMoverStandaloneLiaisonComponent* TargetComp = Target.Get();
	FActorComponentTickFunction::ExecuteTickHelper(TargetComp, /*bTickInEditor=*/ false, DeltaTime, TickType, [TargetComp](float DilatedTime)
		{
			TargetComp->TickApplySimulationState(DilatedTime);
		});
}

FString FMoverStandaloneApplyStateTickFunction::DiagnosticMessage()
{
	return Target->GetFullName() + TEXT("[UMoverStandaloneLiaisonComponent::ApplyState]");
}

FName FMoverStandaloneApplyStateTickFunction::DiagnosticContext(bool bDetailed)
{
	if (bDetailed)
	{
		return FName(*FString::Printf(TEXT("MoverStandaloneApplyStateTickFunction/%s"), *GetFullNameSafe(Target.Get())));
	}

	static const FName MoverStandaloneApplyStateTickFunctionFName(TEXT("MoverStandaloneApplyStateTickFunction"));
	return MoverStandaloneApplyStateTickFunctionFName;
}
