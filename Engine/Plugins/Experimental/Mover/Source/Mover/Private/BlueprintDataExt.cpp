// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintDataExt.h"
#include "MoverLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintDataExt)


static constexpr uint8 NumValueMaps = 6;


bool FMoverDictionaryData::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	Super::NetSerialize(Ar, Map, bOutSuccess);

	uint8 HasAnyValuesBitfield = 0;
	
	if (Ar.IsSaving())
	{
		HasAnyValuesBitfield = ~(  (((uint8)BoolValues.IsEmpty())    << 0)
							     | (((uint8)IntValues.IsEmpty())     << 1)
							     | (((uint8)FloatValues.IsEmpty())   << 2)
							     | (((uint8)VectorValues.IsEmpty())  << 3)
								 | (((uint8)RotatorValues.IsEmpty()) << 4)
								 | (((uint8)NameValues.IsEmpty())    << 5));
	}

	Ar.SerializeBits(&HasAnyValuesBitfield, NumValueMaps);


	if (HasAnyValuesBitfield & (1 << 0))
	{
		Ar << BoolValues;
	}
	else if (Ar.IsLoading())
	{
		BoolValues.Empty();
	}

	if (HasAnyValuesBitfield & (1 << 1))
	{
		Ar << IntValues;
	}
	else if (Ar.IsLoading())
	{
		IntValues.Empty();
	}

	if (HasAnyValuesBitfield & (1 << 2))
	{
		Ar << FloatValues;
	}
	else if (Ar.IsLoading())
	{
		FloatValues.Empty();
	}

	if (HasAnyValuesBitfield & (1 << 3))
	{
		Ar << VectorValues;
	}
	else if (Ar.IsLoading())
	{
		VectorValues.Empty();
	}

	if (HasAnyValuesBitfield & (1 << 4))
	{
		Ar << RotatorValues;
	}
	else if (Ar.IsLoading())
	{
		RotatorValues.Empty();
	}

	if (HasAnyValuesBitfield & (1 << 5))
	{
		Ar << NameValues;
	}
	else if (Ar.IsLoading())
	{
		NameValues.Empty();
	}

	bOutSuccess = !Ar.IsError();
	return true;
}


void FMoverDictionaryData::ToString(FAnsiStringBuilderBase& Out) const
{
	Super::ToString(Out);

	for (const TPair<FName, bool>& Pair : BoolValues)
	{
		Out.Appendf("%s=%i\n", *Pair.Key.ToString(), Pair.Value);
	}

	for (const TPair<FName, int>& Pair : IntValues)
	{
		Out.Appendf("%s=%i\n", *Pair.Key.ToString(), Pair.Value);
	}

	for (const TPair<FName, double>& Pair : FloatValues)
	{
		Out.Appendf("%s=%.2f\n", *Pair.Key.ToString(), Pair.Value);
	}

	for (const TPair<FName, FVector>& Pair : VectorValues)
	{
		Out.Appendf("%s: %s\n", *Pair.Key.ToString(), *Pair.Value.ToCompactString());
	}

	for (const TPair<FName, FRotator>& Pair : RotatorValues)
	{
		Out.Appendf("%s: %s\n", *Pair.Key.ToString(), *Pair.Value.ToCompactString());
	}

	for (const TPair<FName, FName>& Pair : NameValues)
	{
		Out.Appendf("%s=%s\n", *Pair.Key.ToString(), *Pair.Value.ToString());
	}

}


void FMoverDictionaryData::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Alpha)
{
	const FMoverDictionaryData& FromTyped = *static_cast<const FMoverDictionaryData*>(&From);
	const FMoverDictionaryData& ToTyped = *static_cast<const FMoverDictionaryData*>(&To);

	const FMoverDictionaryData& PrimaryInfluence   = Alpha < 0.5f ? FromTyped : ToTyped;
	const FMoverDictionaryData& SecondaryInfluence = Alpha < 0.5f ? ToTyped : FromTyped;

	const float PrimaryWeight = Alpha < 0.5f ? 1.0f-Alpha : Alpha;
	const float SecondaryWeight = 1.0f - PrimaryWeight;

	BoolValues = PrimaryInfluence.BoolValues;
	IntValues  = PrimaryInfluence.IntValues;
	NameValues = PrimaryInfluence.NameValues;
	
	// Interpolation of floats, vectors, rotators:  we declare a primary influence that determines the keys that will exist in the interpolated map. 
	// This is determined by whether we're closer to "From" or "To".
	// Any values that exist in both primary and secondary influence will be interpolated.
	// Values that exist in the primary will be added. 
	// Values that only exist in the secondary will be dropped.

	FloatValues.Empty();
	for (const TPair<FName, double>& PrimaryPair : PrimaryInfluence.FloatValues)
	{
		if (const double* SecondaryPtr = SecondaryInfluence.FloatValues.Find(PrimaryPair.Key))
		{
			FloatValues.Add(PrimaryPair.Key, (PrimaryWeight * PrimaryPair.Value) + (SecondaryWeight * (*SecondaryPtr)));
		}
		else
		{
			FloatValues.Add(PrimaryPair);
		}
	}

	VectorValues.Empty();
	for (const TPair<FName, FVector>& PrimaryPair : PrimaryInfluence.VectorValues)
	{
		if (const FVector* SecondaryPtr = SecondaryInfluence.VectorValues.Find(PrimaryPair.Key))
		{
			// linear weighted interpolation
			VectorValues.Add(PrimaryPair.Key, (PrimaryWeight * PrimaryPair.Value) + (SecondaryWeight * (*SecondaryPtr)));
		}
		else
		{
			VectorValues.Add(PrimaryPair);
		}
	}

	RotatorValues.Empty();
	for (const TPair<FName, FRotator>& PrimaryPair : PrimaryInfluence.RotatorValues)
	{
		if (const FRotator* SecondaryPtr = SecondaryInfluence.RotatorValues.Find(PrimaryPair.Key))
		{
			// interpolate in quaternion space
			const FQuat PrimaryQuat = PrimaryPair.Value.Quaternion();
			const FQuat SecondaryQuat = SecondaryPtr->Quaternion();

			RotatorValues.Add(PrimaryPair.Key, FQuat::Slerp(PrimaryQuat, SecondaryQuat, SecondaryWeight).Rotator());
		}
		else
		{
			RotatorValues.Add(PrimaryPair);
		}
	}
}


void FMoverDictionaryData::Merge(const FMoverDataStructBase& From)
{
	const FMoverDictionaryData& FromTyped = *static_cast<const FMoverDictionaryData*>(&From);

	for (const TPair<FName, bool>& Pair : FromTyped.BoolValues)
	{
		bool* ToPtr = BoolValues.Find(Pair.Key);
		if (!ToPtr)
		{
			BoolValues.Add(Pair.Key, Pair.Value);
		}
		else
		{
			*ToPtr |= Pair.Value;
		}
	}

	for (const TPair<FName, int>& Pair : FromTyped.IntValues)
	{
		const int* ToPtr = IntValues.Find(Pair.Key);
		if (!ToPtr)
		{
			IntValues.Add(Pair.Key, Pair.Value);
		}
	}

	for (const TPair<FName, double>& Pair : FromTyped.FloatValues)
	{
		const double* ToPtr = FloatValues.Find(Pair.Key);
		if (!ToPtr)
		{
			FloatValues.Add(Pair.Key, Pair.Value);
		}
	}

	for (const TPair<FName, FVector>& Pair : FromTyped.VectorValues)
	{
		const FVector* ToPtr = VectorValues.Find(Pair.Key);
		if (!ToPtr)
		{
			VectorValues.Add(Pair.Key, Pair.Value);
		}
	}

	for (const TPair<FName, FRotator>& Pair : FromTyped.RotatorValues)
	{
		const FRotator* ToPtr = RotatorValues.Find(Pair.Key);
		if (!ToPtr)
		{
			RotatorValues.Add(Pair.Key, Pair.Value);
		}
	}

	for (const TPair<FName, FName>& Pair : FromTyped.NameValues)
	{
		const FName* ToPtr = NameValues.Find(Pair.Key);
		if (!ToPtr)
		{
			NameValues.Add(Pair.Key, Pair.Value);
		}
	}
	
}


bool FMoverDictionaryData::ShouldReconcile(const FMoverDataStructBase& AuthorityState) const
{
	// This type isn't set up to cause reconciliation.
	return false;
}
