// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVisualDebugger/MoverCVDRuntimeTrace.h"

#if WITH_CHAOS_VISUAL_DEBUGGER

#include "MoverCVDDataWrappers.h"
#include "MoverSimulationTypes.h"
#include "MoverTypes.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#include "ChaosVisualDebugger/ChaosVDParticleExtraData.h"
#include "MoverComponent.h"
#include "Chaos/PhysicsObjectInterface.h"
#include "Chaos/PhysicsObjectInternalInterface.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "ChaosVisualDebugger/ChaosVDTraceMacros.h"
#include "PBDRigidsSolver.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Interfaces/IPhysicsComponent.h"
#include "Engine/World.h"

namespace UE::MoverUtils
{

CVD_DEFINE_OPTIONAL_DATA_CHANNEL(MoverNetworkedData, EChaosVDDataChannelInitializationFlags::CanChangeEnabledState)
CVD_DEFINE_OPTIONAL_DATA_CHANNEL(MoverLocalSimData, EChaosVDDataChannelInitializationFlags::CanChangeEnabledState)

// FSkipObjectRefsMemoryWriter and FSkipObjectRefsMemoryReader are a
// workaround for serializing mover info structs with object references in them, such as the mover base.
// It currently skips object references altogether, except if those are UScriptStruct objects, which it serializes
// as the script struct name, hoping the type exists on the receiving end. This allows us to pass FInstancedStruct
// as member properties of mover info structs. It is not backwads compatible though, and may cause crashes if the
// underlying types have changed.
// Ultimately we need to implement better backwards compatibility, possiblly using FPropertyBags. When the referenced object
// is an actor with a primitive component though, we might want to attempt to translate the object reference to a particle ID and resolve that on the CVD side
// by linking to the corresponding CVD particle, if found.
class FSkipObjectRefsMemoryWriter : public FMemoryWriter
{
public:
	FSkipObjectRefsMemoryWriter(TArray<uint8, TSizedDefaultAllocator<32>>& InBytes, bool bIsPersistent = false, bool bSetOffset = false, const FName InArchiveName = NAME_None)
		: FMemoryWriter(InBytes, bIsPersistent, bSetOffset, InArchiveName)
	{
	}

	virtual FArchive& operator<<(struct FObjectPtr& Value) override
	{
		UScriptStruct* ScriptStruct = Cast<UScriptStruct>(Value.Get());
		bool bIsScriptStruct = (ScriptStruct != nullptr);
		*this << bIsScriptStruct;
		if (ScriptStruct)
		{
			// The FullName of the script struct will be something like "ScriptStruct /Script/Mover.FCharacterDefaultInputs"
			FString FullStructName = ScriptStruct->GetFullName(nullptr);
			// We don't need to save the first part since we only ever save UScriptStructs (C++ structs)
			FString StructName = FullStructName.RightChop(13); // So we chop the "ScriptStruct " part (hence 13 characters)
			*this << StructName;
		}
		return *this;
	}

	virtual FArchive& operator<<(struct FWeakObjectPtr& Value) override
	{
		return *this;
	}

	virtual FArchive& operator<<(class UObject*& Res) override
	{
		if (Res)
		{
			if (UScriptStruct* ScriptStruct = Cast<UScriptStruct>(Res))
			{
				// The FullName of the script struct will be something like "ScriptStruct /Script/Mover.FCharacterDefaultInputs"
				FString FullStructName = ScriptStruct->GetFullName(nullptr);
				// We don't need to save the first part since we only ever save UScriptStructs (C++ structs)
				FString StructName = FullStructName.RightChop(13); // So we chop the "ScriptStruct " part (hence 13 characters)
				*this << StructName;
			}
		}
		return *this;
	}
};
class FSkipObjectRefsMemoryReader : public FMemoryReader
{
public:
	FSkipObjectRefsMemoryReader(const TArray<uint8, TSizedDefaultAllocator<32>>& InBytes, bool bIsPersistent = false)
		: FMemoryReader(InBytes, bIsPersistent)
	{
	}

	virtual FArchive& operator<<(struct FObjectPtr& Value) override
	{
		bool bIsScriptStruct = false;
		*this << bIsScriptStruct;
		if (bIsScriptStruct)
		{
			FString StructName;
			*this << StructName;
			Value = Cast<UScriptStruct>(FindObject<UStruct>(nullptr, *StructName));
		}
		return *this;
	}

	virtual FArchive& operator<<(struct FWeakObjectPtr& Value)
	{
		return *this;
	}

	virtual FArchive& operator<<(class UObject*& Res) override
	{
		FString StructName;
		*this << StructName;
		Res = Cast<UScriptStruct>(FindObject<UStruct>(nullptr, *StructName));
		return *this;
	}
};

void FMoverCVDRuntimeTrace::UnwrapSimData(const FMoverCVDSimDataWrapper& InSimDataWrapper, TSharedPtr<FMoverInputCmdContext>& OutInputCmd, TSharedPtr<FMoverSyncState>& OutSyncState, TArray<TPair<FName, TSharedPtr<FMoverDataCollection>>>& OutLocalSimDataSections)
{
	// Input Cmd
	{
		// Deserialize into a new struct allocated dynamically
		FMemoryReader ArReader(InSimDataWrapper.InputCmdBytes, true);
		OutInputCmd = MakeShared<FMoverInputCmdContext>();
		FMoverInputCmdContext::StaticStruct()->SerializeBin(ArReader, OutInputCmd.Get());
		// Input cmd's Collection of custom structs
		FSkipObjectRefsMemoryReader ArInputCollectionReader(InSimDataWrapper.InputMoverDataCollectionBytes, true);
		OutInputCmd->InputCollection.SerializeDebugData(ArInputCollectionReader);
	}

	// Sync State
	{
		// Deserialize into a new struct allocated dynamically
		FMemoryReader ArReader(InSimDataWrapper.SyncStateBytes, true);
		OutSyncState = MakeShared<FMoverSyncState>();
		FMoverSyncState::StaticStruct()->SerializeBin(ArReader, OutSyncState.Get());
		// Sync State's Collection of custom structs
		FSkipObjectRefsMemoryReader ArSyncStateCollectionReader(InSimDataWrapper.SyncStateDataCollectionBytes, true);
		OutSyncState->SyncStateCollection.SerializeDebugData(ArSyncStateCollectionReader);
	}

	// Local sim data  - one deserialized FMoverDataCollection per named section
	for (const TPair<FName, TArray<uint8>>& Section : InSimDataWrapper.LocalSimDataSections)
	{
		TSharedPtr<FMoverDataCollection> DataCollection = MakeShared<FMoverDataCollection>();
		FSkipObjectRefsMemoryReader ArReader(Section.Value, true);
		DataCollection->SerializeDebugData(ArReader);
		OutLocalSimDataSections.Add(TPair<FName, TSharedPtr<FMoverDataCollection>>(Section.Key, MoveTemp(DataCollection)));
	}
}

void FMoverCVDRuntimeTrace::WrapSimData(uint32 SolverID, uint32 ParticleID, const FMoverInputCmdContext& InInputCmd, const FMoverSyncState& InSyncState, const NamedDataCollections* LocalSimDataCollections, FMoverCVDSimDataWrapper& OutSimDataWrapper)
{
	OutSimDataWrapper.SolverID = SolverID;
	OutSimDataWrapper.ParticleID = ParticleID;

	// Input Cmd
	{
		FMemoryWriter ArWriter(OutSimDataWrapper.InputCmdBytes, true);
		// This is not version friendly, we need to instead use SerializeTagParticles. Slower and Sergio is working on a faster version, but it's not available yet.
		InInputCmd.StaticStruct()->SerializeBin(ArWriter, &const_cast<FMoverInputCmdContext&>(InInputCmd));
	}
	// Input cmd's Collection of custom structs
	{
		FSkipObjectRefsMemoryWriter ArWriter(OutSimDataWrapper.InputMoverDataCollectionBytes, true);
		const_cast<FMoverInputCmdContext&>(InInputCmd).InputCollection.SerializeDebugData(ArWriter);
	}

	// Sync State
	{
		FMemoryWriter ArWriter(OutSimDataWrapper.SyncStateBytes, true);
		// This is not version friendly, we need to instead use SerializeTagParticles. Slower and Sergio is working on a faster version, but it's not available yet.
		InSyncState.StaticStruct()->SerializeBin(ArWriter, &const_cast<FMoverSyncState&>(InSyncState));
	}
	{
		FSkipObjectRefsMemoryWriter ArWriter(OutSimDataWrapper.SyncStateDataCollectionBytes, true);
		const_cast<FMoverSyncState&>(InSyncState).SyncStateCollection.SerializeDebugData(ArWriter);
	}

	// Local sim data  - serialize each named collection into its own section to preserve origin in the CVD viewer
	if (LocalSimDataCollections)
	{
		for (const TPair<FName, const FMoverDataCollection*>& NamedCollection : *LocalSimDataCollections)
		{
			TPair<FName, TArray<uint8>>& Section = OutSimDataWrapper.LocalSimDataSections.AddDefaulted_GetRef();
			Section.Key = NamedCollection.Key;
			if (NamedCollection.Value)
			{
				FSkipObjectRefsMemoryWriter ArWriter(Section.Value, true);
				const_cast<FMoverDataCollection*>(NamedCollection.Value)->SerializeDebugData(ArWriter);
			}
		}
	}
}

void FMoverCVDRuntimeTrace::TraceMoverData(UMoverComponent* MoverComponent, const FMoverInputCmdContext* InputCmd, const FMoverSyncState* SyncState, const NamedDataCollections* LocalSimDataCollections /*= nullptr*/)
{
	if (!FChaosVisualDebuggerTrace::IsTracing())
	{
		return;
	}

	if (!CVDDC_MoverNetworkedData->IsChannelEnabled())
	{
		return;
	}

	if (!InputCmd || !SyncState)
	{
		return;
	}

	if (UWorld* World = MoverComponent->GetWorld())
	{
		int32 ParticleID = INDEX_NONE;
		if (const IPhysicsComponent* PhysicsComponent = Cast<IPhysicsComponent>(MoverComponent->GetUpdatedComponent()))
		{
			Chaos::FReadPhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetRead();
			const Chaos::FPhysicsObject* PhysicsObject = PhysicsComponent ? PhysicsComponent->GetPhysicsObjectById(0) : nullptr; // get the root physics object
			const Chaos::FGeometryParticleHandle* ParticleHandle = PhysicsObject? Interface.GetParticle(PhysicsObject) : nullptr;
			ParticleID = ParticleHandle ? ParticleHandle->UniqueIdx().Idx : INDEX_NONE;
		}

		int32 SolverID = CVD_TRACE_GET_SOLVER_ID_FROM_WORLD(World);
		const NamedDataCollections* RecordedLocalSimDataCollections = CVDDC_MoverLocalSimData->IsChannelEnabled() ? LocalSimDataCollections : nullptr;
		TraceMoverDataPrivate(SolverID, ParticleID, InputCmd, SyncState, RecordedLocalSimDataCollections);
	}
}

void FMoverCVDRuntimeTrace::TraceMoverData(uint32 SolverID, uint32 ParticleID, const FMoverInputCmdContext* InputCmd, const FMoverSyncState* SyncState, const NamedDataCollections* LocalSimDataCollections /*= nullptr*/)
{
	if (!FChaosVisualDebuggerTrace::IsTracing())
	{
		return;
	}

	if (!CVDDC_MoverNetworkedData->IsChannelEnabled())
	{
		return;
	}

	if (!InputCmd || !SyncState)
	{
		return;
	}

	const NamedDataCollections* RecordedLocalSimDataCollections = CVDDC_MoverLocalSimData->IsChannelEnabled() ? LocalSimDataCollections : nullptr;
	TraceMoverDataPrivate(SolverID, ParticleID, InputCmd, SyncState, RecordedLocalSimDataCollections);
}

void FMoverCVDRuntimeTrace::TraceMoverDataPrivate(uint32 SolverID, uint32 ParticleID, const FMoverInputCmdContext* InputCmd, const FMoverSyncState* SyncState, const NamedDataCollections* LocalSimDataCollections /*= nullptr*/)
{
	FMoverCVDSimDataWrapper SimDataWrapper;
	WrapSimData(SolverID, ParticleID, *InputCmd, *SyncState, LocalSimDataCollections, SimDataWrapper);
	SimDataWrapper.MarkAsValid();

	FChaosVDScopedTLSBufferAccessor TLSDataBuffer;
	Chaos::VisualDebugger::WriteDataToBuffer(TLSDataBuffer.BufferRef, SimDataWrapper);

	FChaosVisualDebuggerTrace::TraceBinaryData(TLSDataBuffer.BufferRef, FMoverCVDSimDataWrapper::WrapperTypeName);

	// Particle extra data: structured view in the CVD particle details panel.
	// Each category name becomes a collapsible section header in the panel.
	// MoverNetworkedData channel is guaranteed enabled by callers; LocalSimDataCollections
	// is already nullptr when the MoverLocalSimData channel is disabled.
	using namespace Chaos::VisualDebugger;

	static const FName SyncStateCategoryName(TEXT("Sync State"));
	static const FName InputCategoryName(TEXT("Input"));
	static const FName NetworkedChannelName(TEXT("MoverNetworkedData"));
	static const FName LocalSimChannelName(TEXT("MoverLocalSimData"));

	FChaosVDParticleExtraData ExtraData;

	// Sync State: fixed UPROPERTY fields (MovementMode, LayeredMoves, etc.) as the first entry,
	// then each collection item (e.g. FMoverDefaultSyncState with Location/Velocity) individually.
	// Note: SyncStateCollection appears empty in the fixed-fields entry because FMoverDataCollection::DataArray
	// is not a UPROPERTY; the actual per-item data is in the entries that follow.
	{
		FChaosVDExtraDataCategory& SyncCat = ExtraData.Categories.AddDefaulted_GetRef();
		SyncCat.CategoryName = SyncStateCategoryName;
		SyncCat.SourceChannelId = NetworkedChannelName;
		SyncCat.AddEntry(FMoverSyncState::StaticStruct(), SyncState);
		for (const TSharedPtr<FMoverDataStructBase>& Item : SyncState->SyncStateCollection.GetDataArray())
		{
			if (Item.IsValid())
			{
				if (UScriptStruct* Struct = Item->GetScriptStruct())
				{
					SyncCat.AddEntry(Struct, Item.Get());
				}
			}
		}
	}

	// Input: FMoverInputCmdContext has no meaningful fixed UPROPERTY fields beyond InputCollection
	// itself, so only the collection items (e.g. FCharacterDefaultInputs) are added as entries.
	{
		FChaosVDExtraDataCategory& InputCat = ExtraData.Categories.AddDefaulted_GetRef();
		InputCat.CategoryName = InputCategoryName;
		InputCat.SourceChannelId = NetworkedChannelName;
		for (const TSharedPtr<FMoverDataStructBase>& Item : InputCmd->InputCollection.GetDataArray())
		{
			if (Item.IsValid())
			{
				if (UScriptStruct* Struct = Item->GetScriptStruct())
				{
					InputCat.AddEntry(Struct, Item.Get());
				}
			}
		}
	}

	// Local sim data: one named category per section (names are set by the caller, e.g. "LocalSimInput").
	if (LocalSimDataCollections)
	{
		for (const TPair<FName, const FMoverDataCollection*>& NamedCollection : *LocalSimDataCollections)
		{
			if (!NamedCollection.Value)
			{
				continue;
			}

			FChaosVDExtraDataCategory& LocalCat = ExtraData.Categories.AddDefaulted_GetRef();
			LocalCat.CategoryName = NamedCollection.Key;
			LocalCat.SourceChannelId = LocalSimChannelName;

			for (const TSharedPtr<FMoverDataStructBase>& Item : NamedCollection.Value->GetDataArray())
			{
				if (Item.IsValid())
				{
					if (UScriptStruct* Struct = Item->GetScriptStruct())
					{
						LocalCat.AddEntry(Struct, Item.Get());
					}
				}
			}
		}
	}

	TraceChaosVDParticleExtraData(static_cast<int32>(SolverID), static_cast<int32>(ParticleID), ExtraData);
}

} // namespace UE::MoverUtils

#endif // WITH_CHAOS_VISUAL_DEBUGGER
