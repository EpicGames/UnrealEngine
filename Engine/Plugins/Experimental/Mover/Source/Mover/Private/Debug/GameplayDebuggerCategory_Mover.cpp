// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebuggerCategory_Mover.h"
#include "MoverComponent.h"
#include "Engine/Engine.h"
#include "Engine/Font.h"
#include "GameFramework/Pawn.h"
#include "Components/PrimitiveComponent.h"
#include "DrawDebugHelpers.h"
#include "Debug/MoverDebugComponent.h"


#if WITH_GAMEPLAY_DEBUGGER

namespace FGameplayDebuggerCategoryTweakables
{
	float MaxMoveIntentDrawLength	= 150.f;	// For visualizing movement intent direction. For full-magnitude intent, how long of an arrow should be drawn?
	float OrientationDrawLength		= 80.f;		// For visualizing orientation directions. 
	bool bShowTrajectory = true;
	bool bShowTrail = false;
	bool bShowCorrections = true;
	bool bShowStateArrowViz = true;			// Toggles state parameter visualization with arrows draw on the actor
	bool bShowInputArrowViz = false;		// Toggles input parameter visualization with arrows draw on the actor
}

namespace
{
	FAutoConsoleVariableRef CVars_GameplayDebuggerCategory_Mover[] = 
	{
		FAutoConsoleVariableRef(TEXT("mover.debug.MaxMoveIntentDrawLength"), 
			FGameplayDebuggerCategoryTweakables::MaxMoveIntentDrawLength, 
			TEXT("Max length (in cm) of move intent visualization arrow"), ECVF_Default),

		FAutoConsoleVariableRef(TEXT("mover.debug.OrientationDrawLength"),
			FGameplayDebuggerCategoryTweakables::OrientationDrawLength,
			TEXT("Max length (in cm) of orientation visualization arrows"), ECVF_Default),
		
		FAutoConsoleVariableRef(TEXT("mover.debug.ShowTrajectory"), 
			FGameplayDebuggerCategoryTweakables::bShowTrajectory, 
			TEXT("Shows predicted trajectory of actor. NOTE: This should only be used on actors controlled by the server. For showing trajectory on the local player use Mover.LocalPlayer.ShowTrajectory"), ECVF_Default),

		FAutoConsoleVariableRef(TEXT("mover.debug.ShowTrail"),
			FGameplayDebuggerCategoryTweakables::bShowTrail,
			TEXT("Shows previous trail of actor. Also shows some networks corrections. NOTE: This should only be used on actors controlled by the server. For showing trails on the local player use Mover.LocalPlayer.ShowTrail"), ECVF_Default),

		FAutoConsoleVariableRef(TEXT("mover.debug.ShowCorrections"),
			FGameplayDebuggerCategoryTweakables::bShowCorrections,
			TEXT("Shows network corrections of the selected actor. NOTE: This should only be used on actors controlled by the server. For showing corrections on the local player use Mover.LocalPlayer.ShowCorrections"), ECVF_Default),

		FAutoConsoleVariableRef(TEXT("mover.debug.ShowStateArrows"),
			FGameplayDebuggerCategoryTweakables::bShowStateArrowViz,
			TEXT("If enabled, in-world arrows will be drawn to show certain state information in the Gameplay Debugger visualization."), ECVF_Default),

		FAutoConsoleVariableRef(TEXT("mover.debug.ShowInputArrows"),
			FGameplayDebuggerCategoryTweakables::bShowInputArrowViz,
			TEXT("If enabled, in-world arrows will be drawn to show certain input cmd information in the Gameplay Debugger visualization."), ECVF_Default),

	};
}


FGameplayDebuggerCategory_Mover::FGameplayDebuggerCategory_Mover()
{
	SetDataPackReplication<FRepData>(&DataPack);
}


void FGameplayDebuggerCategory_Mover::CollectData(APlayerController* OwnerPC, AActor* DebugActor)
{
	APawn* MyPawn = Cast<APawn>(DebugActor);
	UMoverComponent* MyMoverComponent = MyPawn ? Cast<UMoverComponent>(MyPawn->GetComponentByClass(UMoverComponent::StaticClass())) : nullptr;

	if (MyPawn)
	{
		// Get the debug component 
		UMoverDebugComponent* MoverDebugComponent = Cast<UMoverDebugComponent>(MyPawn->GetComponentByClass(UMoverDebugComponent::StaticClass()));
		// Make it if we can't find it
		if (!MoverDebugComponent)
		{
			MoverDebugComponent = Cast<UMoverDebugComponent>(MyPawn->AddComponentByClass(UMoverDebugComponent::StaticClass(), false, FTransform::Identity, false));
			MoverDebugComponent->SetHistoryTracking(1.0f, 20.0f);
		}
			
		MoverDebugComponent->bShowTrajectory = false;
		MoverDebugComponent->bShowTrail = false;
		MoverDebugComponent->bShowCorrections = false;

		if (FGameplayDebuggerCategoryTweakables::bShowTrajectory) 
		{
			MoverDebugComponent->DrawTrajectory();
		}
		if (FGameplayDebuggerCategoryTweakables::bShowTrail)
		{
			MoverDebugComponent->DrawTrail();
		}
		if (FGameplayDebuggerCategoryTweakables::bShowCorrections)
		{
			MoverDebugComponent->DrawCorrections();
		}
	}

	DataPack.PawnName = MyPawn ? MyPawn->GetHumanReadableName() : FString(TEXT("{red}No selected pawn."));
	DataPack.LocalRole = MyPawn ? UEnum::GetValueAsString(TEXT("Engine.ENetRole"), MyPawn->GetLocalRole()) : FString();

	// Set defaults for info that may not be available
	DataPack.MovementModeName = FString("invalid");
	DataPack.MovementBaseInfo = FString("invalid");
	DataPack.Velocity = FVector::ZeroVector;
	DataPack.MoveIntent = FVector::ZeroVector;
	DataPack.ActiveLayeredMoves.Empty();
	DataPack.ActiveModifiers.Empty();
	DataPack.SyncStateDataTypes.Empty();
	DataPack.ModeMap.Empty();
	DataPack.MoveInputType = 0;
	DataPack.MoveInput = FVector::ZeroVector;
	DataPack.OrientIntentDir = FVector::ZeroVector;
	DataPack.SuggestedModeName = FString("invalid");

	if (MyMoverComponent)
	{
		UPrimitiveComponent* MovementBaseComp = MyMoverComponent->GetMovementBase();

		DataPack.MovementModeName = MyMoverComponent->GetMovementModeName().ToString();
		DataPack.MovementBaseInfo = MovementBaseComp ? FString::Printf(TEXT("%s.%s"), *GetNameSafe(MovementBaseComp->GetOwner()), *MovementBaseComp->GetName()) : FString();
		DataPack.MoveIntent = MyMoverComponent->GetMovementIntent();
		DataPack.Velocity = MyMoverComponent->GetVelocity();

		for (const TPair<FName, TObjectPtr<UBaseMovementMode>>& ModeIter : MyMoverComponent->MovementModes)
		{
			UBaseMovementMode* MappedMode = ModeIter.Value;
			DataPack.ModeMap.Add(FString::Printf(TEXT("%s => %s"), *ModeIter.Key.ToString(), (MappedMode ? *MappedMode->GetClass()->GetName() : TEXT("null"))));

			if (ModeIter.Key == MyMoverComponent->GetMovementModeName())
			{
				for (UBaseMovementModeTransition* Transition : MyMoverComponent->MovementModes[ModeIter.Key]->Transitions)
				{
					DataPack.ActiveTransitions.Add(FString::Printf(TEXT("%s (%s)"), (Transition ? *Transition->GetClass()->GetName() : TEXT("null")), *ModeIter.Key.ToString()));
				}
			}
		}

		for (UBaseMovementModeTransition* Transition : MyMoverComponent->Transitions)
		{
			DataPack.ActiveTransitions.Add(FString::Printf(TEXT("%s (global)"), (Transition ? *Transition->GetClass()->GetName() : TEXT("null"))));
		}
		
		const FMoverSyncState& SyncState = MyMoverComponent->GetSyncState();

		for (const TSharedPtr<FLayeredMoveBase>& ActiveMove : SyncState.LayeredMoves.GetActiveMoves())
		{
			DataPack.ActiveLayeredMoves.Add(ActiveMove->ToSimpleString());
		}

		for (auto It = SyncState.MovementModifiers.GetActiveModifiersIterator(); It; ++It)
		{
			DataPack.ActiveModifiers.Add(*It->Get()->ToSimpleString());
		}

		for (auto It = SyncState.SyncStateCollection.GetDataArray().CreateConstIterator(); It; ++It)
		{
			DataPack.SyncStateDataTypes.Add(It->Get()->GetScriptStruct()->GetName());
		}

		const FMoverInputCmdContext& LastInputCmd = MyMoverComponent->GetLastInputCmd();

		if (const FCharacterDefaultInputs* DefaultInputs = LastInputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>())
		{
			DataPack.MoveInputType = (int8)DefaultInputs->GetMoveInputType();
			DataPack.MoveInput = DefaultInputs->GetMoveInput_WorldSpace();
			DataPack.OrientIntentDir = DefaultInputs->GetOrientationIntentDir_WorldSpace();
			DataPack.SuggestedModeName = DefaultInputs->SuggestedMovementMode.ToString();
		}

	}
}


void FGameplayDebuggerCategory_Mover::DrawData(APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext)
{
	AActor* FocusedActor = FindLocalDebugActor();

	if (FocusedActor != nullptr)
	{
		// Display any info attached to the focused actor

		DrawOverheadInfo(*FocusedActor, CanvasContext);
		DrawInWorldInfo(*FocusedActor, CanvasContext);
	}
	
	CanvasContext.Printf(TEXT("{yellow}%s\n{grey}Local Role: {white}%s\n{grey}Mode: {white}%s\n{grey}Velocity: {white}%s\n{grey}Speed: {white}%.2f"),
		*DataPack.PawnName,
		*DataPack.LocalRole,
		*DataPack.MovementModeName,
		*DataPack.Velocity.ToString(),
		 DataPack.Velocity.Length()
		);

	if (DataPack.MoveInputType > 0)
	{
		CanvasContext.Printf(TEXT("{grey}Move Input Type: {white}%i  {grey}Vec: {white}%s\n{grey}Input Suggested Mode: {white}%s\n{grey}Input Orient Intent: {white}%s"),
			DataPack.MoveInputType,
			*DataPack.MoveInput.ToString(),
			*DataPack.SuggestedModeName,
			*DataPack.OrientIntentDir.ToString()
		);
	}

	CanvasContext.Printf(TEXT("{yellow}Active Moves: {white}\n%s\n{yellow}Active Modifiers: {white}\n%s\n{yellow}Mode Map: \n{white}%s\n{yellow}Active Transitions: {white}\n%s\n{yellow}SyncStateTypes: {white}%s"),
		*FString::JoinBy(DataPack.ActiveLayeredMoves, TEXT("\n"), [](FString MoveAsString) { return MoveAsString; }),
		*FString::JoinBy(DataPack.ActiveModifiers, TEXT("\n"), [](FString ModifierAsString) { return ModifierAsString; }),
		*FString::JoinBy(DataPack.ModeMap, TEXT("\n"), [](FString ModeMappingAsString) { return ModeMappingAsString; }),
		*FString::JoinBy(DataPack.ActiveTransitions, TEXT("\n"), [](FString TransitionAsString) { return TransitionAsString; }),
		*FString::JoinBy(DataPack.SyncStateDataTypes, TEXT("  "), [](FString SyncStateTypeAsString) { return SyncStateTypeAsString; })
		);
}



TSharedRef<FGameplayDebuggerCategory> FGameplayDebuggerCategory_Mover::MakeInstance()
{
	return MakeShareable(new FGameplayDebuggerCategory_Mover());
}


void FGameplayDebuggerCategory_Mover::DrawOverheadInfo(AActor& DebugActor, FGameplayDebuggerCanvasContext& CanvasContext)
{
	const FVector OverheadLocation = DebugActor.GetActorLocation() + FVector(0, 0, DebugActor.GetSimpleCollisionHalfHeight());

	if (CanvasContext.IsLocationVisible(OverheadLocation))
	{
		FGameplayDebuggerCanvasContext OverheadContext(CanvasContext);
		TWeakObjectPtr<UFont> testFont = GEngine->GetSmallFont();
		OverheadContext.Font = GEngine->GetSmallFont();
		OverheadContext.FontRenderInfo.bEnableShadow = true;

		const FVector2D ScreenLoc = OverheadContext.ProjectLocation(OverheadLocation);
		
		FString ActorDesc;
		
		if (DataPack.MovementBaseInfo.Len() > 0)
		{
 			ActorDesc = FString::Printf(TEXT("{yellow}%s\n{white}%s\nBase: %s"), *DataPack.PawnName, *DataPack.MovementModeName, *DataPack.MovementBaseInfo);
		}
		else
		{
			ActorDesc = FString::Printf(TEXT("{yellow}%s\n{white}%s"), *DataPack.PawnName, *DataPack.MovementModeName);
		}

		float SizeX(0.f), SizeY(0.f);
		OverheadContext.MeasureString(ActorDesc, SizeX, SizeY);
		OverheadContext.PrintAt(ScreenLoc.X - (SizeX * 0.5f), ScreenLoc.Y - (SizeY * 1.2f), ActorDesc);
	}

}

void FGameplayDebuggerCategory_Mover::DrawInWorldInfo(AActor& DebugActor, FGameplayDebuggerCanvasContext& CanvasContext)
{
	UWorld* MyWorld = CanvasContext.GetWorld();

	UMoverComponent* MoverComp = Cast<UMoverComponent>(DebugActor.GetComponentByClass(UMoverComponent::StaticClass()));

	const FVector ActorMidLocation = DebugActor.GetActorLocation();
	const FVector ActorLowLocation = ActorMidLocation - FVector(0,0,DebugActor.GetSimpleCollisionHalfHeight()*0.95f);	// slightly above lowest point

	const FVector NudgeUp(0.0, 0.0, 2.0);


	// Draw approximate bounds
	if (CanvasContext.IsLocationVisible(ActorMidLocation))
	{
		DrawDebugCapsule(MyWorld, 
			ActorMidLocation,
			DebugActor.GetSimpleCollisionHalfHeight(), 
			DebugActor.GetSimpleCollisionRadius(), 
			FQuat(DebugActor.GetActorRotation()), 
			FColor::Green);
	}

	if (FGameplayDebuggerCategoryTweakables::bShowStateArrowViz)
	{
		// Draw arrow showing movement intent (direction + magnitude)
		if (CanvasContext.IsLocationVisible(ActorLowLocation))
		{
			DrawDebugDirectionalArrow(MyWorld,
				ActorMidLocation,
				ActorMidLocation + (DataPack.MoveIntent*FGameplayDebuggerCategoryTweakables::MaxMoveIntentDrawLength),
				40.f, FColor::Blue, false, -1.f, 0, 3.f);
		}

		// Draw overlaid arrows showing target orientation and actual
		if (MoverComp)
		{
			const FMoverSyncState& LastState = MoverComp->GetSyncState();
			const FMoverDefaultSyncState* MoverState = LastState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();

			if (MoverState)
			{
				const FVector ActualFacingDir = MoverState->GetOrientation_WorldSpace().Vector();
				const FVector TargetFacingDir = MoverComp->GetTargetOrientation().Vector();

				DrawDebugDirectionalArrow(MyWorld,
					ActorLowLocation,
					ActorLowLocation + (TargetFacingDir * FGameplayDebuggerCategoryTweakables::OrientationDrawLength),
					30.f, FColor::Yellow, false, -1.f, 0, 2.5f);

				DrawDebugDirectionalArrow(MyWorld,
					ActorLowLocation + NudgeUp,
					ActorLowLocation + NudgeUp + (ActualFacingDir * FGameplayDebuggerCategoryTweakables::OrientationDrawLength * 0.9),
					10.f, FColor::Green, false, -1.f, 0, 1.25f);

			}
		}
	}

	if (FGameplayDebuggerCategoryTweakables::bShowInputArrowViz)
	{
		// Draw arrows showing what the input cmds want to do
		if (CanvasContext.IsLocationVisible(ActorMidLocation))
		{
			if (!DataPack.MoveInput.IsNearlyZero())
			{
				DrawDebugDirectionalArrow(MyWorld,
					ActorMidLocation,
					ActorMidLocation + (DataPack.MoveInput.GetSafeNormal() * FGameplayDebuggerCategoryTweakables::MaxMoveIntentDrawLength),
					40.f, FColor::Cyan, false, -1.f, 0, 3.f);
			}

			if (!DataPack.OrientIntentDir.IsNearlyZero())
			{
				DrawDebugDirectionalArrow(MyWorld,
					ActorMidLocation + NudgeUp,
					ActorMidLocation + NudgeUp + (DataPack.OrientIntentDir.GetSafeNormal() * FGameplayDebuggerCategoryTweakables::MaxMoveIntentDrawLength),
					30.f, FColor::Orange, false, -1.f, 0, 3.f);
			}
		}

	}

}

void FGameplayDebuggerCategory_Mover::FRepData::Serialize(FArchive& Ar)
{
	Ar << PawnName;
	Ar << LocalRole;
	Ar << MovementModeName;
	Ar << MovementBaseInfo;
	Ar << Velocity;
	Ar << MoveIntent;
	Ar << ActiveLayeredMoves;
	Ar << ActiveModifiers;
	Ar << SyncStateDataTypes;
	Ar << ModeMap;
	Ar << ActiveTransitions;
	Ar << MoveInputType;
	Ar << MoveInput;
	Ar << OrientIntentDir;
	Ar << SuggestedModeName;

}

#endif // WITH_GAMEPLAY_DEBUGGER