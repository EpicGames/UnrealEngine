// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_GAMEPLAY_DEBUGGER

#include "CoreMinimal.h"
#include "GameplayDebuggerCategory.h"

// System for adding Mover-related debugging info and visualization to the Gameplay Debugger tool
class FGameplayDebuggerCategory_Mover : public FGameplayDebuggerCategory
{
public:
	FGameplayDebuggerCategory_Mover();

	virtual void CollectData(APlayerController* OwnerPC, AActor* DebugActor) override;
	virtual void DrawData(APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext) override;

	static TSharedRef<FGameplayDebuggerCategory> MakeInstance();

protected:
	void DrawOverheadInfo(AActor& DebugActor, FGameplayDebuggerCanvasContext& CanvasContext);
	void DrawInWorldInfo(AActor& DebugActor, FGameplayDebuggerCanvasContext& CanvasContext);

protected:
	struct FRepData
	{
		FString PawnName;
		FString LocalRole;

		// State
		FString MovementModeName;
		FString MovementBaseInfo;
		FVector Velocity;
		FVector MoveIntent;
		TArray<FString> ActiveLayeredMoves;
		TArray<FString> ModeMap;
		TArray<FString> ActiveTransitions;
		TArray<FString> ActiveModifiers;
		TArray<FString> SyncStateDataTypes;

		// Last input
		int8 MoveInputType;
		FVector MoveInput;
		FVector OrientIntentDir;
		FString SuggestedModeName;

		FRepData() {}

		void Serialize(FArchive& Ar);
	};
	FRepData DataPack;
};

#endif // WITH_GAMEPLAY_DEBUGGER