// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debug/MoverDebugComponent.h"

#include "DrawDebugHelpers.h"
#include "MovementModeStateMachine.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoverComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoverDebugComponent)

UMoverDebugComponent::UMoverDebugComponent()
{
	PrimaryComponentTick.TickGroup = TG_PrePhysics;
	PrimaryComponentTick.bCanEverTick = true;

	bWantsInitializeComponent = true;
	bAutoActivate = true;
	SetIsReplicatedByDefault(false);
}

void UMoverDebugComponent::InitializeComponent()
{
	Super::InitializeComponent();

	if (!SimulatedSamples.IsValid())
	{
		SimulatedSamples = MakeUnique<TCircularBuffer<FTrailSample>>(NumSimulatedSamplesToBuffer);
	}

	if (!RolledBackSamples.IsValid())
	{
		RolledBackSamples = MakeUnique<TCircularBuffer<FTrailSample>>(NumRolledBackSamplesToBuffer);
	}

	InitHistoryTracking();
}

void UMoverDebugComponent::BeginPlay()
{
	Super::BeginPlay();
	
	if (UMoverComponent* MoverComp = GetOwner()->FindComponentByClass<UMoverComponent>())
	{
		MoverComp->OnPostSimulationTick.AddDynamic(this, &UMoverDebugComponent::OnMovementSimTick);
	}
}

void UMoverDebugComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	if (UMoverComponent* MoverComp = GetOwner()->FindComponentByClass<UMoverComponent>())
	{
		MoverComp->OnPostSimulationTick.RemoveAll(this);
		MoverComp->OnPostSimulationRollback.RemoveAll(this);
	}

	Super::EndPlay(EndPlayReason);
}

void UMoverDebugComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if (bIsTrackingHistory)
	{
		UMoverComponent* MoverComp = GetOwner()->FindComponentByClass<UMoverComponent>();
		if (MoverComp && MoverComp->CachedLastSimTickTimeStep.ServerFrame > 0)
		{
			UpdateHistoryTrackingForFrame(MoverComp->CachedLastSimTickTimeStep, &MoverComp->GetSyncState(), &MoverComp->CachedLastAuxState);
		}
	}
	
	if (bShowTrail)
	{
		DrawTrail();
	}
	if (bShowTrajectory)
	{
		DrawTrajectory();
	}
	if (bShowCorrections)
	{
		DrawCorrections();
	}
}

void UMoverDebugComponent::SetHistoryTracking(float SecondsToTrack, float SamplesPerSecond)
{
	HistoryTrackingSeconds = SecondsToTrack;
    HistorySamplesPerSecond = SamplesPerSecond;

    InitHistoryTracking();
}

TArray<FTrajectorySampleInfo> UMoverDebugComponent::GetPastTrajectory() const
{
	if (bIsTrackingHistory && !HistorySamples.IsEmpty())
	{
		TArray<FTrajectorySampleInfo> HistoryAsArray;
		HistoryAsArray.Reserve( HistorySamples.Num() );

		for (const auto& Sample : HistorySamples)
		{
			HistoryAsArray.Emplace(Sample);
		}

		return HistoryAsArray;
	}

	return TArray<FTrajectorySampleInfo>();
}

void UMoverDebugComponent::InitHistoryTracking()
{
	if (UMoverComponent* MoverComp = GetOwner()->FindComponentByClass<UMoverComponent>())
	{
		const float ExtraSampleBufferPct = 1.5f;
		bIsTrackingHistory = HistoryTrackingSeconds > 0.f;
		
		if (bIsTrackingHistory)
		{
			HistorySamples.Reset();
			HistorySamples.Reserve(FMath::CeilToInt(HistoryTrackingSeconds * HistorySamplesPerSecond * ExtraSampleBufferPct));
	   
			MoverComp->OnPostSimulationRollback.RemoveAll(this);
			MoverComp->OnPostSimulationRollback.AddDynamic(this, &UMoverDebugComponent::OnHistoryTrackingRollback);
		}
		else
		{
			HistorySamples.Empty();
			MoverComp->OnPostSimulationRollback.RemoveAll(this);
		}
		
		MoverComp->OnPostSimulationRollback.AddDynamic(this, &UMoverDebugComponent::OnMovementSimRollback);
	}
}

void UMoverDebugComponent::UpdateHistoryTrackingForFrame(const FMoverTimeStep& TimeStep, const FMoverSyncState* SyncState, const FMoverAuxStateContext* AuxState)
{
	const double CurrentSimTimeMs = TimeStep.BaseSimTimeMs;
	const double MaxTimeBetweenSamplesMs = (1000.0 / HistorySamplesPerSecond);
	const double MinSimTimeOfSampleMs = CurrentSimTimeMs - (HistoryTrackingSeconds * 1000.0);

	// Check if it's time to kick any samples out
	HistorySamples.RemoveAll([&](const FTrajectorySampleInfo& Sample)
		{
			return (Sample.SimTimeMs < MinSimTimeOfSampleMs);	// remove old samples
		});


	// Check if it's time to cache another sample
	FTrajectorySampleInfo* MostRecentSample = HistorySamples.IsEmpty() ? nullptr : &HistorySamples.Last();

	if (!MostRecentSample || (CurrentSimTimeMs > MostRecentSample->SimTimeMs + MaxTimeBetweenSamplesMs))
	{
		const FMoverDefaultSyncState* MoverState = SyncState->SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();

		// TODO: Also need to cache samples based on other criteria such as changes in orientation, distance moves, acceleration changes, etc.

		FTrajectorySampleInfo Sample;
		Sample.SimTimeMs = CurrentSimTimeMs;
		Sample.Transform = FTransform(MoverState->GetOrientation_WorldSpace(), MoverState->GetLocation_WorldSpace());
		Sample.LinearVelocity = MoverState->GetVelocity_WorldSpace();

		const float DeltaSeconds = MostRecentSample ? (Sample.SimTimeMs - MostRecentSample->SimTimeMs) : 0.f;

		if (MostRecentSample && DeltaSeconds > 0.f)
		{
			Sample.InstantaneousAcceleration = (Sample.LinearVelocity - MostRecentSample->LinearVelocity) / DeltaSeconds;
			Sample.AngularVelocity = (MoverState->GetOrientation_WorldSpace() - MostRecentSample->Transform.Rotator()) * (1.f / DeltaSeconds);
		}
		else
		{
			Sample.InstantaneousAcceleration = FVector::ZeroVector;
			Sample.AngularVelocity = FRotator::ZeroRotator;
		}

		HistorySamples.Emplace(Sample);
	}
}

void UMoverDebugComponent::OnHistoryTrackingRollback(const FMoverTimeStep& NewTimeStep, const FMoverTimeStep& InvalidatedTimeStep)
{
	const double CurrentSimTimeMs = NewTimeStep.BaseSimTimeMs;

	// Remove any samples that are newer than the current sim time. This implies they've been rolled back.
	HistorySamples.RemoveAll([&](const FTrajectorySampleInfo& Sample)
		{
			return (Sample.SimTimeMs >= CurrentSimTimeMs);
		});
}

void UMoverDebugComponent::DrawTrajectory()
{
	const float CoordinateDrawLength = 20.f;
	const float CoordinateDrawWidth = 1.0f;
	const float ConnectiveLineWidth = 0.4f;

	if (UMoverComponent* MoverComp = GetOwner()->FindComponentByClass<UMoverComponent>())
	{
		FMoverPredictTrajectoryParams PredictionParams;
		PredictionParams.NumPredictionSamples = FMath::Max(1, LookaheadSeconds * LookaheadSamplesPerSecond);
		PredictionParams.SecondsPerSample = LookaheadSeconds / (float)PredictionParams.NumPredictionSamples;

		TArray<FTrajectorySampleInfo> TrajectorySamples = MoverComp->GetPredictedTrajectory(PredictionParams);

		for (int32 i = 0; i < TrajectorySamples.Num() - 1; ++i)
		{
			const FTrajectorySampleInfo& Sample = TrajectorySamples[i];
			const FTrajectorySampleInfo& NextSample = TrajectorySamples[i + 1];

			// Draw each sample's location/orientation and line to next sample
			DrawDebugCoordinateSystem(GetWorld(), Sample.Transform.GetLocation(), Sample.Transform.GetRotation().Rotator(), CoordinateDrawLength, false, -1.f, 1, CoordinateDrawWidth);
			DrawDebugLine(GetWorld(), Sample.Transform.GetLocation(), NextSample.Transform.GetLocation(), FColor::White, false, -1.f, 0, ConnectiveLineWidth);
		}
	}

	TArray<FTrajectorySampleInfo> PastTrajectorySamples = GetPastTrajectory(); 

	for (int i = 0; i < PastTrajectorySamples.Num()-1; ++i)
	{
		const FTrajectorySampleInfo& Sample = PastTrajectorySamples[i];
		const FTrajectorySampleInfo& NextSample = PastTrajectorySamples[i + 1];

		// Draw each sample's location/orientation and line to next sample
		DrawDebugCoordinateSystem(GetWorld(), Sample.Transform.GetLocation(), Sample.Transform.GetRotation().Rotator(), CoordinateDrawLength, false, -1.f, 1, CoordinateDrawWidth);
		DrawDebugLine(GetWorld(), Sample.Transform.GetLocation(), NextSample.Transform.GetLocation(), FColor::Silver, false, -1.f, 0, ConnectiveLineWidth);
	}
}

void UMoverDebugComponent::OnMovementSimTick(const FMoverTimeStep& TimeStep)
{
	//  Capture a sample, indexed by sim frame
	FrameOfLastSample = TimeStep.ServerFrame;

	FTrailSample NewSample;
	NewSample.Location = GetOwner()->GetActorLocation();
	NewSample.SimFrame = FrameOfLastSample;
	NewSample.GameTimeSecs = GetWorld()->TimeSeconds;

	(*SimulatedSamples)[FrameOfLastSample] = NewSample;
}

void UMoverDebugComponent::OnMovementSimRollback(const FMoverTimeStep& NewTimeStep, const FMoverTimeStep& InvalidatedTimeStep)
{
	// Use both timesteps to get a frame range. Transfer those samples from SimulatedSamples to RolledBackSamples.
	// Note that this method may cause lost rollback samples if we see rapid repeated corrections overlapping.
	float GameTimeSeconds = GetWorld()->TimeSeconds;

	int32 NewestRolledBackFrameNum = InvalidatedTimeStep.ServerFrame;
	int32 OldestRolledBackFrameNum = NewTimeStep.ServerFrame + 1;

	check(OldestRolledBackFrameNum >= 0 && NewestRolledBackFrameNum >= 0);
	
	if (OldestRolledBackFrameNum <= NewestRolledBackFrameNum)
	{
		for (int i = NewestRolledBackFrameNum; i >= OldestRolledBackFrameNum; --i)
		{
			(*RolledBackSamples)[i] = (*SimulatedSamples)[i];
			(*RolledBackSamples)[i].GameTimeSecs = GameTimeSeconds;
		}
	}

	bHasValidRollbackSamples = true;
	HighestRolledBackFrame = FMath::Max(HighestRolledBackFrame, NewestRolledBackFrameNum);
	FrameOfLastSample = NewTimeStep.ServerFrame-1;
	
	if (bShowCorrections)
	{
		CorrectedLocations.Add(GetOwner()->GetActorLocation());
		ClientLocations.Add((*SimulatedSamples)[HighestRolledBackFrame].Location);
	}
}

void UMoverDebugComponent::DrawCorrections()
{
	if (ClientLocations.Num() != CorrectedLocations.Num())
	{
		UE_LOGF(LogMover, Warning, "Correction arrays differed in size!");
		CorrectedLocations.Empty();
		ClientLocations.Empty();
		return;
	}

	if (ClientLocations.IsEmpty())
	{
		return;
	}
	
	for (int i = 0; i < ClientLocations.Num(); i++)
	{
		const FVector CorrectedLocation = CorrectedLocations[i];
		const FVector ClientLocation = ClientLocations[i];
		const FVector LocDiff = CorrectedLocation - ClientLocation;
		const float HalfHeight = GetOwner()->GetSimpleCollisionHalfHeight();
		const float CollisionRadius = GetOwner()->GetSimpleCollisionRadius();
		const float DebugLifetime = 4.0f;
	
		if (!LocDiff.IsNearlyZero())
		{
			// When server corrects us to a new location, draw red at location where client thought they were, green where the server corrected us to
			DrawDebugCapsule(GetWorld(), CorrectedLocation, HalfHeight, CollisionRadius, FQuat::Identity, FColor::Green, false, DebugLifetime);
			DrawDebugCapsule(GetWorld(), ClientLocation, HalfHeight, CollisionRadius, FQuat::Identity, FColor::Red, false, DebugLifetime);
		}
		else
		{
			// When we receive a server correction that doesn't change our position from where our client move had us, draw yellow (otherwise would be overlapping)
			// This occurs when we receive an initial correction, replay moves to get us into the right location, and then receive subsequent corrections by the server (who doesn't know if we corrected already
			// so continues to send corrections). This is a "no-op" server correction with regards to location since we already corrected (occurs with latency)
			DrawDebugCapsule(GetWorld(), ClientLocation, HalfHeight, CollisionRadius, FQuat::Identity, FColor::Yellow, false, DebugLifetime);
		}
	}

	CorrectedLocations.Empty();
	ClientLocations.Empty();
}

void UMoverDebugComponent::DrawTrail()
{
	const float HalfHeight = GetOwner()->GetSimpleCollisionHalfHeight();
	const FVector UpOffset = FVector::UpVector * HalfHeight;
	const FVector DownOffset = FVector::DownVector * HalfHeight;
	const float OldestGameTimeToDraw = GetWorld()->TimeSeconds - OldestSampleToRenderByGameSecs;
	const int32 OldestPossibleSample = FrameOfLastSample - (int32)SimulatedSamples->Capacity();

	// Draw all samples, newest to oldest within a world time limit
	for (int32 i= FrameOfLastSample; (i >= 0) && (i > OldestPossibleSample); --i)
	{
		const FTrailSample& Sample = (*SimulatedSamples)[i];

		if (Sample.GameTimeSecs < OldestGameTimeToDraw)
		{
			break;	// we've reached older than desired samples and there won't be any newer ones, so exit now
		}

		DrawDebugLine(GetWorld(), Sample.Location + DownOffset, Sample.Location + UpOffset, FColor::Blue, false, -1.f, 0, 1.f);

	}

	//Draw all rolled back samples if they are reasonably fresh, and stop trying after we detect they're all stale
	if (bHasValidRollbackSamples)
	{
		const int32 OldestPossibleRollbackFrame = HighestRolledBackFrame - (int32)RolledBackSamples->Capacity();

		bool bDrewAtLeastOneRollbackFrame = false;

		for (int32 i = HighestRolledBackFrame; (i >= 0 && i > OldestPossibleRollbackFrame); --i)
		{
			const FTrailSample& Sample = (*RolledBackSamples)[i];

			if (Sample.GameTimeSecs < OldestGameTimeToDraw)
			{
				break;	// we've reached older than desired samples and there won't be any newer ones, so exit now
			}
			
			DrawDebugLine(GetWorld(), Sample.Location + DownOffset, Sample.Location + UpOffset, FColor::Red, false, -1.f, 0, 1.f);
			bDrewAtLeastOneRollbackFrame = true;
		}

		if (!bDrewAtLeastOneRollbackFrame)
		{
			bHasValidRollbackSamples = false;
		}
	}
}
