// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultMovementSet/CharacterMoverComponent.h"

#include "Animation/AnimInstance.h"
#include "Animation/AnimMontage.h"
#include "Components/SkeletalMeshComponent.h"
#include "DefaultMovementSet/InstantMovementEffects/BasicInstantMovementEffects.h"
#include "DefaultMovementSet/Modes/FallingMode.h"
#include "DefaultMovementSet/Modes/FlyingMode.h"
#include "DefaultMovementSet/Modes/WalkingMode.h"
#include "DefaultMovementSet/Settings/CommonLegacyMovementSettings.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/MovementUtils.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(CharacterMoverComponent)

#if !UE_BUILD_SHIPPING
FAutoConsoleVariable CVarLogSimProxyMontageReplication(
	TEXT("mover.debug.LogSimProxyMontageReplication"),
	false,
	TEXT("Whether to log detailed information about montage replication on a sim proxy using the Character-focused MoverComponent. 0: Disable, 1: Enable"),
	ECVF_Cheat);
#endif	// !UE_BUILD_SHIPPING

UCharacterMoverComponent::UCharacterMoverComponent()
{
	// Default movement modes
	MovementModes.Add(DefaultModeNames::Walking, CreateDefaultSubobject<UWalkingMode>(TEXT("DefaultWalkingMode")));
	MovementModes.Add(DefaultModeNames::Falling, CreateDefaultSubobject<UFallingMode>(TEXT("DefaultFallingMode")));
	MovementModes.Add(DefaultModeNames::Flying,  CreateDefaultSubobject<UFlyingMode>(TEXT("DefaultFlyingMode")));

	StartingMovementMode = DefaultModeNames::Falling;
}

void UCharacterMoverComponent::BeginPlay()
{
	Super::BeginPlay();

	OnHandlerSettingChanged();

	OnPostFinalize.AddDynamic(this, &UCharacterMoverComponent::OnMoverPostFinalize);
}

bool UCharacterMoverComponent::GetHandleJump() const
{
	return bHandleJump;
}

void UCharacterMoverComponent::SetHandleJump(bool bInHandleJump)
{
	if (bHandleJump != bInHandleJump)
	{
		bHandleJump = bInHandleJump;
		OnHandlerSettingChanged();
	}
}

bool UCharacterMoverComponent::GetHandleStanceChanges() const
{
	return bHandleStanceChanges;
}

void UCharacterMoverComponent::SetHandleStanceChanges(bool bInHandleStanceChanges)
{
	if (bHandleStanceChanges != bInHandleStanceChanges)
	{
		bHandleStanceChanges = bInHandleStanceChanges;
		OnHandlerSettingChanged();
	}
}

bool UCharacterMoverComponent::IsCrouching() const
{
	return HasGameplayTag(Mover_IsCrouching, true);
}

bool UCharacterMoverComponent::IsFlying() const
{
	return HasGameplayTag(Mover_IsFlying, true);
}

bool UCharacterMoverComponent::IsFalling() const
{
	return HasGameplayTag(Mover_IsFalling, true);
}

bool UCharacterMoverComponent::IsAirborne() const
{
	return HasGameplayTag(Mover_IsInAir, true);
}

bool UCharacterMoverComponent::IsOnGround() const
{
	return HasGameplayTag(Mover_IsOnGround, true);
}

bool UCharacterMoverComponent::IsSwimming() const
{
	return HasGameplayTag(Mover_IsSwimming, true);
}

bool UCharacterMoverComponent::IsSlopeSliding() const
{
	if (IsAirborne())
	{
		FFloorCheckResult HitResult;
		const UMoverBlackboard* MoverBlackboard = GetSimBlackboard();
		if (MoverBlackboard && MoverBlackboard->TryGet(CommonBlackboard::LastFloorResult, HitResult))
		{
			return HitResult.bBlockingHit && !HitResult.bWalkableFloor;
		}
	}

	return false;
}

bool UCharacterMoverComponent::CanActorJump() const
{
	return IsOnGround();
}

bool UCharacterMoverComponent::Jump()
{
	if (const UCommonLegacyMovementSettings* CommonSettings = FindSharedSettings<UCommonLegacyMovementSettings>())
	{
		TSharedPtr<FJumpImpulseEffect> JumpMove = MakeShared<FJumpImpulseEffect>();
		JumpMove->UpwardsSpeed = CommonSettings->JumpUpwardsSpeed;
		
		QueueInstantMovementEffect(JumpMove);

		return true;
	}

	return false;
}

bool UCharacterMoverComponent::CanCrouch()
{
	return true;
}

void UCharacterMoverComponent::Crouch()
{
	if (CanCrouch())
	{
		bWantsToCrouch = true;
	}
}

void UCharacterMoverComponent::UnCrouch()
{
	bWantsToCrouch = false;
}

void UCharacterMoverComponent::OnMoverPreSimulationTick(const FMoverTimeStep& TimeStep, const FMoverInputCmdContext& InputCmd)
{
	if (bHandleJump)
	{
		const FCharacterDefaultInputs* CharacterInputs = InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
		if (CharacterInputs && CharacterInputs->bIsJumpJustPressed && CanActorJump())
		{
			Jump();
		}
	}
	
	if (bHandleStanceChanges)
	{
		const FStanceModifier* StanceModifier = static_cast<const FStanceModifier*>(FindMovementModifier(StanceModifierHandle));
		// This is a fail safe in case our handle was bad - try finding the modifier by type if we can
		if (!StanceModifier)
		{
			StanceModifier = FindMovementModifierByType<FStanceModifier>();
		}
	
		EStanceMode OldActiveStance = EStanceMode::Invalid;
		if (StanceModifier)
		{
			OldActiveStance = StanceModifier->ActiveStance;
		}
	
		const bool bIsCrouching = HasGameplayTag(Mover_IsCrouching, true);
		if (bIsCrouching && (!bWantsToCrouch || !CanCrouch()))
		{	
			if (StanceModifier && StanceModifier->CanExpand(this))
			{
				CancelModifierFromHandle(StanceModifier->GetHandle());
				StanceModifierHandle.Invalidate();

				StanceModifier = nullptr;
			}
		}
		else if (!bIsCrouching && bWantsToCrouch && CanCrouch())
		{
			TSharedPtr<FStanceModifier> NewStanceModifier = MakeShared<FStanceModifier>();
			StanceModifierHandle = QueueMovementModifier(NewStanceModifier);

			StanceModifier = NewStanceModifier.Get();
		}
	
		EStanceMode NewActiveStance = EStanceMode::Invalid;
		if (StanceModifier)
		{
			NewActiveStance = StanceModifier->ActiveStance;
		}

		if (OldActiveStance != NewActiveStance)
		{
			OnStanceChanged.Broadcast(OldActiveStance, NewActiveStance);
		}
	}
}

void UCharacterMoverComponent::OnMoverPostFinalize(const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState)
{
	UpdateSyncedMontageState(GetLastTimeStep(), SyncState, AuxState);
}

void UCharacterMoverComponent::OnHandlerSettingChanged()
{
	const bool bIsHandlingAnySettings = bHandleJump || bHandleStanceChanges;

	if (bIsHandlingAnySettings)
	{
		OnPreSimulationTick.AddUniqueDynamic(this, &UCharacterMoverComponent::OnMoverPreSimulationTick);
	}
	else
	{
		OnPreSimulationTick.RemoveDynamic(this, &UCharacterMoverComponent::OnMoverPreSimulationTick);
	}
}

void UCharacterMoverComponent::UpdateSyncedMontageState(const FMoverTimeStep& TimeStep, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState)
{
	if (GetOwnerRole() == ROLE_SimulatedProxy)
	{
		const FLayeredMove_MontageStateProvider* MontageStateProvider = static_cast<const FLayeredMove_MontageStateProvider*>(SyncState.LayeredMoves.FindActiveMove(FLayeredMove_MontageStateProvider::StaticStruct()));

		bool bShouldStopSyncedMontage = false;
		bool bShouldStartNewMontage = false;
		FMoverAnimMontageState NewMontageState;

		if (SyncedMontageState.Montage)
		{
			if (MontageStateProvider)
			{
				NewMontageState = MontageStateProvider->GetMontageState();

				if (NewMontageState.Montage != SyncedMontageState.Montage)
				{
					bShouldStartNewMontage = true;
					bShouldStopSyncedMontage = true;
				}
				else
				{
					// We are continuing a montage. Check for the local mesh's montage drifting and subtly adjust playback rate to keep it close to the movement state.
					const USkeletalMeshComponent* MeshComp = Cast<USkeletalMeshComponent>(GetPrimaryVisualComponent());
					if (UAnimInstance* MeshAnimInstance = MeshComp ? MeshComp->GetAnimInstance() : nullptr)
					{
						const float LocalMontagePos = MeshAnimInstance->Montage_GetPosition(SyncedMontageState.Montage);

						const float MontagePosError = NewMontageState.CurrentPosition - LocalMontagePos;

						constexpr float MaxDriftCorrectionPct = 0.1f;	// +- 10% play rate adjustment
						constexpr float DriftSecsForMaxCorrection = 0.010f;	// if more than 10 ms away, apply full drift correction

						const float CorrectivePlayRate = NewMontageState.PlayRate + (MaxDriftCorrectionPct * (FMath::Clamp(MontagePosError, -DriftSecsForMaxCorrection, DriftSecsForMaxCorrection) / DriftSecsForMaxCorrection));
					
						MeshAnimInstance->Montage_SetPlayRate(SyncedMontageState.Montage, CorrectivePlayRate);

						#if !UE_BUILD_SHIPPING
						UE_CLOG(CVarLogSimProxyMontageReplication->GetBool(), LogMover, Log, TEXT("Mover SP montage continued (SimF %i SimT: %.3f) LocalMontagePos %.3f (Drift %.3f) => corrective playrate: %.2f"),
							TimeStep.ServerFrame, TimeStep.BaseSimTimeMs, LocalMontagePos, MontagePosError, CorrectivePlayRate);
						#endif // !UE_BUILD_SHIPPING
					}
				}
			}
			else
			{
				bShouldStopSyncedMontage = true;
			}
		}
		else // We aren't actively syncing a montage state yet
		{
			if (MontageStateProvider)
			{
				// We have just received a montage state to sync against
				NewMontageState = MontageStateProvider->GetMontageState();
				bShouldStartNewMontage = true;
			}
		}

		if (bShouldStopSyncedMontage || bShouldStartNewMontage)
		{
			const USkeletalMeshComponent* MeshComp = Cast<USkeletalMeshComponent>(GetPrimaryVisualComponent());
			UAnimInstance* MeshAnimInstance = MeshComp ? MeshComp->GetAnimInstance() : nullptr;

			if (bShouldStopSyncedMontage)
			{
				#if !UE_BUILD_SHIPPING
				UE_CLOGF(CVarLogSimProxyMontageReplication->GetBool(), LogMover, Log, "Mover SP montage repl (SimF %i SimT: %.3f): STOP %ls",
					TimeStep.ServerFrame, TimeStep.BaseSimTimeMs, *SyncedMontageState.Montage->GetName());
				#endif // !UE_BUILD_SHIPPING

				if (MeshAnimInstance)
				{
					MeshAnimInstance->Montage_Stop(SyncedMontageState.Montage->GetDefaultBlendOutTime(), SyncedMontageState.Montage);
				}

				SyncedMontageState.Reset();
			}

			if (bShouldStartNewMontage && NewMontageState.Montage && MeshAnimInstance)
			{
				const float StartPosition = NewMontageState.CurrentPosition;
				const float PlaySeconds = MeshAnimInstance->Montage_Play(NewMontageState.Montage, NewMontageState.PlayRate, EMontagePlayReturnType::MontageLength, StartPosition);

				#if !UE_BUILD_SHIPPING
				UE_CLOGF(CVarLogSimProxyMontageReplication->GetBool(), LogMover, Log, "Mover SP montage repl (SimF %i SimT: %.3f): PLAY %ls (StartPos: %.3f  Rate: %.3f  PlaySecs: %.3f)",
					TimeStep.ServerFrame, TimeStep.BaseSimTimeMs, *NewMontageState.Montage->GetName(), StartPosition, NewMontageState.PlayRate, PlaySeconds);
				#endif // !UE_BUILD_SHIPPING

				if (PlaySeconds > 0.0f)
				{
					SyncedMontageState = NewMontageState;	// only consider us sync'd if the montage actually started
				}
			}
		}
	}
}