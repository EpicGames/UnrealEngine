// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultMovementSet/CharacterMoverSimulationTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CharacterMoverSimulationTypes)

bool FFloorResultData::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	Super::NetSerialize(Ar, Map, bOutSuccess);

	bool bBlockingHit = (bool)FloorResult.bBlockingHit;
	bool bLineTrace = (bool)FloorResult.bLineTrace;
	bool bWalkableFloor = (bool)FloorResult.bWalkableFloor;
	Ar.SerializeBits(&bBlockingHit, 1);
	Ar.SerializeBits(&bLineTrace, 1);
	Ar.SerializeBits(&bWalkableFloor, 1);

	if (Ar.IsLoading())
	{
		FloorResult.bBlockingHit = bBlockingHit;
		FloorResult.bLineTrace = bLineTrace;
		FloorResult.bWalkableFloor = bWalkableFloor;
	}

	Ar << FloorResult.FloorDist;
	FloorResult.HitResult.NetSerialize(Ar, Map, bOutSuccess);

	return true;
}

void FFloorResultData::ToString(FAnsiStringBuilderBase& Out) const
{
	Super::ToString(Out);

	Out.Appendf("bBlockingHit: %i | ", FloorResult.bBlockingHit);
	Out.Appendf("bLineTrace: %i | ", FloorResult.bLineTrace);
	Out.Appendf("bWalkableFloor: %i | ", FloorResult.bWalkableFloor);
	Out.Appendf("FloorDist: %.2f/n", FloorResult.FloorDist);
	Out.Appendf("HitResult: %s/n", *FloorResult.HitResult.ToString());
}

bool FFloorResultData::ShouldReconcile(const FMoverDataStructBase& AuthorityState) const
{
	return false;
}

void FFloorResultData::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct)
{
	if (Pct < 0.5f)
	{
		*this = static_cast<const FFloorResultData&>(From);
	}
	else
	{
		*this = static_cast<const FFloorResultData&>(To);
	}
}

void FFloorResultData::Merge(const FMoverDataStructBase& From)
{
}

void FFloorResultData::Decay(float DecayAmount)
{
}
