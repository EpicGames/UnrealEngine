// Copyright Epic Games, Inc. All Rights Reserved.


#include "DefaultMovementSet/InstantMovementEffects/BasicInstantMovementEffects.h"

#include "MoverComponent.h"
#include "MoverDataModelTypes.h"
#include "MoverSimulationTypes.h"
#include "MoverSimulation.h"
#include "DefaultMovementSet/Settings/CommonLegacyMovementSettings.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoveLibrary/MoverBlackboard.h"
#include "DrawDebugHelpers.h"

// -------------------------------------------------------------------
// FTeleportEffect
// -------------------------------------------------------------------

#include UE_INLINE_GENERATED_CPP_BY_NAME(BasicInstantMovementEffects)

static int32 ShowTeleportDiffs = 0;
static float ShowTeleportDiffsLifetimeSecs = 3.0f;
FAutoConsoleVariableRef CVarShowTeleportDiffs(
	TEXT("mover.debug.ShowTeleportDiffs"),
	ShowTeleportDiffs,
	TEXT("Whether to draw teleportation differences (red is initially blocked, green is corrected).\n")
	TEXT("0: Disable, 1: Enable"),
	ECVF_Cheat);


FTeleportEffect::FTeleportEffect()
	: TargetLocation(FVector::ZeroVector)
	, bUseActorRotation(true)
	, TargetRotation(FRotator::ZeroRotator)
{
}

bool FTeleportEffect::ApplyMovementEffect(FApplyMovementEffectParams& ApplyEffectParams, FMoverSyncState& OutputState)
{
	USceneComponent* UpdatedComponent = ApplyEffectParams.UpdatedComponent;
	const FRotator FinalTargetRotation = bUseActorRotation ? UpdatedComponent->GetComponentRotation() : TargetRotation;
	
	const FVector PreviousLocation = UpdatedComponent->GetComponentLocation();
	const FQuat PreviousRotation = UpdatedComponent->GetComponentQuat();
	AActor* OwnerActor = UpdatedComponent->GetOwner();

	if (OwnerActor->TeleportTo(TargetLocation, FinalTargetRotation))
	{
		const FVector UpdatedLocation = UpdatedComponent->GetComponentLocation();
#if !UE_BUILD_SHIPPING
		if (ShowTeleportDiffs)
		{
			if (!(UpdatedLocation - TargetLocation).IsNearlyZero())	// if it was adjusted, show the original error
			{
				DrawDebugCapsule(OwnerActor->GetWorld(), TargetLocation, OwnerActor->GetSimpleCollisionHalfHeight(), OwnerActor->GetSimpleCollisionRadius(), FQuat::Identity, FColor::Red, false, ShowTeleportDiffsLifetimeSecs);
			}
			DrawDebugCapsule(OwnerActor->GetWorld(), UpdatedLocation, OwnerActor->GetSimpleCollisionHalfHeight(), OwnerActor->GetSimpleCollisionRadius(), FQuat::Identity, FColor(100, 100, 255), false, ShowTeleportDiffsLifetimeSecs);
		}
#endif // !UE_BUILD_SHIPPING

		FMoverDefaultSyncState& OutputSyncState = OutputState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();
		OutputSyncState.SetTransforms_WorldSpace(UpdatedLocation,
													UpdatedComponent->GetComponentRotation(),
													OutputSyncState.GetVelocity_WorldSpace(),
													OutputSyncState.GetAngularVelocityDegrees_WorldSpace(),
													nullptr ); // no movement base
		
		// TODO: instead of invalidating it, consider checking for a floor. Possibly a dynamic base?
		if (UMoverBlackboard* SimBlackboard = ApplyEffectParams.MoverComp->GetSimBlackboard_Mutable())
		{
			SimBlackboard->Invalidate(CommonBlackboard::LastFloorResult);
			SimBlackboard->Invalidate(CommonBlackboard::LastFoundDynamicMovementBase);
		}

		ApplyEffectParams.OutputEvents.Add(MakeShared<FTeleportSucceededEventData>(*ApplyEffectParams.TimeStep, PreviousLocation, PreviousRotation, TargetLocation, FQuat(FinalTargetRotation)));

		return true;
	}

#if !UE_BUILD_SHIPPING
	if (ShowTeleportDiffs)
	{
		DrawDebugCapsule(OwnerActor->GetWorld(), TargetLocation, OwnerActor->GetSimpleCollisionHalfHeight(), OwnerActor->GetSimpleCollisionRadius(), FQuat::Identity, FColor::Red, false, ShowTeleportDiffsLifetimeSecs);
	}
#endif // !UE_BUILD_SHIPPING

	ApplyEffectParams.OutputEvents.Add(MakeShared<FTeleportFailedEventData>(*ApplyEffectParams.TimeStep, PreviousLocation, PreviousRotation, TargetLocation, FQuat(FinalTargetRotation), ETeleportFailureReason::Reason_NotAvailable));

	return false;
}

bool FTeleportEffect::ApplyMovementEffect_Async(FApplyMovementEffectParams_Async& ApplyEffectParams, FMoverSyncState& OutputState)
{
	if (ApplyEffectParams.Simulation)
	{
		ApplyEffectParams.Simulation->AttemptTeleport(*ApplyEffectParams.TimeStep, FTransform(TargetRotation, TargetLocation), bUseActorRotation, OutputState);
		return true;
	}

	return false;
}

FInstantMovementEffect* FTeleportEffect::Clone() const
{
	FTeleportEffect* CopyPtr = new FTeleportEffect(*this);
	return CopyPtr;
}

void FTeleportEffect::NetSerialize(FArchive& Ar)
{
	Super::NetSerialize(Ar);

	Ar << TargetLocation;
	
	Ar.SerializeBits(&bUseActorRotation, 1);
	if (!bUseActorRotation)
	{
		Ar << TargetRotation;
	}
}

UScriptStruct* FTeleportEffect::GetScriptStruct() const
{
	return FTeleportEffect::StaticStruct();
}

FString FTeleportEffect::ToSimpleString() const
{
	return bUseActorRotation ? FString::Printf(TEXT("Teleport to %s (bUseActorRotation = True)"), *TargetLocation.ToString()) : FString::Printf(TEXT("Teleport to %s, %s (bUseActorRotation = False)"), *TargetLocation.ToString(), *FRotator(TargetRotation).ToString());
}

void FTeleportEffect::AddReferencedObjects(FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);
}


// -------------------------------------------------------------------
// FAsyncTeleportEffect
// -------------------------------------------------------------------

bool FAsyncTeleportEffect::ApplyMovementEffect(FApplyMovementEffectParams& ApplyEffectParams, FMoverSyncState& OutputState)
{
	FVector TeleportLocation = TargetLocation;
	const FRotator TeleportRotation = bUseActorRotation ? ApplyEffectParams.UpdatedComponent->GetComponentRotation() : TargetRotation;

	if (UMovementUtils::FindTeleportSpot(ApplyEffectParams.MoverComp, OUT TeleportLocation, TeleportRotation))
	{
		if (ShowTeleportDiffs)
		{
			const AActor* OwnerActor = ApplyEffectParams.UpdatedComponent->GetOwner();

			if (!(TeleportLocation - TargetLocation).IsNearlyZero())	// if it was adjusted, show the original error
			{
				DrawDebugCapsule(OwnerActor->GetWorld(), TargetLocation, OwnerActor->GetSimpleCollisionHalfHeight(), OwnerActor->GetSimpleCollisionRadius(), FQuat::Identity, FColor::Red, false, ShowTeleportDiffsLifetimeSecs);
			}

			DrawDebugCapsule(OwnerActor->GetWorld(), TeleportLocation, OwnerActor->GetSimpleCollisionHalfHeight(), OwnerActor->GetSimpleCollisionRadius(), FQuat::Identity, FColor(100, 100, 255), false, ShowTeleportDiffsLifetimeSecs);
		}

		FMoverDefaultSyncState& OutputSyncState = OutputState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();

		if (const FMoverDefaultSyncState* StartingSyncState = ApplyEffectParams.StartState->SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>())
		{
			OutputSyncState.SetTransforms_WorldSpace(TeleportLocation,
				TeleportRotation,
				OutputSyncState.GetVelocity_WorldSpace(),
				OutputSyncState.GetAngularVelocityDegrees_WorldSpace(),
				nullptr); // no movement base

			// TODO: instead of invalidating it, consider checking for a floor. Possibly a dynamic base?
			if (UMoverBlackboard* SimBlackboard = ApplyEffectParams.MoverComp->GetSimBlackboard_Mutable())
			{
				SimBlackboard->Invalidate(CommonBlackboard::LastFloorResult);
				SimBlackboard->Invalidate(CommonBlackboard::LastFoundDynamicMovementBase);
			}

			return true;
		}
	}


	if (ShowTeleportDiffs)
	{
		const AActor* OwnerActor = ApplyEffectParams.UpdatedComponent->GetOwner();
		DrawDebugCapsule(OwnerActor->GetWorld(), TargetLocation, OwnerActor->GetSimpleCollisionHalfHeight(), OwnerActor->GetSimpleCollisionRadius(), FQuat::Identity, FColor::Red, false, ShowTeleportDiffsLifetimeSecs);
	}

	return false;
}

FInstantMovementEffect* FAsyncTeleportEffect::Clone() const
{
	FAsyncTeleportEffect* CopyPtr = new FAsyncTeleportEffect(*this);
	return CopyPtr;
}

UScriptStruct* FAsyncTeleportEffect::GetScriptStruct() const
{
	return FAsyncTeleportEffect::StaticStruct();
}

FString FAsyncTeleportEffect::ToSimpleString() const
{
	return FString::Printf(TEXT("Async Teleport"));
}


// -------------------------------------------------------------------
// FJumpImpulseEffect
// -------------------------------------------------------------------

FJumpImpulseEffect::FJumpImpulseEffect()
	: UpwardsSpeed(0.f)
{
}

bool FJumpImpulseEffect::ApplyMovementEffect(FApplyMovementEffectParams& ApplyEffectParams, FMoverSyncState& OutputState)
{
	if (const FMoverDefaultSyncState* SyncState = ApplyEffectParams.StartState->SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>())
	{
		FMoverDefaultSyncState& OutputSyncState = OutputState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();
		
		// If we're leaving a movement base, include its current velocity in our initial airborne velocity
		FVector MovementBaseVelocityWS;
		UBasedMovementUtils::GetMovementBaseVelocityAtPoint(SyncState->GetMovementBase(), ApplyEffectParams.UpdatedComponent->GetComponentLocation(), OUT MovementBaseVelocityWS);

		const FVector UpDir = ApplyEffectParams.MoverComp->GetUpDirection();
		const FVector ImpulseVelocity = UpDir * UpwardsSpeed;
	
		// Jump impulse overrides vertical velocity while maintaining the rest
		const FVector PriorVelocityWS = SyncState->GetVelocity_WorldSpace();
		const FVector StartingNonUpwardsVelocity = PriorVelocityWS - PriorVelocityWS.ProjectOnToNormal(UpDir);

		if (const UCommonLegacyMovementSettings* CommonSettings = ApplyEffectParams.MoverComp->FindSharedSettings<UCommonLegacyMovementSettings>())
		{
			OutputState.MovementMode = CommonSettings->AirMovementModeName;
		}
		
		FRelativeBaseInfo MovementBaseInfo;
		if (const UMoverBlackboard* SimBlackboard = ApplyEffectParams.MoverComp->GetSimBlackboard())
		{
			SimBlackboard->TryGet(CommonBlackboard::LastFoundDynamicMovementBase, MovementBaseInfo);
		}

		const FVector FinalVelocity = StartingNonUpwardsVelocity + ImpulseVelocity + MovementBaseVelocityWS;
		
		OutputSyncState.SetTransforms_WorldSpace( ApplyEffectParams.UpdatedComponent->GetComponentLocation(),
												  ApplyEffectParams.UpdatedComponent->GetComponentRotation(),
												  FinalVelocity,
												  FVector::ZeroVector,
												  MovementBaseInfo.MovementBase.Get(),
												  MovementBaseInfo.BoneName);
		
		ApplyEffectParams.UpdatedComponent->ComponentVelocity = FinalVelocity;
		
		return true;
	}

	return false;
}

FInstantMovementEffect* FJumpImpulseEffect::Clone() const
{
	FJumpImpulseEffect* CopyPtr = new FJumpImpulseEffect(*this);
	return CopyPtr;
}

void FJumpImpulseEffect::NetSerialize(FArchive& Ar)
{
	Super::NetSerialize(Ar);

	Ar << UpwardsSpeed;
}

UScriptStruct* FJumpImpulseEffect::GetScriptStruct() const
{
	return FJumpImpulseEffect::StaticStruct();
}

FString FJumpImpulseEffect::ToSimpleString() const
{
	return FString::Printf(TEXT("JumpImpulse"));
}

void FJumpImpulseEffect::AddReferencedObjects(FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);
}

// -------------------------------------------------------------------
// FApplyVelocityEffect
// -------------------------------------------------------------------

FApplyVelocityEffect::FApplyVelocityEffect()
	: VelocityToApply(FVector::ZeroVector)
	, bAdditiveVelocity(false)
	, ForceMovementMode(NAME_None)
{
}

bool FApplyVelocityEffect::ApplyMovementEffect(FApplyMovementEffectParams& ApplyEffectParams, FMoverSyncState& OutputState)
{
	FMoverDefaultSyncState& OutputSyncState = OutputState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();
	
	OutputState.MovementMode = ForceMovementMode;
	
	FRelativeBaseInfo MovementBaseInfo;
	if (const UMoverBlackboard* SimBlackboard = ApplyEffectParams.MoverComp->GetSimBlackboard())
	{
		SimBlackboard->TryGet(CommonBlackboard::LastFoundDynamicMovementBase, MovementBaseInfo);
	}

	FVector Velocity = VelocityToApply;
	if (bAdditiveVelocity)
	{
		if (const FMoverDefaultSyncState* SyncState = ApplyEffectParams.StartState->SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>())
		{
			Velocity += SyncState->GetVelocity_WorldSpace();
		}
	}

	OutputSyncState.SetTransforms_WorldSpace( ApplyEffectParams.UpdatedComponent->GetComponentLocation(),
											  ApplyEffectParams.UpdatedComponent->GetComponentRotation(),
											  Velocity,
											  FVector::ZeroVector,
											  MovementBaseInfo.MovementBase.Get(),
											  MovementBaseInfo.BoneName);

	ApplyEffectParams.UpdatedComponent->ComponentVelocity = Velocity;
	
	return true;
}

FInstantMovementEffect* FApplyVelocityEffect::Clone() const
{
	FApplyVelocityEffect* CopyPtr = new FApplyVelocityEffect(*this);
	return CopyPtr;
}

void FApplyVelocityEffect::NetSerialize(FArchive& Ar)
{
	Super::NetSerialize(Ar);

	SerializePackedVector<10, 16>(VelocityToApply, Ar);

	Ar << bAdditiveVelocity;
	
	bool bUsingForcedMovementMode = !ForceMovementMode.IsNone();
	Ar.SerializeBits(&bUsingForcedMovementMode, 1);

	if (bUsingForcedMovementMode)
	{
		Ar << ForceMovementMode;
	}
}

UScriptStruct* FApplyVelocityEffect::GetScriptStruct() const
{
	return FApplyVelocityEffect::StaticStruct();
}

FString FApplyVelocityEffect::ToSimpleString() const
{
	return FString::Printf(TEXT("ApplyVelocity"));
}

void FApplyVelocityEffect::AddReferencedObjects(FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);
}
