// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultMovementSet/LayeredMoves/AnimRootMotionLayeredMove.h"
#include "MoverComponent.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimMontage.h"
#include "Components/SkeletalMeshComponent.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoverTypes.h"
#include "MoverLog.h"
#include "DefaultMovementSet/MoverMontageSimulationTypes.h"
#include "MoveLibrary/MoverBlackboard.h"
#include "MotionWarpingComponent.h"
#include "RootMotionModifier_SkewWarp.h"
#include "AnimNotifyState_MotionWarping.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimRootMotionLayeredMove)

UE_DEFINE_GAMEPLAY_TAG_COMMENT(Mover_AnimRootMotion_Montage, "Mover.AnimRootMotion.Montage", "Signifies an association with root motion that comes via a montage");

#if !UE_BUILD_SHIPPING
FAutoConsoleVariable CVarLogAnimRootMotionSteps(
	TEXT("mover.debug.LogAnimRootMotionSteps"),
	false,
	TEXT("Whether to log detailed information about anim root motion layered moves. 0: Disable, 1: Enable"),
	ECVF_Cheat);
#endif	// !UE_BUILD_SHIPPING


// ----------------------------------------------------------------------------
// FLayeredMove_AnimRootMotion
// ----------------------------------------------------------------------------

FLayeredMove_AnimRootMotion::FLayeredMove_AnimRootMotion()
{
	DurationMs = 0.f;
	MixMode = EMoveMixMode::OverrideAll;
}

bool FLayeredMove_AnimRootMotion::GenerateMove(const FMoverTickStartData& SimState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove)
{
	float MeshMontagePos = -1.f;
	// Stop this move if the montage is no longer playing on the mesh
	if (!TimeStep.bIsResimulating)
	{
		bool bIsMontageStillPlaying = false;

		if (const USkeletalMeshComponent* MeshComp = Cast<USkeletalMeshComponent>(MoverComp->GetPrimaryVisualComponent()))
		{
			if (const UAnimInstance* MeshAnimInstance = MeshComp->GetAnimInstance())
			{
				bIsMontageStillPlaying = MontageState.Montage && MeshAnimInstance->Montage_IsPlaying(MontageState.Montage);
				if (bIsMontageStillPlaying)
				{
					MeshMontagePos = MeshAnimInstance->Montage_GetPosition(MontageState.Montage);
				}
			}
		}

		if (!bIsMontageStillPlaying)
		{
			DurationMs = 0.f;
			return false;
		}
	}

	const float DeltaSeconds = TimeStep.StepMs / 1000.f;

	const FMoverDefaultSyncState* SyncState = SimState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();

	if (SyncState == nullptr)
	{
		return false;
	}

	// First pass simply samples based on the duration. For long animations, this has the potential to diverge.
	// Future improvements could include:
	//     - speeding up or slowing down slightly to match the associated montage instance
	//     - detecting if the montage instance is interrupted and attempting to interrupt and scheduling this move to end at the same sim time
	const float MontageRateScale = (MontageState.Montage ? MontageState.Montage->RateScale : 1.0f);

	// Note that Montage 'position' equates to seconds when PlayRate is 1
	const double SecondsSinceMontageStarted = (TimeStep.BaseSimTimeMs - StartSimTimeMs) / 1000.0;
	const double ScaledSecondsSinceMontageStarted = SecondsSinceMontageStarted * MontageState.PlayRate * MontageRateScale;

	const float ExtractionStartPosition = MontageState.StartingMontagePosition + ScaledSecondsSinceMontageStarted;
	const float ExtractionEndPosition   = ExtractionStartPosition + (DeltaSeconds * MontageState.PlayRate * MontageRateScale);

	// Read the local transform directly from the montage
	const FTransform LocalRootMotion = MontageState.Montage ? UMotionWarpingUtilities::ExtractRootMotionFromAnimation(MontageState.Montage, ExtractionStartPosition, ExtractionEndPosition) : FTransform::Identity;

	FMotionWarpingUpdateContext WarpingContext;
	WarpingContext.Animation = MontageState.Montage;
	WarpingContext.CurrentPosition = ExtractionEndPosition;
	WarpingContext.PreviousPosition = ExtractionStartPosition;
	WarpingContext.PlayRate = MontageState.PlayRate * MontageRateScale;
	WarpingContext.Weight = 1.f;

	// Note that we're forcing the use of the sync state's actor transform data. This is necessary when the movement simulation
	// is running ahead of the actor's visual representation and may be rotated differently, such as in an async physics sim.
	const FTransform SimActorTransform = FTransform(SyncState->GetOrientation_WorldSpace().Quaternion(), SyncState->GetLocation_WorldSpace());
	const FTransform WorldSpaceRootMotion = MoverComp->ConvertLocalRootMotionToWorld(LocalRootMotion, DeltaSeconds, &SimActorTransform, &WarpingContext);

	OutProposedMove = FProposedMove();
	OutProposedMove.MixMode = MixMode;

	// Convert the transform into linear and angular velocities
	if (DeltaSeconds > UE_KINDA_SMALL_NUMBER)
	{
		OutProposedMove.LinearVelocity    = WorldSpaceRootMotion.GetTranslation() / DeltaSeconds;
		const FQuat ShortestArcRootRotation = WorldSpaceRootMotion.GetRotation().GetShortestArcWith(FQuat::Identity);
		OutProposedMove.AngularVelocityDegrees   = FMath::RadiansToDegrees(ShortestArcRootRotation.ToRotationVector() / DeltaSeconds);
	}

	MontageState.CurrentPosition = ExtractionEndPosition;

#if !UE_BUILD_SHIPPING
	UE_CLOGF(CVarLogAnimRootMotionSteps->GetBool(), LogMover, Log, "AnimRootMotion. SimF %i / SimT %.1f (Resim? %i) (dt %.3f) Range [%.3f, %.3f] => LocalT: %ls (WST: %ls)  Vel: %.3f  MntPos: %.3f",
	        TimeStep.ServerFrame, TimeStep.BaseSimTimeMs, TimeStep.bIsResimulating, DeltaSeconds, ExtractionStartPosition, ExtractionEndPosition,
	        *LocalRootMotion.GetTranslation().ToString(), *WorldSpaceRootMotion.GetTranslation().ToString(), OutProposedMove.LinearVelocity.Length(), MeshMontagePos);
#endif // !UE_BUILD_SHIPPING

	return true;
}

bool FLayeredMove_AnimRootMotion::HasGameplayTag(FGameplayTag TagToFind, bool bExactMatch) const
{
	static const FGameplayTag MontageAttributeTag = Mover_AnimRootMotion_Montage.GetTag();
	const bool bFoundMatch = bExactMatch ? MontageAttributeTag.MatchesTagExact(TagToFind) : MontageAttributeTag.MatchesTag(TagToFind);

	return bFoundMatch || Super::HasGameplayTag(TagToFind, bExactMatch);
}

FLayeredMoveBase* FLayeredMove_AnimRootMotion::Clone() const
{
	FLayeredMove_AnimRootMotion* CopyPtr = new FLayeredMove_AnimRootMotion(*this);
	return CopyPtr;
}

void FLayeredMove_AnimRootMotion::NetSerialize(FArchive& Ar)
{
	Super::NetSerialize(Ar);

	MontageState.NetSerialize(Ar);
}

UScriptStruct* FLayeredMove_AnimRootMotion::GetScriptStruct() const
{
	return FLayeredMove_AnimRootMotion::StaticStruct();
}

FString FLayeredMove_AnimRootMotion::ToSimpleString() const
{
	return FString::Printf(TEXT("AnimRootMotion"));
}

void FLayeredMove_AnimRootMotion::AddReferencedObjects(class FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);
}

FMoverAnimMontageState FLayeredMove_AnimRootMotion::GetMontageState() const
{
	return MontageState;
}


// ----------------------------------------------------------------------------
// FLayeredMove_AnimRootMotion_SimDriven
// ----------------------------------------------------------------------------

void FLayeredMove_AnimRootMotion_SimDriven::OnStart_Async(UMoverBlackboard* SimBlackboard, const FMoverTime& SimTime)
{
	StartMoverTime = SimTime;
}

bool FLayeredMove_AnimRootMotion_SimDriven::GenerateMove_Async(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove)
{
	// In async we cannot poll the AnimInstance. Instead, bIsFinished tracks when we have
	// detected that the montage position reached the end of a non-looping montage.
	if (!TimeStep.bIsResimulating && bIsFinished)
	{
		DurationMs = 0.f;
		return false;
	}

	const float DeltaSeconds = TimeStep.StepMs * 0.001f;

	const FMoverDefaultSyncState* SyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();

	if (SyncState == nullptr)
	{
		return false;
	}

	if (!MontageState.Montage)
	{
		return false;
	}

	// Compute elapsed simulation time since the montage started using frame-based arithmetic
	// when possible. Note that Montage 'position' equates to seconds when PlayRate is 1.
	const float MontageRateScale = (MontageState.Montage ? MontageState.Montage->RateScale : 1.0f);

	const double SecondsSinceMontageStarted = StartMoverTime.ElapsedSecondsTo(TimeStep.ToStartTime(), TimeStep.StepMs * 0.001);
	const double ScaledSecondsSinceMontageStarted = SecondsSinceMontageStarted * MontageState.PlayRate * MontageRateScale;

	const float ExtractionStartPosition = MontageState.StartingMontagePosition + ScaledSecondsSinceMontageStarted;
	const float ExtractionEndPosition = ExtractionStartPosition + (DeltaSeconds * MontageState.PlayRate * MontageRateScale);

	// Read the local transform directly from the montage. Non-const so warping can modify it in place.
	FTransform LocalRootMotion = UMotionWarpingUtilities::ExtractRootMotionFromAnimation(MontageState.Montage, ExtractionStartPosition, ExtractionEndPosition);

	// Note that we're forcing the use of the sync state's actor transform data. This is necessary when the movement simulation
	// is running ahead of the actor's visual representation and may be rotated differently, such as in an async physics sim.
	const FTransform SimActorTransform = FTransform(SyncState->GetOrientation_WorldSpace().Quaternion(), SyncState->GetLocation_WorldSpace());

	// Adaptation of UCharacterMovementComponent::ConvertLocalRootMotionToWorld.
	// Root motion extracted from the montage is in mesh-component-local space. To convert it to world
	// space we need the component-to-world transform, which is:
	//   ComponentTransform = PrimaryVisualComponentRelativeTransform * SimActorTransform
	// where PrimaryVisualComponentRelativeTransform is the mesh component's fixed offset relative to
	// the actor root (equivalent to BaseVisualComponentTransform on UMoverComponent), marshalled from
	// the game thread via FChaosMoverSimulationDefaultInputs and written to the blackboard each substep.
	FTransform PrimaryVisualCompRelativeTransform = FTransform::Identity;
	if (SimBlackboard)
	{
		SimBlackboard->TryGet(AnimRootMotionBlackboard::LastPrimaryVisualComponentRelativeTransform, PrimaryVisualCompRelativeTransform);
	}
	const FTransform ComponentTransform = PrimaryVisualCompRelativeTransform * SimActorTransform;

	// Apply motion warping: skew-warp the local root motion translation toward any active warp targets.
	// Warp targets are snapshotted from UMotionWarpingComponent each frame by the backend and written
	// to the blackboard by ChaosMoverStateMachine: no game-thread UObjects are touched here.
	// We read WarpTargetName from the animation notify's modifier (a stable, read-only asset sub-object).
	TArray<FMoverResolvedWarpTarget> ResolvedWarpTargets;
	if (SimBlackboard)
	{
		SimBlackboard->TryGet(AnimRootMotionBlackboard::LastResolvedMotionWarpTargets, ResolvedWarpTargets);
	}
	if (!ResolvedWarpTargets.IsEmpty() && MontageState.Montage && !LocalRootMotion.GetTranslation().IsNearlyZero())
	{
		TArray<FMotionWarpingWindowData> WarpWindows;
		UMotionWarpingUtilities::GetMotionWarpingWindowsFromAnimation(MontageState.Montage, WarpWindows);

		for (const FMotionWarpingWindowData& Window : WarpWindows)
		{
			// Skip if the current frame is entirely outside this warp window
			if (ExtractionStartPosition >= Window.EndTime || ExtractionEndPosition <= Window.StartTime)
			{
				continue;
			}

			// Resolve the warp target name from the notify's root motion modifier.
			// AnimNotifyState and its instanced modifier are stable asset sub-objects: safe to read on a worker thread.
			FName WarpTargetName = NAME_None;
			if (Window.AnimNotify && Window.AnimNotify->RootMotionModifier)
			{
				if (const URootMotionModifier_Warp* WarpModifier = Cast<URootMotionModifier_Warp>(Window.AnimNotify->RootMotionModifier))
				{
					WarpTargetName = WarpModifier->WarpTargetName;
				}
			}
			if (WarpTargetName.IsNone())
			{
				continue;
			}

			// Look up the GT-snapshotted target
			const FMoverResolvedWarpTarget* ResolvedTarget = ResolvedWarpTargets.FindByPredicate(
				[&WarpTargetName](const FMoverResolvedWarpTarget& T) { return T.Name == WarpTargetName; });
			if (!ResolvedTarget)
			{
				continue;
			}

			// Total remaining root motion from the substep start to the window end. Matches RootMotionTotal
			// in URootMotionModifier_SkewWarp::ProcessRootMotion.
			const FTransform TotalRootMotion = UMotionWarpingUtilities::ExtractRootMotionFromAnimation(
				MontageState.Montage, ExtractionStartPosition, Window.EndTime);
			const FVector TotalTranslation = TotalRootMotion.GetTranslation();

			if (!TotalTranslation.IsNearlyZero())
			{
				// Convert the world-space warp target location into mesh-component local space:
				// the same space in which LocalRootMotion is expressed.
				const FVector LocalTargetLocation = ComponentTransform.InverseTransformPositionNoScale(ResolvedTarget->Location);

				// Mirrors URootMotionModifier_SkewWarp::ProcessRootMotion: clamp delta to the window boundary
				// and accumulate post-window remainder as unwarped PostWarpingWindowRootMotion. Without clamping,
				// when the substep straddles the window end InWarpingWindowRootMotion > TotalTranslation, which
				// inflates ProjectedScale in WarpTranslation and produces arbitrarily large output translations.
				// When straddling, InWarpingWindowRootMotion == TotalTranslation so we reuse that result directly.
				const bool bStraddles = ExtractionEndPosition > Window.EndTime;
				const FVector InWarpingWindowRootMotion = bStraddles ? TotalTranslation : LocalRootMotion.GetTranslation();
				const FVector PostWarpingWindowRootMotion = bStraddles
					? UMotionWarpingUtilities::ExtractRootMotionFromAnimation(MontageState.Montage, Window.EndTime, ExtractionEndPosition).GetTranslation()
					: FVector::ZeroVector;

				// WarpTranslation is a public static on URootMotionModifier_SkewWarp: no instance needed.
				const FVector WarpedTranslation = URootMotionModifier_SkewWarp::WarpTranslation(
					FTransform::Identity, InWarpingWindowRootMotion, TotalTranslation, LocalTargetLocation)
					+ PostWarpingWindowRootMotion;

				// Intentional stomp: matches UMotionWarpingComponent's last-modifier-wins behavior. Each modifier in
				// UMotionWarpingComponent independently re-extracts InWarpingWindowRootMotion from the montage and
				// overwrites the result via SetTranslation, so if two windows overlap the last one wins.
				// InWarpingWindowRootMotion above is likewise extracted fresh from the montage each iteration,
				// not from the previous window's warped output.
				LocalRootMotion.SetTranslation(WarpedTranslation);
			}
		}
	}

	const FTransform ActorToWorld = SimActorTransform;
	const FTransform ActorToComponent = ActorToWorld.GetRelativeTransform(ComponentTransform);
	const FTransform NewComponentToWorld = LocalRootMotion * ComponentTransform;
	const FTransform NewActorTransform = ActorToComponent * NewComponentToWorld;
	const FVector DeltaWorldTranslation = NewActorTransform.GetTranslation() - ActorToWorld.GetTranslation();
	const FQuat NewWorldRotation = ComponentTransform.GetRotation() * LocalRootMotion.GetRotation();
	const FQuat DeltaWorldRotation = NewWorldRotation * ComponentTransform.GetRotation().Inverse();
	const FTransform WorldSpaceRootMotion(DeltaWorldRotation, DeltaWorldTranslation);

	OutProposedMove = FProposedMove();
	OutProposedMove.MixMode = MixMode;

	// Convert the transform into linear and angular velocities
	if (DeltaSeconds > UE_KINDA_SMALL_NUMBER)
	{
		OutProposedMove.LinearVelocity = WorldSpaceRootMotion.GetTranslation() / DeltaSeconds;
		const FQuat ShortestArcRootRotation = WorldSpaceRootMotion.GetRotation().GetShortestArcWith(FQuat::Identity);
		OutProposedMove.AngularVelocityDegrees = FMath::RadiansToDegrees(ShortestArcRootRotation.ToRotationVector() / DeltaSeconds);
	}

	// Clamp the end position to the valid montage range so that on the final substep we do not
	// write a position past the end of the asset into MontageState (which the GT would then use
	// as a start position when reconstructing the montage on a sim proxy).
	const float PlayLength = MontageState.Montage ? MontageState.Montage->GetPlayLength() : 0.f;
	const float ClampedEndPosition = (MontageState.PlayRate >= 0.f)
		? FMath::Min(ExtractionEndPosition, PlayLength)
		: FMath::Max(ExtractionEndPosition, 0.f);

	MontageState.CurrentPosition = ClampedEndPosition;

	// Blend-out and finish detection: direction-aware. Skipped when auto blend-out is disabled.
	if (MontageState.bEnableAutoBlendOut && MontageState.Montage)
	{
		if (MontageState.PlayRate >= 0.f)
		{
			// Forward playback: position increases toward PlayLength
			if (!bShouldBlendOut && MontageState.BlendOutTimeSeconds > 0.f)
			{
				const float TriggerPos = PlayLength - MontageState.BlendOutTimeSeconds;
				if (ClampedEndPosition >= TriggerPos)
				{
					bShouldBlendOut        = true;
					BlendOutServerFrame = TimeStep.ServerFrame;
				}
			}
			if (!bIsFinished && ClampedEndPosition >= PlayLength)
			{
				bIsFinished         = true;
				FinishServerFrame = TimeStep.ServerFrame;
			}
		}
		else
		{
			// Reverse playback: position decreases toward 0
			if (!bShouldBlendOut && MontageState.BlendOutTimeSeconds > 0.f)
			{
				const float TriggerPos = MontageState.BlendOutTimeSeconds;
				if (ClampedEndPosition <= TriggerPos)
				{
					bShouldBlendOut        = true;
					BlendOutServerFrame = TimeStep.ServerFrame;
				}
			}
			if (!bIsFinished && ClampedEndPosition <= 0.f)
			{
				bIsFinished         = true;
				FinishServerFrame = TimeStep.ServerFrame;
			}
		}
	}

	// DurationMs-based expiry: sets blend-out and finish flags even when bEnableAutoBlendOut
	// is false. Without this, IsFinished() removes the move from FlushMoveArrays before
	// bShouldBlendOut is ever set, causing the game thread to hard-stop the montage
	// instead of blending it out. Sticky-flag guards make this a no-op when the position-based
	// path above already fired.
	if (DurationMs >= 0.f && MontageState.Montage)
	{
		const double StepEndTimeMs = TimeStep.BaseSimTimeMs + TimeStep.StepMs;
		const double MoveEndTimeMs = StartSimTimeMs + (double)DurationMs;

		if (!bShouldBlendOut && MontageState.BlendOutTimeSeconds > 0.f
			&& StepEndTimeMs >= MoveEndTimeMs - MontageState.BlendOutTimeSeconds * 1000.0)
		{
			bShouldBlendOut     = true;
			BlendOutServerFrame = TimeStep.ServerFrame;
		}
		if (!bIsFinished && StepEndTimeMs >= MoveEndTimeMs)
		{
			bIsFinished       = true;
			FinishServerFrame = TimeStep.ServerFrame;
		}
	}

	if (bShouldBlendOut)
	{
		OutProposedMove.MixMode = EMoveMixMode::OverrideAllExceptVerticalVelocity;
	}

#if !UE_BUILD_SHIPPING
	UE_CLOG(CVarLogAnimRootMotionSteps->GetBool(), LogMover, Log, TEXT("AnimRootMotion_SimDriven. SimF %i (dt %.3f) Range [%.3f, %.3f] => LocalT: %s (WST: %s)  Vel: %.3f"),
		TimeStep.ServerFrame, DeltaSeconds, ExtractionStartPosition, ExtractionEndPosition,
		*LocalRootMotion.GetTranslation().ToString(), *WorldSpaceRootMotion.GetTranslation().ToString(), OutProposedMove.LinearVelocity.Length());
#endif // !UE_BUILD_SHIPPING

	return true;
}

FLayeredMoveBase* FLayeredMove_AnimRootMotion_SimDriven::Clone() const
{
	FLayeredMove_AnimRootMotion_SimDriven* CopyPtr = new FLayeredMove_AnimRootMotion_SimDriven(*this);
	return CopyPtr;
}

void FLayeredMove_AnimRootMotion_SimDriven::NetSerialize(FArchive& Ar)
{
	Super::NetSerialize(Ar);

	// Async lifecycle flags must be serialized so that server corrections can authoritatively
	// restore the move's state before the client resimulates. Without these, a client that
	// finished or started blending out the move earlier than the server (e.g. due to warp target
	// discrepancy) will resimulate with a dead move, producing no root motion and triggering a
	// spurious Falling mode transition.
	uint8 bFinished = bIsFinished    ? 1 : 0;
	uint8 bBlendOut = bShouldBlendOut ? 1 : 0;
	Ar.SerializeBits(&bFinished, 1);
	Ar.SerializeBits(&bBlendOut, 1);
	if (Ar.IsLoading())
	{
		bIsFinished     = (bFinished != 0);
		bShouldBlendOut = (bBlendOut != 0);
	}

	// Server frame timestamps: only meaningful when the corresponding flag is set.
	if (bBlendOut)
	{
		Ar << BlendOutServerFrame;
	}
	else if (Ar.IsLoading())
	{
		BlendOutServerFrame = INDEX_NONE;
	}

	if (bFinished)
	{
		Ar << FinishServerFrame;
	}
	else if (Ar.IsLoading())
	{
		FinishServerFrame = INDEX_NONE;
	}

	// Serialize StartMoverTime so that GenerateMove_Async can compute ElapsedSecondsTo correctly
	// after a server correction. TimeMs is already covered by the base-class StartSimTimeMs field;
	// only FrameCount needs to travel separately (4 bytes). On load, TimeMs is taken from
	// StartSimTimeMs to keep both fields in sync without redundant wire cost.
	// Without this, StartMoverTime stays at its USTRUCT default {FrameCount=INDEX_NONE, TimeMs=-1.0}
	// after deserialization, causing ElapsedSecondsTo to return ~CurrentSessionTimeSeconds and
	// ExtractionStartPosition to overshoot PlayLength on the first resim substep, immediately
	// setting bIsFinished = true and hard-killing the move.
	Ar << StartMoverTime.FrameCount;
	if (Ar.IsLoading())
	{
		StartMoverTime.TimeMs = StartSimTimeMs;
	}
}

UScriptStruct* FLayeredMove_AnimRootMotion_SimDriven::GetScriptStruct() const
{
	return FLayeredMove_AnimRootMotion_SimDriven::StaticStruct();
}

FString FLayeredMove_AnimRootMotion_SimDriven::ToSimpleString() const
{
	return FString::Printf(TEXT("AnimRootMotion_SimDriven"));
}

void FLayeredMove_AnimRootMotion_SimDriven::AppendMontageOutputEntry(TArray<FMoverSimDrivenMontageEntry>& OutEntries, const FMoverTimeStep& TimeStep) const
{
	FMoverSimDrivenMontageEntry& Entry = OutEntries.AddDefaulted_GetRef();
	Entry.Montage             = MontageState.Montage;
	Entry.CurrentPosition     = MontageState.CurrentPosition;
	Entry.PlayRate            = MontageState.PlayRate;
	Entry.bShouldBlendOut     = bShouldBlendOut;
	Entry.BlendOutTimeSeconds = MontageState.BlendOutTimeSeconds;
	Entry.bIsFinished         = bIsFinished;
	// Reconstruct ms thresholds for FMoverSimDrivenMontageData::Interpolate from the stored server frames.
	// Not triggered: -1 (never fires). Triggered this substep: substep end time (fires at exact crossing).
	// Triggered a previous substep: 0 (always satisfies the >= condition).
	const float EndOfStepSimTimeMs = TimeStep.BaseSimTimeMs + TimeStep.StepMs;
	Entry.FrameSimTimeMs    = EndOfStepSimTimeMs;
	Entry.StartSimTimeMs    = (float)StartMoverTime.TimeMs;
	Entry.BlendOutSimTimeMs = bShouldBlendOut ? (BlendOutServerFrame == TimeStep.ServerFrame ? EndOfStepSimTimeMs : 0.f) : -1.f;
	Entry.FinishSimTimeMs   = bIsFinished     ? (FinishServerFrame   == TimeStep.ServerFrame ? EndOfStepSimTimeMs : 0.f) : -1.f;
#if !UE_BUILD_SHIPPING
	Entry.Debug_MontageName = MontageState.Debug_MontageName;
#endif
}
