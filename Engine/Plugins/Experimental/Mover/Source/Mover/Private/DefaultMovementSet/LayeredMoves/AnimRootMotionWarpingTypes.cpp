// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultMovementSet/LayeredMoves/AnimRootMotionWarpingTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimRootMotionWarpingTypes)

namespace AnimRootMotionBlackboard
{
	const FName LastPrimaryVisualComponentRelativeTransform = TEXT("LastVisualCompRelativeTransform");
	const FName LastResolvedMotionWarpTargets               = TEXT("LastResolvedMotionWarpTargets");
}

FMoverDataStructBase* FMoverMotionWarpingInputs::Clone() const
{
	return new FMoverMotionWarpingInputs(*this);
}

bool FMoverMotionWarpingInputs::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	uint16 NumTargets = static_cast<uint16>(WarpTargets.Num());
	Ar << NumTargets;
	if (Ar.IsLoading())
	{
		WarpTargets.SetNum(NumTargets);
	}
	for (FMoverResolvedWarpTarget& Target : WarpTargets)
	{
		Ar << Target.Name;
		Ar << Target.Location;
		Ar << Target.Rotation;
	}
	bOutSuccess = true;
	return true;
}

UScriptStruct* FMoverMotionWarpingInputs::GetScriptStruct() const
{
	return FMoverMotionWarpingInputs::StaticStruct();
}

bool FMoverMotionWarpingInputs::ShouldReconcile(const FMoverDataStructBase& AuthorityState) const
{
	const FMoverMotionWarpingInputs& Authority = static_cast<const FMoverMotionWarpingInputs&>(AuthorityState);
	if (WarpTargets.Num() != Authority.WarpTargets.Num())
	{
		return true;
	}
	for (int32 i = 0; i < WarpTargets.Num(); ++i)
	{
		if (WarpTargets[i].Name != Authority.WarpTargets[i].Name ||
		    !WarpTargets[i].Location.Equals(Authority.WarpTargets[i].Location, UE_KINDA_SMALL_NUMBER) ||
		    !WarpTargets[i].Rotation.Equals(Authority.WarpTargets[i].Rotation, UE_KINDA_SMALL_NUMBER))
		{
			return true;
		}
	}
	return false;
}

void FMoverMotionWarpingInputs::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct)
{
	// Warp targets are discrete point-in-time snapshots -- snap to whichever endpoint is closer.
	WarpTargets = (Pct < 0.5f)
		? static_cast<const FMoverMotionWarpingInputs&>(From).WarpTargets
		: static_cast<const FMoverMotionWarpingInputs&>(To).WarpTargets;
}

void FMoverMotionWarpingInputs::Merge(const FMoverDataStructBase& From)
{
	// Warp targets are a full snapshot each frame. If the current frame has no targets yet
	// (e.g. input not yet produced on this endpoint), fall back to the previous frame's targets.
	if (WarpTargets.IsEmpty())
	{
		WarpTargets = static_cast<const FMoverMotionWarpingInputs&>(From).WarpTargets;
	}
}
