// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultMovementSet/LayeredMoves/BasicLayeredMoves.h"
#include "MoverSimulationTypes.h"
#include "MoverComponent.h"
#include "Curves/CurveVector.h"
#include "Curves/CurveFloat.h"
#include "MoverLog.h"
#include "DefaultMovementSet/Settings/CommonLegacyMovementSettings.h"
#include "MoveLibrary/MovementUtils.h"


// -------------------------------------------------------------------
// FLayeredMove_LinearVelocity
// -------------------------------------------------------------------

#include UE_INLINE_GENERATED_CPP_BY_NAME(BasicLayeredMoves)

FLayeredMove_LinearVelocity::FLayeredMove_LinearVelocity()
	: Velocity(FVector::ZeroVector)
	, MagnitudeOverTime(nullptr)
	, SettingsFlags(0)
{
}

bool FLayeredMove_LinearVelocity::GenerateMove(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove)
{
	bool GeneratedAProposedMove = GenerateMove_Async(StartState, TimeStep, SimBlackboard, OutProposedMove);
	
	if (GeneratedAProposedMove && MagnitudeOverTime && DurationMs > 0)
	{
		const double TimeValue = FMath::Clamp((TimeStep.BaseSimTimeMs - StartSimTimeMs) / DurationMs, 0.0, 1.0);
		const float TimeFactor = MagnitudeOverTime->GetFloatValue(TimeValue);
		OutProposedMove.LinearVelocity *= TimeFactor;
	}

	return GeneratedAProposedMove;
}

bool FLayeredMove_LinearVelocity::GenerateMove_Async(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove)
{
	if (!ensureMsgf(IsInGameThread() || MagnitudeOverTime == nullptr, TEXT("MagnitudeOverTime is not thread safe and cannot be set in async mode")))
	{
		// Setting the duration to 0.0 to also end this move
		DurationMs = 0.0f;
		return false;
	}

	const FMoverDefaultSyncState* SyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(SyncState);

	const float DeltaSeconds = TimeStep.StepMs * 0.001f;

	// Convert starting velocity based on starting orientation, if settings call for it
	if (SettingsFlags & (uint8)ELayeredMove_ConstantVelocitySettingsFlags::VelocityStartRelative &&
		StartSimTimeMs == TimeStep.BaseSimTimeMs)
	{
		SettingsFlags &= ~(uint8)ELayeredMove_ConstantVelocitySettingsFlags::VelocityStartRelative;
		Velocity = SyncState->GetOrientation_WorldSpace().RotateVector(Velocity);
	}

	FVector VelocityThisFrame = Velocity;

	// Put velocity into world space
	if (SettingsFlags & (uint8)ELayeredMove_ConstantVelocitySettingsFlags::VelocityAlwaysRelative)
	{
		VelocityThisFrame = SyncState->GetOrientation_WorldSpace().RotateVector(Velocity);
	}

	OutProposedMove.LinearVelocity = VelocityThisFrame;

	return true;
}

FLayeredMoveBase* FLayeredMove_LinearVelocity::Clone() const
{
	FLayeredMove_LinearVelocity* CopyPtr = new FLayeredMove_LinearVelocity(*this);
	return CopyPtr;
}

void FLayeredMove_LinearVelocity::NetSerialize(FArchive& Ar)
{
	Super::NetSerialize(Ar);

	SerializePackedVector<10, 16>(Velocity, Ar);
	Ar << SettingsFlags;
	Ar << MagnitudeOverTime;
}

UScriptStruct* FLayeredMove_LinearVelocity::GetScriptStruct() const
{
	return FLayeredMove_LinearVelocity::StaticStruct();
}

FString FLayeredMove_LinearVelocity::ToSimpleString() const
{
	return FString::Printf(TEXT("LinearVelocity"));
}

void FLayeredMove_LinearVelocity::AddReferencedObjects(class FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);
}


// -------------------------------------------------------------------
// FLinearVelocityMoveData
// -------------------------------------------------------------------

void FLinearVelocityMoveData::ActivateFromContext(const FLayeredMoveActivationParams* ActivationParams)
{
	if (ActivationParams)
	{
		const FLinearVelocityMoveActivationParams* Params = static_cast<const FLinearVelocityMoveActivationParams*>(ActivationParams);
		DurationMs = Params->DurationMs;
		Velocity = Params->Velocity;
		SettingsFlags = Params->SettingsFlags;
	}
}

void FLinearVelocityMoveData::NetSerialize(FArchive& Ar)
{
	Super::NetSerialize(Ar);

	SerializePackedVector<10, 16>(Velocity, Ar);
	Ar << SettingsFlags;
}

// -------------------------------------------------------------------
// ULinearVelocityMoveLogic
// -------------------------------------------------------------------

ULinearVelocityMoveLogic::ULinearVelocityMoveLogic()
{
	MixMode = EMoveMixMode::AdditiveVelocity;
	InstancedDataStructType = FLinearVelocityMoveData::StaticStruct();
}

bool ULinearVelocityMoveLogic::GenerateMove_Implementation(const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, const FMoverTickStartData& StartState, FProposedMove& OutProposedMove)
{
	return GenerateMove_Async_Implementation(TimeStep, SimBlackboard, StartState, OutProposedMove);
}

bool ULinearVelocityMoveLogic::GenerateMove_Async_Implementation(const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, const FMoverTickStartData& StartState, FProposedMove& OutProposedMove)
{
	const FMoverDefaultSyncState* SyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(SyncState);

	FLinearVelocityMoveData& MoveData = AccessExecutionMoveData<FLinearVelocityMoveData>();

	// On the first tick, convert a start-relative velocity into world space and clear the flag.
	if (MoveData.SettingsFlags & (uint8)ELayeredMove_ConstantVelocitySettingsFlags::VelocityStartRelative &&
		MoveData.StartSimTimeMs == TimeStep.BaseSimTimeMs)
	{
		MoveData.SettingsFlags &= ~(uint8)ELayeredMove_ConstantVelocitySettingsFlags::VelocityStartRelative;
		MoveData.Velocity = SyncState->GetOrientation_WorldSpace().RotateVector(MoveData.Velocity);
	}

	FVector VelocityThisFrame = MoveData.Velocity;

	if (MoveData.SettingsFlags & (uint8)ELayeredMove_ConstantVelocitySettingsFlags::VelocityAlwaysRelative)
	{
		VelocityThisFrame = SyncState->GetOrientation_WorldSpace().RotateVector(MoveData.Velocity);
	}

	OutProposedMove.MixMode = MixMode;
	OutProposedMove.LinearVelocity = VelocityThisFrame;

	return true;
}


// -------------------------------------------------------------------
// FLayeredMove_JumpImpulseOverDuration
// -------------------------------------------------------------------

FLayeredMove_JumpImpulseOverDuration::FLayeredMove_JumpImpulseOverDuration() 
	: UpwardsSpeed(0.f)
{
	DurationMs = 100.f;
	MixMode = EMoveMixMode::OverrideVelocity;
}


bool FLayeredMove_JumpImpulseOverDuration::GenerateMove(const FMoverTickStartData& SimState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove)
{	
	const FMoverDefaultSyncState* SyncState = SimState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(SyncState);

	const FVector UpDir = MoverComp->GetUpDirection();

	const FVector ImpulseVelocity = UpDir * UpwardsSpeed;

	// Jump impulse overrides vertical velocity while maintaining the rest
	if (MixMode == EMoveMixMode::OverrideVelocity)
	{
		const FVector PriorVelocityWS = SyncState->GetVelocity_WorldSpace();
		const FVector StartingNonUpwardsVelocity = PriorVelocityWS - PriorVelocityWS.ProjectOnToNormal(UpDir);

		// If we're leaving a movement base, include its current velocity in our initial airborne velocity
		FVector MovementBaseVelocityWS;
		UBasedMovementUtils::GetMovementBaseVelocityAtPoint(SyncState->GetMovementBase(), MoverComp->GetUpdatedComponent()->GetComponentLocation(), OUT MovementBaseVelocityWS);

		OutProposedMove.LinearVelocity = StartingNonUpwardsVelocity + ImpulseVelocity + MovementBaseVelocityWS;
	}
	else
	{
		ensureMsgf(false, TEXT("JumpImpulse layered move only supports Override Velocity mix mode and was queued with a different mix mode. Layered move will do nothing."));
		return false;
	}

	return true;
}

FLayeredMoveBase* FLayeredMove_JumpImpulseOverDuration::Clone() const
{
	FLayeredMove_JumpImpulseOverDuration* CopyPtr = new FLayeredMove_JumpImpulseOverDuration(*this);
	return CopyPtr;
}

void FLayeredMove_JumpImpulseOverDuration::NetSerialize(FArchive& Ar)
{
	Super::NetSerialize(Ar);

	Ar << UpwardsSpeed;
}

UScriptStruct* FLayeredMove_JumpImpulseOverDuration::GetScriptStruct() const
{
	return FLayeredMove_JumpImpulseOverDuration::StaticStruct();
}

FString FLayeredMove_JumpImpulseOverDuration::ToSimpleString() const
{
	return FString::Printf(TEXT("JumpImpulseOverDuration"));
}

void FLayeredMove_JumpImpulseOverDuration::AddReferencedObjects(class FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);
}

// -------------------------------------------------------------------
// FLayeredMove_JumpTo
// -------------------------------------------------------------------

FLayeredMove_JumpTo::FLayeredMove_JumpTo()
	: JumpDistance(-1.0f)
	, JumpHeight(-1.0f)
	, bUseActorRotation(true)
	, JumpRotation(ForceInitToZero)
	, PathOffsetCurve(nullptr)
	, TimeMappingCurve(nullptr)
{
	DurationMs = 1.f;
	MixMode = EMoveMixMode::OverrideVelocity;
}

bool FLayeredMove_JumpTo::GenerateMove(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove)
{
	const FMoverDefaultSyncState* SyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(SyncState);

	if (DurationMs == 0)
	{
		ensureMsgf(false, TEXT("JumpTo expected a non-zero duration."));
		return false;
	}
	
	const float DeltaSeconds = TimeStep.StepMs / 1000.f;
	float CurrentTimeFraction = (TimeStep.BaseSimTimeMs - StartSimTimeMs) / DurationMs;
	float TargetTimeFraction = CurrentTimeFraction + DeltaSeconds;
	
	// If we're beyond specified duration, we need to re-map times so that
	// we continue our desired ending velocity
	if (TargetTimeFraction > 1.f)
	{
		float TimeFractionPastAllowable = TargetTimeFraction - 1.0f;
		TargetTimeFraction -= TimeFractionPastAllowable;
		CurrentTimeFraction -= TimeFractionPastAllowable;
	}

	float CurrentMoveFraction = CurrentTimeFraction;
	float TargetMoveFraction = TargetTimeFraction;

	if (TimeMappingCurve)
	{
		CurrentMoveFraction = EvaluateFloatCurveAtFraction(*TimeMappingCurve, CurrentMoveFraction);
		TargetMoveFraction  = EvaluateFloatCurveAtFraction(*TimeMappingCurve, TargetMoveFraction);
	}

	const FRotator Rotation = bUseActorRotation ? SyncState->GetOrientation_WorldSpace() : JumpRotation;
	const FVector CurrentRelativeLocation = GetRelativeLocation(CurrentMoveFraction, Rotation);
	const FVector TargetRelativeLocation = GetRelativeLocation(TargetMoveFraction, Rotation);

	OutProposedMove.LinearVelocity = (TargetRelativeLocation - CurrentRelativeLocation) / DeltaSeconds;;
	if (const TObjectPtr<const UCommonLegacyMovementSettings> CommonLegacySettings = MoverComp->FindSharedSettings<UCommonLegacyMovementSettings>())
	{
		OutProposedMove.PreferredMode = CommonLegacySettings->AirMovementModeName;
	}
	
	return true;
}

FLayeredMoveBase* FLayeredMove_JumpTo::Clone() const
{
	FLayeredMove_JumpTo* CopyPtr = new FLayeredMove_JumpTo(*this);
	return CopyPtr;
}

void FLayeredMove_JumpTo::NetSerialize(FArchive& Ar)
{
	Super::NetSerialize(Ar);

	Ar << JumpDistance;
	Ar << JumpHeight;
	Ar.SerializeBits(&bUseActorRotation, 1);

	if (!bUseActorRotation)
	{
		Ar << JumpRotation;
	}

	Ar << PathOffsetCurve;
	Ar << TimeMappingCurve;
}

UScriptStruct* FLayeredMove_JumpTo::GetScriptStruct() const
{
	return FLayeredMove_JumpTo::StaticStruct();
}

FString FLayeredMove_JumpTo::ToSimpleString() const
{
	return FString::Printf(TEXT("JumpTo"));
}

void FLayeredMove_JumpTo::AddReferencedObjects(FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);
}

FVector FLayeredMove_JumpTo::GetPathOffset(const float MoveFraction) const
{
	FVector PathOffset(FVector::ZeroVector);
	if (PathOffsetCurve)
	{
		// Calculate path offset
		float MinCurveTime(0.f);
		float MaxCurveTime(1.f);

		PathOffsetCurve->GetTimeRange(MinCurveTime, MaxCurveTime);
		PathOffset =  PathOffsetCurve->GetVectorValue(FMath::GetRangeValue(FVector2f(MinCurveTime, MaxCurveTime), MoveFraction));
	}
	else
	{
		// Default to "jump parabola", a simple x^2 shifted to be upside-down and shifted
		// to get [0,1] X (MoveFraction/Distance) mapping to [0,1] Y (height)
		// Height = -(2x-1)^2 + 1
		const float Phi = 2.f*MoveFraction - 1;
		const float Z = -(Phi*Phi) + 1;
		PathOffset.Z = Z;
	}

	// Scale Z offset to height. If Height < 0, we use direct path offset values
	if (JumpHeight >= 0.f)
	{
		PathOffset.Z *= JumpHeight;
	}

	return PathOffset;
}

float FLayeredMove_JumpTo::EvaluateFloatCurveAtFraction(const UCurveFloat& Curve, const float Fraction) const
{
	float MinCurveTime(0.f);
	float MaxCurveTime(1.f);

	Curve.GetTimeRange(MinCurveTime, MaxCurveTime);
	return Curve.GetFloatValue(FMath::GetRangeValue(FVector2f(MinCurveTime, MaxCurveTime), Fraction));
}

FVector FLayeredMove_JumpTo::GetRelativeLocation(float MoveFraction, const FRotator& Rotator) const
{
	// Given MoveFraction, what relative location should a character be at?
	FRotator FacingRotation(Rotator);
	FacingRotation.Pitch = 0.f; // By default we don't include pitch, but an option could be added if necessary

	const FVector RelativeLocationFacingSpace = FVector(MoveFraction * JumpDistance, 0.f, 0.f) + GetPathOffset(MoveFraction);

	return FacingRotation.RotateVector(RelativeLocationFacingSpace);
}

// -------------------------------------------------------------------
// FLayeredMove_MoveTo
// -------------------------------------------------------------------

FLayeredMove_MoveTo::FLayeredMove_MoveTo()
	: StartLocation(ForceInitToZero)
	, TargetLocation(ForceInitToZero)
	, bRestrictSpeedToExpected(false)
	, PathOffsetCurve(nullptr)
	, TimeMappingCurve(nullptr)
{
	DurationMs = 1000.f;
	MixMode = EMoveMixMode::OverrideVelocity;
}

FVector FLayeredMove_MoveTo::GetPathOffsetInWorldSpace(const float MoveFraction) const
{
	if (PathOffsetCurve)
	{
		float MinCurveTime(0.f);
		float MaxCurveTime(1.f);

		PathOffsetCurve->GetTimeRange(MinCurveTime, MaxCurveTime);
		
		// Calculate path offset
		const FVector PathOffsetInFacingSpace = PathOffsetCurve->GetVectorValue(FMath::GetRangeValue(FVector2f(MinCurveTime, MaxCurveTime), MoveFraction));;
		FRotator FacingRotation((TargetLocation-StartLocation).Rotation());
		FacingRotation.Pitch = 0.f; // By default we don't include pitch in the offset, but an option could be added if necessary
		return FacingRotation.RotateVector(PathOffsetInFacingSpace);
	}

	return FVector::ZeroVector;
}

float FLayeredMove_MoveTo::EvaluateFloatCurveAtFraction(const UCurveFloat& Curve, const float Fraction) const
{
	float MinCurveTime(0.f);
	float MaxCurveTime(1.f);

	Curve.GetTimeRange(MinCurveTime, MaxCurveTime);
	return Curve.GetFloatValue(FMath::GetRangeValue(FVector2f(MinCurveTime, MaxCurveTime), Fraction));
}

bool FLayeredMove_MoveTo::GenerateMove(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove)
{
	OutProposedMove.MixMode = MixMode;

	const float DeltaSeconds = TimeStep.StepMs / 1000.f;
	
	float MoveFraction = (TimeStep.BaseSimTimeMs - StartSimTimeMs) / DurationMs;

	if (TimeMappingCurve)
	{
		MoveFraction = EvaluateFloatCurveAtFraction(*TimeMappingCurve, MoveFraction);
	}
	
	const AActor* MoverActor = MoverComp->GetOwner();
	
	FVector CurrentTargetLocation = FMath::Lerp<FVector, float>(StartLocation, TargetLocation, MoveFraction);
	FVector PathOffset = GetPathOffsetInWorldSpace(MoveFraction);
	CurrentTargetLocation += PathOffset;

	const FVector CurrentLocation = MoverActor->GetActorLocation();

	FVector Velocity = (CurrentTargetLocation - CurrentLocation) / DeltaSeconds;

	if (bRestrictSpeedToExpected && !Velocity.IsNearlyZero(UE_KINDA_SMALL_NUMBER))
	{
		// Calculate expected current location (if we didn't have collision and moved exactly where our velocity should have taken us)
		const float PreviousMoveFraction = (TimeStep.BaseSimTimeMs - StartSimTimeMs - TimeStep.StepMs) / DurationMs;
		FVector CurrentExpectedLocation = FMath::Lerp<FVector, float>(StartLocation, TargetLocation, PreviousMoveFraction);
		CurrentExpectedLocation += GetPathOffsetInWorldSpace(PreviousMoveFraction);

		// Restrict speed to the expected speed, allowing some small amount of error
		const FVector ExpectedForce = (CurrentTargetLocation - CurrentExpectedLocation) / DeltaSeconds;
		const float ExpectedSpeed = ExpectedForce.Size();
		const float CurrentSpeedSqr = Velocity.SizeSquared();

		const float ErrorAllowance = 0.5f; // in cm/s
		if (CurrentSpeedSqr > FMath::Square(ExpectedSpeed + ErrorAllowance))
		{
			Velocity.Normalize();
			Velocity *= ExpectedSpeed;
		}
	}
	
	OutProposedMove.LinearVelocity = Velocity;
	
	return true;
}

FLayeredMoveBase* FLayeredMove_MoveTo::Clone() const
{
	FLayeredMove_MoveTo* CopyPtr = new FLayeredMove_MoveTo(*this);
	return CopyPtr;
}

void FLayeredMove_MoveTo::NetSerialize(FArchive& Ar)
{
	Super::NetSerialize(Ar);

	Ar << StartLocation;
	Ar << TargetLocation;
	Ar << bRestrictSpeedToExpected;
	Ar << PathOffsetCurve;
	Ar << TimeMappingCurve;
}

UScriptStruct* FLayeredMove_MoveTo::GetScriptStruct() const
{
	return FLayeredMove_MoveTo::StaticStruct();
}

FString FLayeredMove_MoveTo::ToSimpleString() const
{
	return FString::Printf(TEXT("Move To"));
}

void FLayeredMove_MoveTo::AddReferencedObjects(FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);
}

// -------------------------------------------------------------------
// FLayeredMove_MoveToDynamic
// -------------------------------------------------------------------

FLayeredMove_MoveToDynamic::FLayeredMove_MoveToDynamic()
	: LocationActor(nullptr)
{
	StartLocation = FVector::ZeroVector;
	TargetLocation = FVector::ZeroVector;
	bRestrictSpeedToExpected = false;
	PathOffsetCurve = nullptr;
	TimeMappingCurve = nullptr;
	DurationMs = 1000.f;
	MixMode = EMoveMixMode::OverrideVelocity;
}

bool FLayeredMove_MoveToDynamic::GenerateMove(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove)
{
	if (LocationActor)
	{
		TargetLocation = LocationActor->GetActorLocation();
	}
	
	return Super::GenerateMove(StartState, TimeStep, MoverComp, SimBlackboard, OutProposedMove);
}

FLayeredMoveBase* FLayeredMove_MoveToDynamic::Clone() const
{
	FLayeredMove_MoveToDynamic* CopyPtr = new FLayeredMove_MoveToDynamic(*this);
	return CopyPtr;
}

void FLayeredMove_MoveToDynamic::NetSerialize(FArchive& Ar)
{
	Super::NetSerialize(Ar);

	Ar << LocationActor;
}

UScriptStruct* FLayeredMove_MoveToDynamic::GetScriptStruct() const
{
	return FLayeredMove_MoveToDynamic::StaticStruct();
}

FString FLayeredMove_MoveToDynamic::ToSimpleString() const
{
	return FString::Printf(TEXT("Move To Dynamic"));
}

void FLayeredMove_MoveToDynamic::AddReferencedObjects(FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);
}

// -------------------------------------------------------------------
// FLayeredMove_RadialImpulse
// -------------------------------------------------------------------

FLayeredMove_RadialImpulse::FLayeredMove_RadialImpulse()
	: Location(ForceInitToZero)
	, LocationActor(nullptr)
	, Radius(1.f)
	, Magnitude(0.f)
	, bIsPush(true)
	, bNoVerticalVelocity(false)
	, DistanceFalloff(nullptr)
	, MagnitudeOverTime(nullptr)
	, bUseFixedWorldDirection(false)
	, FixedWorldDirection(ForceInitToZero)
{
	DurationMs = 0.f;
	MixMode = EMoveMixMode::AdditiveVelocity;
}

bool FLayeredMove_RadialImpulse::GenerateMove(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove)
{
	const FMoverDefaultSyncState* SyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(SyncState);
	
	const FVector CharacterLocation = SyncState->GetLocation_WorldSpace();
	FVector Velocity = FVector::ZeroVector;
	const FVector VelocityLocation = LocationActor ? LocationActor->GetActorLocation() : Location;
	float Distance = FVector::Dist(VelocityLocation, CharacterLocation);
	
	if (Distance < Radius)
	{
		// Calculate magnitude
		float CurrentMagnitude = Magnitude;
		{
			float AdditiveMagnitudeFactor = 1.f;
			if (DistanceFalloff)
			{
				const float DistanceFactor = DistanceFalloff->GetFloatValue(FMath::Clamp(Distance / Radius, 0.f, 1.f));
				AdditiveMagnitudeFactor -= (1.f - DistanceFactor);
			}

			if (MagnitudeOverTime && DurationMs > 0)
			{
				const double TimeValue = FMath::Clamp((TimeStep.BaseSimTimeMs - StartSimTimeMs) / DurationMs, 0.0, 1.0);
				const float TimeFactor = MagnitudeOverTime->GetFloatValue(TimeValue);
				AdditiveMagnitudeFactor -= (1.f - TimeFactor);
			}

			CurrentMagnitude = Magnitude * FMath::Clamp(AdditiveMagnitudeFactor, 0.f, 1.f);
		}

		if (bUseFixedWorldDirection)
		{
			Velocity = FixedWorldDirection.Vector() * CurrentMagnitude;
		}
		else
		{
			Velocity = (VelocityLocation - CharacterLocation).GetSafeNormal() * CurrentMagnitude;
			
			if (bIsPush)
			{
				Velocity *= -1.f;
			}
		}
		
		if (bNoVerticalVelocity)
        {
       		const FPlane MovementPlane(FVector::ZeroVector, MoverComp->GetUpDirection());
       		Velocity = UMovementUtils::ConstrainToPlane(Velocity, MovementPlane);
       	}
	}
	else 
	{
	     return false;
	}

	OutProposedMove.LinearVelocity = Velocity;

	return true;
}

FLayeredMoveBase* FLayeredMove_RadialImpulse::Clone() const
{
	FLayeredMove_RadialImpulse* CopyPtr = new FLayeredMove_RadialImpulse(*this);
    return CopyPtr;
}

void FLayeredMove_RadialImpulse::NetSerialize(FArchive& Ar)
{
	Super::NetSerialize(Ar);
	
	Ar << Location;
	Ar << LocationActor;
	Ar << Radius;
	Ar << Magnitude;
	Ar << bIsPush;
	Ar << bNoVerticalVelocity;
	Ar << DistanceFalloff;
	Ar << MagnitudeOverTime;
	Ar << bUseFixedWorldDirection;
	if (bUseFixedWorldDirection)
	{
		Ar << FixedWorldDirection;
	}
}

UScriptStruct* FLayeredMove_RadialImpulse::GetScriptStruct() const
{
	return FLayeredMove_RadialImpulse::StaticStruct();
}

FString FLayeredMove_RadialImpulse::ToSimpleString() const
{
	return FString::Printf(TEXT("Radial Impulse"));
}

void FLayeredMove_RadialImpulse::AddReferencedObjects(FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);
}
