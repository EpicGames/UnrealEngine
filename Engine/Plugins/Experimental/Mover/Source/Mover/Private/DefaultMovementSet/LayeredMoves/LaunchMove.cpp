// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultMovementSet/LayeredMoves/LaunchMove.h"
#include "MoverComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LaunchMove)

void FLaunchMoveData::ActivateFromContext(const FLayeredMoveActivationParams* ActivationParams)
{
	if (ActivationParams)
	{
		if (const FLaunchMoveActivationParams* LaunchMoveActivationParams = static_cast<const FLaunchMoveActivationParams*>(ActivationParams))
		{
			LaunchVelocity = LaunchMoveActivationParams->LaunchVelocity;
			DurationMs = LaunchMoveActivationParams->DurationMs;
			ForceMovementMode = LaunchMoveActivationParams->ForceMovementMode;
		}
	}
}

void FLaunchMoveData::NetSerialize(FArchive& Ar)
{
	Super::NetSerialize(Ar);

	SerializePackedVector<10, 16>(LaunchVelocity, Ar);

	bool bUsingForcedMovementMode = !ForceMovementMode.IsNone();

	Ar.SerializeBits(&bUsingForcedMovementMode, 1);

	if (bUsingForcedMovementMode)
	{
		Ar << ForceMovementMode;
	}
	else
	{
		ForceMovementMode = NAME_None;
	}
}

ULaunchMoveLogic::ULaunchMoveLogic()
{
	DefaultDurationMs = 0.f;
	MixMode = EMoveMixMode::OverrideVelocity;
	InstancedDataStructType = FLaunchMoveData::StaticStruct();
}

bool ULaunchMoveLogic::GenerateMove_Implementation(const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, const FMoverTickStartData& StartState, FProposedMove& OutProposedMove)
{
	return GenerateMove_Async_Implementation(TimeStep, SimBlackboard, StartState, OutProposedMove);
}

bool ULaunchMoveLogic::GenerateMove_Async_Implementation(const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, const FMoverTickStartData& StartState, FProposedMove& OutProposedMove)
{
	const FLaunchMoveData& LaunchMoveData = AccessExecutionMoveData<FLaunchMoveData>();

	OutProposedMove.MixMode = MixMode;
	OutProposedMove.LinearVelocity = LaunchMoveData.LaunchVelocity;
	OutProposedMove.PreferredMode = LaunchMoveData.ForceMovementMode;

	return true;
}

bool FLayeredMove_Launch::GenerateMove_Async(const FMoverTickStartData& SimState, const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove)
{
	OutProposedMove.MixMode = MixMode;
	OutProposedMove.LinearVelocity = LaunchVelocity;
	OutProposedMove.PreferredMode = ForceMovementMode;
	return true;
}

FLayeredMove_Launch::FLayeredMove_Launch()
{
	DurationMs = 0.f;
	MixMode = EMoveMixMode::OverrideVelocity;
}

bool FLayeredMove_Launch::GenerateMove(const FMoverTickStartData& SimState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove)
{
	return GenerateMove_Async(SimState, TimeStep, SimBlackboard, OutProposedMove);
}

FLayeredMoveBase* FLayeredMove_Launch::Clone() const
{
	FLayeredMove_Launch* CopyPtr = new FLayeredMove_Launch(*this);
	return CopyPtr;
}

void FLayeredMove_Launch::NetSerialize(FArchive& Ar)
{
	Super::NetSerialize(Ar);

	SerializePackedVector<10, 16>(LaunchVelocity, Ar);

	bool bUsingForcedMovementMode = !ForceMovementMode.IsNone();

	Ar.SerializeBits(&bUsingForcedMovementMode, 1);

	if (bUsingForcedMovementMode)
	{
		Ar << ForceMovementMode;
	}

}

UScriptStruct* FLayeredMove_Launch::GetScriptStruct() const
{
	return FLayeredMove_Launch::StaticStruct();
}

FString FLayeredMove_Launch::ToSimpleString() const
{
	return FString::Printf(TEXT("Launch"));
}

void FLayeredMove_Launch::AddReferencedObjects(class FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);
}
