// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultMovementSet/LayeredMoves/MontageStateProvider.h"
#include "Animation/AnimMontage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MontageStateProvider)


void FMoverAnimMontageState::Reset()
{
	Montage = nullptr;
	BlendOutTimeSeconds = 0.f;
	bEnableAutoBlendOut = true;
#if !UE_BUILD_SHIPPING
	Debug_MontageName = NAME_None;
#endif
}

void FMoverAnimMontageState::NetSerialize(FArchive& Ar)
{
	Ar << Montage;

	uint8 bHasNonDefaultStartingPosition = 0;
	uint8 bHasNonDefaultPlayRate = 0;

	if (Ar.IsSaving())
	{
		bHasNonDefaultStartingPosition = (StartingMontagePosition != 0.0f);
		bHasNonDefaultPlayRate = (PlayRate != 1.0f);
	}

	Ar.SerializeBits(&bHasNonDefaultStartingPosition, 1);
	Ar.SerializeBits(&bHasNonDefaultPlayRate, 1);

	if (bHasNonDefaultStartingPosition)
	{
		Ar << StartingMontagePosition;
	}
	else
	{
		StartingMontagePosition = 0.0f;
	}

	if (bHasNonDefaultPlayRate)
	{
		Ar << PlayRate;
	}
	else
	{
		PlayRate = 1.0f;
	}

	Ar << CurrentPosition;

	// Blend-out fields: conditionally serialized to maintain backward compatibility.
	// The bit flag is always serialized; the payload is only present when the flag is set.
	uint8 bHasBlendOutData = 0;
	if (Ar.IsSaving())
	{
		bHasBlendOutData = (BlendOutTimeSeconds > 0.f || !bEnableAutoBlendOut) ? 1 : 0;
	}
	Ar.SerializeBits(&bHasBlendOutData, 1);
	if (bHasBlendOutData)
	{
		Ar << BlendOutTimeSeconds;
		uint8 bEnableAutoBlendOutBit = bEnableAutoBlendOut ? 1 : 0;
		Ar.SerializeBits(&bEnableAutoBlendOutBit, 1);
		if (Ar.IsLoading())
		{
			bEnableAutoBlendOut = (bEnableAutoBlendOutBit != 0);
		}
	}
	else if (Ar.IsLoading())
	{
		BlendOutTimeSeconds = 0.f;
		bEnableAutoBlendOut = true;
	}

#if !UE_BUILD_SHIPPING
	if (Ar.IsLoading())
	{
		// Debug_MontageName is not serialized -- derive it from the Montage asset on arrival so CVD
		// displays the correct name on machines that received the move over the network.
		Debug_MontageName = Montage ? Montage->GetFName() : NAME_None;
	}
#endif
}
