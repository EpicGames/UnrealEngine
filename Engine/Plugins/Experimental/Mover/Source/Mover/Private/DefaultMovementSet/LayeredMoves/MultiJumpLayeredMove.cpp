// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultMovementSet/LayeredMoves/MultiJumpLayeredMove.h"
#include "MoverComponent.h"
#include "MoverSimulationTypes.h"
#include "DefaultMovementSet/Settings/CommonLegacyMovementSettings.h"
#include "MoveLibrary/FloorQueryUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MultiJumpLayeredMove)

FLayeredMove_MultiJump::FLayeredMove_MultiJump()
	: MaximumInAirJumps(1)
	, UpwardsSpeed(800)
	, JumpsInAirRemaining(-1)
	, TimeOfLastJumpMS(0)
{
	DurationMs = -1.f;
	MixMode = EMoveMixMode::OverrideVelocity;
}

bool FLayeredMove_MultiJump::WantsToJump(const FMoverInputCmdContext& InputCmd)
{
	if (const FCharacterDefaultInputs* CharacterInputs = InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>())
	{
		return CharacterInputs->bIsJumpJustPressed;
	}
	
	return false;
}

bool FLayeredMove_MultiJump::GenerateMove(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove)
{
	const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	const FMoverDefaultSyncState* SyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(SyncState);

	OutProposedMove.MixMode = MixMode;

	FFloorCheckResult FloorHitResult;
	bool bValidBlackboard = SimBlackboard->TryGet(CommonBlackboard::LastFloorResult, OUT FloorHitResult);

	if (FMath::IsNearlyEqual(StartSimTimeMs, TimeStep.BaseSimTimeMs))
	{
		JumpsInAirRemaining = MaximumInAirJumps;
	}
	
	bool bPerformedJump = false;
	if (CharacterInputs && CharacterInputs->bIsJumpJustPressed)
	{
		if (StartSimTimeMs == TimeStep.BaseSimTimeMs)
		{
			// if this was the first jump and its a valid floor we do the initial jump from walking and back out so we don't get extra jump
			if (bValidBlackboard && FloorHitResult.IsWalkableFloor())
			{
				bPerformedJump = PerformJump(SyncState, TimeStep, MoverComp, OutProposedMove);
				return bPerformedJump;
			}
		}

		// perform in air jump
		if (TimeStep.BaseSimTimeMs > TimeOfLastJumpMS && JumpsInAirRemaining > 0)
		{
			JumpsInAirRemaining--;
			bPerformedJump = PerformJump(SyncState, TimeStep, MoverComp, OutProposedMove);
		}
		else
		{
			// setting mix mode to additive when we're not adding any jump input so air movement acts as expected
			OutProposedMove.MixMode = EMoveMixMode::AdditiveVelocity;
		}
	}
	else
	{
		// setting mix mode to additive when we're not adding any jump input so air movement acts as expected
		OutProposedMove.MixMode = EMoveMixMode::AdditiveVelocity;
	}

	// if we hit a valid floor and it's not the start of the move (since we could start this move on the ground) end this move
	if ((bValidBlackboard && FloorHitResult.IsWalkableFloor() && StartSimTimeMs < TimeStep.BaseSimTimeMs) || JumpsInAirRemaining <= 0)
	{
		DurationMs = 0;
	}

	return bPerformedJump;
}

FLayeredMoveBase* FLayeredMove_MultiJump::Clone() const
{
	FLayeredMove_MultiJump* CopyPtr = new FLayeredMove_MultiJump(*this);
	return CopyPtr;
}

void FLayeredMove_MultiJump::NetSerialize(FArchive& Ar)
{
	Super::NetSerialize(Ar);

	Ar << MaximumInAirJumps;
	Ar << UpwardsSpeed;
	Ar << JumpsInAirRemaining;
	Ar << TimeOfLastJumpMS;
}

UScriptStruct* FLayeredMove_MultiJump::GetScriptStruct() const
{
	return FLayeredMove_MultiJump::StaticStruct();
}

FString FLayeredMove_MultiJump::ToSimpleString() const
{
	return FString::Printf(TEXT("Multi-jump"));
}

void FLayeredMove_MultiJump::AddReferencedObjects(FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);
}

bool FLayeredMove_MultiJump::PerformJump(const FMoverDefaultSyncState* SyncState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, FProposedMove& OutProposedMove)
{
	TimeOfLastJumpMS = TimeStep.BaseSimTimeMs;
	if (const TObjectPtr<const UCommonLegacyMovementSettings> CommonLegacySettings = MoverComp->FindSharedSettings<UCommonLegacyMovementSettings>())
	{
		OutProposedMove.PreferredMode = CommonLegacySettings->AirMovementModeName;
	}

	const FVector UpDir = MoverComp->GetUpDirection();

	const FVector ImpulseVelocity = UpDir * UpwardsSpeed;

	switch (MixMode)
	{
	case EMoveMixMode::AdditiveVelocity:
		{
			OutProposedMove.LinearVelocity = ImpulseVelocity;
			break;
		}
		
	case EMoveMixMode::OverrideAll:
	case EMoveMixMode::OverrideVelocity:
		{
			// Jump impulse overrides vertical velocity while maintaining the rest
			const FVector PriorVelocityWS = SyncState->GetVelocity_WorldSpace();
			const FVector StartingNonUpwardsVelocity = PriorVelocityWS - PriorVelocityWS.ProjectOnToNormal(UpDir);

			// If we're leaving a movement base, include its current velocity in our initial airborne velocity
			FVector MovementBaseVelocityWS;
			UBasedMovementUtils::GetMovementBaseVelocityAtPoint(SyncState->GetMovementBase(), MoverComp->GetUpdatedComponent()->GetComponentLocation(), OUT MovementBaseVelocityWS);

			OutProposedMove.LinearVelocity = StartingNonUpwardsVelocity + ImpulseVelocity + MovementBaseVelocityWS;
			break;
		}
		
	default:
		ensureMsgf(false, TEXT("Multi-Jump layered move has an invalid MixMode and will do nothing."));
		return false;
	}

	return true;
}
