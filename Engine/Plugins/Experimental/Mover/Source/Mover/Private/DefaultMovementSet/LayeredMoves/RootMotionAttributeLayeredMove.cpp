// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultMovementSet/LayeredMoves/RootMotionAttributeLayeredMove.h"
#include "MoverComponent.h"
#include "Animation/AttributesContainer.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimInstance.h"
#include "Components/SkeletalMeshComponent.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoverTypes.h"
#include "MoverLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RootMotionAttributeLayeredMove)

UE_DEFINE_GAMEPLAY_TAG_COMMENT(Mover_AnimRootMotion_MeshAttribute, "Mover.AnimRootMotion.MeshAttribute", "Signifies an association with root motion that comes via a skeletal mesh attribute");


#if !UE_BUILD_SHIPPING
FAutoConsoleVariable CVarLogRootMotionAttrSteps(
	TEXT("mover.debug.LogRootMotionAttrSteps"),
	false,
	TEXT("Whether to log detailed information about root motion attribute layered moves. 0: Disable, 1: Enable"),
	ECVF_Cheat);

FAutoConsoleVariable CVarDisableRootMotionAttrContributions(
	TEXT("mover.debug.DisableRootMotionAttributes"),
	false,
	TEXT("If enabled, contributions from root motion attributes will be ignored in favor of other Mover influences"),
	ECVF_Cheat);

static float ExcessiveLinearVelocitySquaredThreshold = 2000.f * 2000.f;
FAutoConsoleVariableRef CVarExcessiveLinearSpeedSquaredThreshold(
	TEXT("mover.debug.RootMotionAttributesExcessiveSpeedSq"),
	ExcessiveLinearVelocitySquaredThreshold,
	TEXT("If > 0, a warning will be logged when a root motion attribute's squared speed exceeds this threshold\n"));

#endif	// !UE_BUILD_SHIPPING


static const FName RootMotionAttributeName = "RootMotionDelta";
static const UE::Anim::FAttributeId RootMotionAttributeId = { RootMotionAttributeName, FCompactPoseBoneIndex(0) };

FLayeredMove_RootMotionAttribute::FLayeredMove_RootMotionAttribute()
{
	DurationMs = -1.f;
	MixMode = EMoveMixMode::OverrideAll;
}

bool FLayeredMove_RootMotionAttribute::GenerateMove(const FMoverTickStartData& SimState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove)
{
#if !UE_BUILD_SHIPPING
	if (CVarDisableRootMotionAttrContributions->GetBool())
	{
		return false;	// do not contribute any movement
	}
#endif // !UE_BUILD_SHIPPING

	const float DeltaSeconds = TimeStep.StepMs / 1000.f;

	bool bDidAttrHaveRootMotion = false;
	FTransform LocalRootMotion;
	bool bHasValidWarpingContext = false;
	FMotionWarpingUpdateContext WarpingContext;

	if (!TimeStep.bIsResimulating)
	{
		// Clear resim values
		bDidAttrHaveRootMotionForResim = false;
		LocalRootMotionForResim = FTransform::Identity;
		WarpingContextForResim = FMotionWarpingUpdateContext();

		// Sample root motion from the mesh attribute
		const USkeletalMeshComponent* Mesh = Cast<const USkeletalMeshComponent>(MoverComp->GetPrimaryVisualComponent());

		if (Mesh)
		{
			// TODO: support options for different interpretations, such as velocity
			if (const FTransformAnimationAttribute* RootMotionAttribute = Mesh->GetCustomAttributes().Find<FTransformAnimationAttribute>(RootMotionAttributeId))
			{
				// NOTE this will only work for ticking modes that tick in time with the world tick, because it relies on the Mesh ticking at the same rate as the movement simulation. 
				// For fixed-tick modes, the attribute would be better as an accumulator for the movement sim to consume, along with a time accumulation attribute.
				LocalRootMotion = RootMotionAttribute->Value;
				LocalRootMotion.SetScale3D(FVector::OneVector);	// sanitize any scaling factor
				bDidAttrHaveRootMotion = true;
			}
		}

		static const bool bShouldWarpFromMontage = true;

		if (bDidAttrHaveRootMotion && bShouldWarpFromMontage)
		{
			// Not resimulating: we are following along with any root motion montages
			const FAnimMontageInstance* RootMotionMontageInstance = Mesh && Mesh->GetAnimInstance() ? Mesh->GetAnimInstance()->GetRootMotionMontageInstance() : nullptr;

			if (RootMotionMontageInstance)
			{
				const UAnimMontage* Montage = RootMotionMontageInstance->Montage;

				WarpingContext.DeltaSeconds = DeltaSeconds;
				WarpingContext.Animation = Montage;
				WarpingContext.CurrentPosition = RootMotionMontageInstance->GetPosition();
				WarpingContext.PreviousPosition = RootMotionMontageInstance->GetPreviousPosition();
				WarpingContext.Weight = RootMotionMontageInstance->GetWeight();
				WarpingContext.PlayRate = RootMotionMontageInstance->Montage->RateScale * RootMotionMontageInstance->GetPlayRate();

				bHasValidWarpingContext = true;
			}
		}

		// Save values for resim
		{
			bDidAttrHaveRootMotionForResim = bDidAttrHaveRootMotion;
			LocalRootMotionForResim = LocalRootMotion;

			if (bHasValidWarpingContext)
			{
				WarpingContextForResim = WarpingContext;
			}
		}

	}
	else   // resimulating...
	{
		// restore the cached transform and warping parameters (if set)
		bDidAttrHaveRootMotion	= bDidAttrHaveRootMotionForResim;
		LocalRootMotion			= LocalRootMotionForResim;
		bHasValidWarpingContext = WarpingContextForResim.Animation != nullptr;
		WarpingContext			= WarpingContextForResim;
	}

	if (bDidAttrHaveRootMotion)
	{
		const FMoverDefaultSyncState* SyncState = SimState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
		const FCharacterDefaultInputs* InputCmd = SimState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();

		if (!bHasValidWarpingContext &&
				((InputCmd && InputCmd->bIsJumpJustPressed) || MoverComp->HasGameplayTagInState(SimState.SyncState, Mover_SkipAnimRootMotion, false)))
		{
			return false;	// do not perform root motion while we have the SkipAnimRootMotion tag. i.e. jumping or falling, so that we can have air control, unless we're under montage influence
		}

		// Note that we're forcing the use of the sync state's actor transform data. This is necessary when the movement simulation 
		// is running ahead of the actor's visual representation and may be rotated differently, such as in an async physics sim.
		const FTransform SimActorTransform = FTransform(SyncState->GetOrientation_WorldSpace().Quaternion(), SyncState->GetLocation_WorldSpace());
		FTransform WorldSpaceRootMotion = MoverComp->ConvertLocalRootMotionToWorld(LocalRootMotion, DeltaSeconds, &SimActorTransform, (bHasValidWarpingContext ? &WarpingContext : nullptr));

		if (bConstrainWorldRotToMovementPlane)
		{
			// up direction is the locked axis. The original rotated direction is effectively projected onto the movement plane defined by the up direction.
			const FMatrix ConstrainedRot = FRotationMatrix::MakeFromZX(MoverComp->GetUpDirection(), WorldSpaceRootMotion.GetRotation().GetForwardVector());
			WorldSpaceRootMotion.SetRotation(ConstrainedRot.ToQuat());
		}

		OutProposedMove = FProposedMove();
		OutProposedMove.MixMode = MixMode;

		// Convert the transform into linear and angular velocities
		if (DeltaSeconds > UE_SMALL_NUMBER)
		{
			OutProposedMove.LinearVelocity  = WorldSpaceRootMotion.GetTranslation() * (1.f / DeltaSeconds);
			const FQuat ShortestArcRootRotation = WorldSpaceRootMotion.GetRotation().GetShortestArcWith(FQuat::Identity);
			OutProposedMove.AngularVelocityDegrees = FMath::RadiansToDegrees(ShortestArcRootRotation.ToRotationVector()  / DeltaSeconds);
		}
		else
		{
			OutProposedMove.LinearVelocity = FVector::ZeroVector;
			OutProposedMove.AngularVelocityDegrees = FVector::ZeroVector;
		}

#if !UE_BUILD_SHIPPING
		UE_CLOGF(CVarLogRootMotionAttrSteps->GetBool(), LogMover, Log, "RootMotionAttr. SimF %i (dt %.3f) => LocalT: %ls (WST: %ls)  XY Speed: %.6f Z: %.6f   AngV: %ls",
			TimeStep.ServerFrame, DeltaSeconds,
			*LocalRootMotion.GetTranslation().ToString(), *WorldSpaceRootMotion.GetTranslation().ToString(),
			OutProposedMove.LinearVelocity.Size2D(), OutProposedMove.LinearVelocity.Z, *OutProposedMove.AngularVelocityDegrees.ToCompactString());

		if (ExcessiveLinearVelocitySquaredThreshold > 0.f && ExcessiveLinearVelocitySquaredThreshold < OutProposedMove.LinearVelocity.SquaredLength())
		{
			UE_LOGF(LogMover, Warning, "RootMotionAttr on %ls has excessive speed.  LocalTrans: %ls (DT: %.5f) -> XY Vel: %.6f  Z Vel: %.6f",
				*GetNameSafe(MoverComp->GetOwner()),
				*LocalRootMotion.GetTranslation().ToString(), DeltaSeconds, OutProposedMove.LinearVelocity.Size2D(), OutProposedMove.LinearVelocity.Z);
		}


#endif // !UE_BUILD_SHIPPING

		return true;
	}

	return false;
}

bool FLayeredMove_RootMotionAttribute::HasGameplayTag(FGameplayTag TagToFind, bool bExactMatch) const
{
	static const FGameplayTag MeshAttributeTag = Mover_AnimRootMotion_MeshAttribute.GetTag();
	const bool bFoundMatch = bExactMatch ? MeshAttributeTag.MatchesTagExact(TagToFind) : MeshAttributeTag.MatchesTag(TagToFind);

	return bFoundMatch || Super::HasGameplayTag(TagToFind, bExactMatch);
}

FLayeredMoveBase* FLayeredMove_RootMotionAttribute::Clone() const
{
	FLayeredMove_RootMotionAttribute* CopyPtr = new FLayeredMove_RootMotionAttribute(*this);
	return CopyPtr;
}

void FLayeredMove_RootMotionAttribute::NetSerialize(FArchive& Ar)
{
	Super::NetSerialize(Ar);
}

UScriptStruct* FLayeredMove_RootMotionAttribute::GetScriptStruct() const
{
	return FLayeredMove_RootMotionAttribute::StaticStruct();
}

FString FLayeredMove_RootMotionAttribute::ToSimpleString() const
{
	return FString::Printf(TEXT("RootMotionAttribute"));
}

void FLayeredMove_RootMotionAttribute::AddReferencedObjects(class FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);
}
