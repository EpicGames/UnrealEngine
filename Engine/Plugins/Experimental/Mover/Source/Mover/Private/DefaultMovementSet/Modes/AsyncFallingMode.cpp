// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultMovementSet/Modes/AsyncFallingMode.h"
#include "Components/SkeletalMeshComponent.h"
#include "MoverComponent.h"
#include "MoveLibrary/AirMovementUtils.h"
#include "MoveLibrary/AsyncMovementUtils.h"
#include "MoveLibrary/BasedMovementUtils.h"
#include "MoveLibrary/GroundMovementUtils.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/MovementUtils.h"
#include "DefaultMovementSet/Settings/CommonLegacyMovementSettings.h"

#include "DrawDebugHelpers.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncFallingMode)

UAsyncFallingMode::UAsyncFallingMode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bCancelVerticalSpeedOnLanding(true)
	, AirControlPercentage(0.4f)
	, FallingDeceleration(200.0f)
	, OverTerminalSpeedFallingDeceleration(800.0f)
	, TerminalMovementPlaneSpeed(1500.0f)
	, bShouldClampTerminalVerticalSpeed(true)
	, VerticalFallingDeceleration(4000.0f)
	, TerminalVerticalSpeed(2000.0f)
{
	SharedSettingsClasses.Add(UCommonLegacyMovementSettings::StaticClass());

	GameplayTags.AddTag(Mover_IsInAir);
	GameplayTags.AddTag(Mover_IsFalling);
	GameplayTags.AddTag(Mover_SkipVerticalAnimRootMotion);	// allows combination of gravity falling and root motion
}


void UAsyncFallingMode::GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const
{
	const UMoverComponent* MoverComp = GetMoverComponent();
	const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(StartingSyncState);

	if (!CommonLegacySettings.IsValid())
	{
		return;
	}

	const float DeltaSeconds = TimeStep.StepMs * 0.001f;

	FVector UpDirection = MoverComp->GetUpDirection();
	
	// We don't want velocity limits to take the falling velocity component into account, since it is handled 
	//   separately by the terminal velocity of the environment.
	const FVector StartVelocity = StartingSyncState->GetVelocity_WorldSpace();
	const FVector StartHorizontalVelocity =  FVector::VectorPlaneProject(StartVelocity, UpDirection);

	FFreeMoveParams Params;
	if (CharacterInputs)
	{
		Params.MoveInputType = CharacterInputs->GetMoveInputType();
		const bool bMaintainInputMagnitude = true;
		Params.MoveInput = UPlanarConstraintUtils::ConstrainDirectionToPlane(MoverComp->GetPlanarConstraint(), CharacterInputs->GetMoveInput_WorldSpace(), bMaintainInputMagnitude);
	}
	else
	{
		Params.MoveInputType = EMoveInputType::None;
		Params.MoveInput = FVector::ZeroVector;
	}

	Params.MoveInput *= AirControlPercentage;
	// Don't care about up axis input since falling - if up input matters that should probably be a different movement mode
	Params.MoveInput = FVector::VectorPlaneProject(Params.MoveInput, UpDirection);
	
	FRotator IntendedOrientation_WorldSpace;
	// If there's no intent from input to change orientation, use the current orientation
	if (!CharacterInputs || CharacterInputs->OrientationIntent.IsNearlyZero())
	{
		IntendedOrientation_WorldSpace = StartingSyncState->GetOrientation_WorldSpace();
	}
	else
	{
		IntendedOrientation_WorldSpace = CharacterInputs->GetOrientationIntentDir_WorldSpace().ToOrientationRotator();
	}

	IntendedOrientation_WorldSpace = UMovementUtils::ApplyGravityToOrientationIntent(IntendedOrientation_WorldSpace, MoverComp->GetWorldToGravityTransform(), CommonLegacySettings->bShouldRemainVertical);
	
	Params.OrientationIntent = IntendedOrientation_WorldSpace;
	Params.PriorVelocity = StartHorizontalVelocity;
	Params.PriorOrientation = StartingSyncState->GetOrientation_WorldSpace();
	Params.DeltaSeconds = DeltaSeconds;
	Params.TurningRate = CommonLegacySettings->TurningRate;
	Params.TurningBoost = CommonLegacySettings->TurningBoost;
	Params.MaxSpeed = CommonLegacySettings->MaxSpeed;
	Params.Acceleration = CommonLegacySettings->Acceleration;
	Params.Deceleration = FallingDeceleration;
	Params.WorldToGravityQuat = MoverComp->GetWorldToGravityTransform();
	Params.bUseAccelerationForVelocityMove = CommonLegacySettings->bUseAccelerationForVelocityMove;

	// Check if any current velocity values are over our terminal velocity - if so limit the move input in that direction and apply OverTerminalVelocityFallingDeceleration
	if (Params.MoveInput.Dot(StartVelocity) > 0 && StartHorizontalVelocity.Size() >= TerminalMovementPlaneSpeed)
	{
		Params.Deceleration = OverTerminalSpeedFallingDeceleration;
	}
	
	UMoverBlackboard* SimBlackboard = MoverComp->GetSimBlackboard_Mutable();
	FFloorCheckResult LastFloorResult;
	// limit our moveinput based on the floor we're on
	if (SimBlackboard && SimBlackboard->TryGet(CommonBlackboard::LastFloorResult, LastFloorResult))
	{
		if (LastFloorResult.HitResult.IsValidBlockingHit() && LastFloorResult.HitResult.Normal.Dot(UpDirection) > UE::MoverUtils::VERTICAL_SLOPE_NORMAL_MAX_DOT && !LastFloorResult.IsWalkableFloor())
		{
			// If acceleration is into the wall, limit contribution.
			if (FVector::DotProduct(Params.MoveInput, LastFloorResult.HitResult.Normal) < 0.f)
			{
				// Allow movement parallel to the wall, but not into it because that may push us up.
				const FVector FallingHitNormal = FVector::VectorPlaneProject( LastFloorResult.HitResult.Normal, -UpDirection).GetSafeNormal();
				Params.MoveInput = FVector::VectorPlaneProject(Params.MoveInput, FallingHitNormal);
			}
		}
	}
	
	OutProposedMove = UAirMovementUtils::ComputeControlledFreeMove(Params);
	const FVector VelocityWithGravity = StartVelocity + UMovementUtils::ComputeVelocityFromGravity(MoverComp->GetGravityAcceleration(), DeltaSeconds);

	//  If we are going faster than TerminalVerticalVelocity apply VerticalFallingDeceleration otherwise reset Z velocity to before we applied deceleration 
	if (VelocityWithGravity.GetAbs().Dot(UpDirection) > TerminalVerticalSpeed)
	{
		if (bShouldClampTerminalVerticalSpeed)
		{
			const float ClampedVerticalSpeed = FMath::Sign(VelocityWithGravity.Dot(UpDirection)) * TerminalVerticalSpeed;
			UMovementUtils::SetGravityVerticalComponent(OutProposedMove.LinearVelocity, ClampedVerticalSpeed, UpDirection);
		}
		else
		{
			float DesiredDeceleration = FMath::Abs(TerminalVerticalSpeed - VelocityWithGravity.GetAbs().Dot(UpDirection)) / DeltaSeconds;
			float DecelerationToApply = FMath::Min(DesiredDeceleration, VerticalFallingDeceleration);
			DecelerationToApply = FMath::Sign(VelocityWithGravity.Dot(UpDirection)) * DecelerationToApply * DeltaSeconds;
			FVector MaxUpDirVelocity = VelocityWithGravity * UpDirection - (UpDirection * DecelerationToApply);
			
			UMovementUtils::SetGravityVerticalComponent(OutProposedMove.LinearVelocity, MaxUpDirVelocity.Dot(UpDirection), UpDirection);
		}
	}
	else
	{
		UMovementUtils::SetGravityVerticalComponent(OutProposedMove.LinearVelocity, VelocityWithGravity.Dot(UpDirection), UpDirection);
	}
}

void UAsyncFallingMode::SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState)
{
	if (!CommonLegacySettings.IsValid())
	{
		return;
	}

	UMoverComponent* MoverComponent = GetMoverComponent();
	const FMoverTickStartData& StartState = Params.StartState;
	USceneComponent* UpdatedComponent = Params.MovingComps.UpdatedComponent.Get();
	FProposedMove ProposedMove = Params.ProposedMove;

	const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(StartingSyncState);

	FMoverDefaultSyncState& OutputSyncState = OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();

	const float DeltaSeconds = Params.TimeStep.StepMs * 0.001f;
	float PctTimeApplied = 0.f;

	FMovementRecord MoveRecord;
	MoveRecord.SetDeltaSeconds(DeltaSeconds);
	
	UMoverBlackboard* SimBlackboard = MoverComponent->GetSimBlackboard_Mutable();

	SimBlackboard->Invalidate(CommonBlackboard::LastFloorResult);	// falling = no valid floor
	SimBlackboard->Invalidate(CommonBlackboard::LastFoundDynamicMovementBase);

	OutputSyncState.MoveDirectionIntent = (ProposedMove.bHasDirIntent ? ProposedMove.DirectionIntent : FVector::ZeroVector);

	// Use the orientation intent directly. If no intent is provided, use last frame's orientation. Note that we are assuming rotation changes can't fail. 
	const FRotator StartingOrient = StartingSyncState->GetOrientation_WorldSpace();
	const FRotator TargetOrient = UMovementUtils::ApplyAngularVelocityToRotator(StartingOrient, ProposedMove.AngularVelocityDegrees, DeltaSeconds);

	const FVector StartingFallingVelocity = StartingSyncState->GetVelocity_WorldSpace();

	//FVector MoveDelta = 0.5f * (PriorFallingVelocity + ProposedMove.LinearVelocity) * DeltaSeconds; 	// TODO: revive midpoint integration
	const FVector StartLocation = StartingSyncState->GetLocation_WorldSpace();
	const FVector TargetLocation = StartLocation + (ProposedMove.LinearVelocity * DeltaSeconds);

	const FQuat StartRotation = StartingOrient.Quaternion();
	FQuat TargetRotation = TargetOrient.Quaternion();
	if (CommonLegacySettings->bShouldRemainVertical)
	{
		TargetRotation = FRotationMatrix::MakeFromZX(MoverComponent->GetUpDirection(), TargetRotation.GetForwardVector()).ToQuat();
	}


	FHitResult SweepHit(1.f);

	FVector LocationInProgress = StartLocation;
	FQuat   RotationInProgress = StartRotation;

	const FVector MoveDelta = TargetLocation - StartLocation;

	const bool bWouldMove = UAsyncMovementUtils::TestDepenetratingMove(Params.MovingComps, StartLocation, TargetLocation, StartRotation, TargetRotation, /* bShouldSweep */ true, OUT SweepHit, IN OUT MoveRecord);

	LocationInProgress = StartLocation + ((TargetLocation - StartLocation) * SweepHit.Time);
	RotationInProgress = FQuat::Slerp(StartRotation, TargetRotation, SweepHit.Time);

	// Compute final velocity based on how long we actually go until we get a hit.

	FFloorCheckResult LandingFloor;

	FFloorCheckSettings FloorCheckSettings;
	FloorCheckSettings.FloorSweepDistance = CommonLegacySettings->FloorSweepDistance;
	FloorCheckSettings.MaxWalkSlopeCosine = CommonLegacySettings->MaxWalkSlopeCosine;
	FloorCheckSettings.bUseFlatBaseForFloorChecks = CommonLegacySettings->bUseFlatBaseForFloorChecks;
	FloorCheckSettings.PerchRadiusThreshold = CommonLegacySettings->PerchRadiusThreshold;

	// Handle impact, whether it's a landing surface or something to slide on
	if (SweepHit.IsValidBlockingHit())
	{
		float LastMoveTimeSlice = DeltaSeconds;
		float SubTimeTickRemaining = LastMoveTimeSlice * (1.f - SweepHit.Time);

		PctTimeApplied += SweepHit.Time * (1.f - PctTimeApplied);

		// Check for hitting a landing surface
		if (UAirMovementUtils::IsValidLandingSpot(Params.MovingComps, LocationInProgress,
				SweepHit, FloorCheckSettings, LandingFloor))
		{
			// Adjust height to float slightly above walkable floor
			LocationInProgress = UGroundMovementUtils::TestMoveToKeepMinHeightAboveFloor(Params.MovingComps, LocationInProgress, RotationInProgress, CommonLegacySettings->MaxWalkSlopeCosine, IN OUT LandingFloor, IN OUT MoveRecord);
			CaptureFinalState(StartingSyncState, LocationInProgress, RotationInProgress.Rotator(), LandingFloor, DeltaSeconds, DeltaSeconds * PctTimeApplied, ProposedMove.AngularVelocityDegrees, OutputSyncState, OutputState, IN OUT MoveRecord);
			return;
		}

		
		LandingFloor.HitResult = SweepHit;
		SimBlackboard->Set(CommonBlackboard::LastFloorResult, LandingFloor);
		
		FMoverOnImpactParams ImpactParams(DefaultModeNames::Falling, SweepHit, MoveDelta);
		MoverComponent->HandleImpact(ImpactParams);

		// We didn't land on a walkable surface, so let's try to slide along it
		const float PctOfTimeUsedForSliding = UAirMovementUtils::TestFallingMoveAlongHitSurface(
			Params.MovingComps, TargetLocation-StartLocation, LocationInProgress, TargetRotation, 
			/*bHandleImpact=*/true, FloorCheckSettings,
			IN OUT SweepHit, OUT LandingFloor, IN OUT MoveRecord);

		if (PctOfTimeUsedForSliding > 0.f)
		{
			LocationInProgress = SweepHit.TraceStart + ((SweepHit.TraceEnd - SweepHit.TraceStart) * PctOfTimeUsedForSliding);
			RotationInProgress = FQuat::Slerp(RotationInProgress, TargetRotation, PctOfTimeUsedForSliding);
		}

		PctTimeApplied += SweepHit.Time * (1.f - PctTimeApplied);

		if (LandingFloor.IsWalkableFloor())
		{
			// Adjust height to float slightly above walkable floor
			LocationInProgress = UGroundMovementUtils::TestMoveToKeepMinHeightAboveFloor(Params.MovingComps, LocationInProgress, RotationInProgress, CommonLegacySettings->MaxWalkSlopeCosine, IN OUT LandingFloor, IN OUT MoveRecord);
		}
	}
	else
	{
		// This indicates an unimpeded full move
		PctTimeApplied = 1.f;
	}
	
	CaptureFinalState(StartingSyncState, LocationInProgress, RotationInProgress.Rotator(), LandingFloor, DeltaSeconds, DeltaSeconds* PctTimeApplied,ProposedMove.AngularVelocityDegrees, OutputSyncState, OutputState, MoveRecord);
}


void UAsyncFallingMode::OnRegistered(const FName ModeName, const FMoverSimContext& SimContext)
{
	Super::OnRegistered(ModeName, SimContext);

	CommonLegacySettings = GetMoverComponent()->FindSharedSettings<UCommonLegacyMovementSettings>();
	ensureMsgf(CommonLegacySettings.IsValid(), TEXT("Failed to find instance of CommonLegacyMovementSettings on %s. Movement may not function properly."), *GetPathNameSafe(this));
}


void UAsyncFallingMode::OnUnregistered(const FMoverSimContext& SimContext)
{
	CommonLegacySettings = nullptr;

	Super::OnUnregistered(SimContext);
}

void UAsyncFallingMode::ProcessLanded(const FFloorCheckResult& FloorResult, FVector Location, FVector& Velocity, FRelativeBaseInfo& BaseInfo, FMoverTickEndData& TickEndData) const
{
	// TODO: Refactor landed events for async movement. For now, leave the immediate event, but listeners are limited in what they can do from a worker thread.

	const UMoverComponent* MoverComp = GetMoverComponent();
	UMoverBlackboard* SimBlackboard = MoverComp->GetSimBlackboard_Mutable();

	FName NextMovementMode = NAME_None; 
	// if we can walk on the floor we landed on
	if (FloorResult.IsWalkableFloor())
	{
		const FVector UpDir = MoverComp->GetUpDirection();

		if (bCancelVerticalSpeedOnLanding)
		{
			const FPlane MovementPlane(FVector::ZeroVector, UpDir);
			Velocity = UMovementUtils::ConstrainToPlane(Velocity, MovementPlane, false);
		}
		else
		{
			Velocity = FVector::VectorPlaneProject(Velocity, FloorResult.HitResult.Normal);
		}
		
		// Transfer to LandingMovementMode (usually walking), and cache any floor / movement base info
		NextMovementMode = CommonLegacySettings->GroundMovementModeName;

		SimBlackboard->Set(CommonBlackboard::LastFloorResult, FloorResult);

		if (UBasedMovementUtils::IsADynamicBase(FloorResult.HitResult.GetComponent()))
		{
			BaseInfo.SetFromFloorResult(FloorResult, Location, UpDir);
		}
	}
	// we could check for other surfaces here (i.e. when swimming is implemented we can check the floor hit here and see if we need to go into swimming)

	// This would also be a good spot for implementing some falling physics interactions (i.e. falling into a movable object and pushing it based off of this actors velocity)
	
	// if a new mode was set go ahead and switch to it after this tick and broadcast we landed
	if (!NextMovementMode.IsNone())
	{
		TickEndData.MovementEndState.NextModeName = NextMovementMode;
		OnLanded.Broadcast(NextMovementMode, FloorResult.HitResult);
	}
}

void UAsyncFallingMode::CaptureFinalState(const FMoverDefaultSyncState* StartSyncState, const FVector FinalLocation, const FRotator FinalRotation, const FFloorCheckResult& FloorResult, float DeltaSeconds, float DeltaSecondsUsed, const FVector& AngularVelocityDegrees, FMoverDefaultSyncState& OutputSyncState, FMoverTickEndData& TickEndData, FMovementRecord& Record) const
{
	UMoverBlackboard* SimBlackboard = GetMoverComponent()->GetSimBlackboard_Mutable();

	// Check for time refunds
	constexpr float MinRemainingSecondsToRefund = 0.0001f;	// If we have this amount of time (or more) remaining, give it to the next simulation step.

	if ((DeltaSeconds - DeltaSecondsUsed) >= MinRemainingSecondsToRefund)
	{
		const float PctOfTimeRemaining = (1.0f - (DeltaSecondsUsed / DeltaSeconds));
		TickEndData.MovementEndState.RemainingMs = PctOfTimeRemaining * DeltaSeconds * 1000.f;
	}
	else
	{
		TickEndData.MovementEndState.RemainingMs = 0.f;
	}
	
	Record.SetDeltaSeconds( DeltaSecondsUsed );
	
	// If we didn't use any time lets just pass along velocity so we don't lose it when we go into the next mode with refunded time
	FVector EffectiveVelocity = DeltaSecondsUsed <= UE_SMALL_NUMBER ? StartSyncState->GetVelocity_WorldSpace() : Record.GetRelevantVelocity();

	FRelativeBaseInfo MovementBaseInfo;
	ProcessLanded(FloorResult, FinalLocation, EffectiveVelocity, MovementBaseInfo, TickEndData);

	if (MovementBaseInfo.HasRelativeInfo())
	{
		SimBlackboard->Set(CommonBlackboard::LastFoundDynamicMovementBase, MovementBaseInfo);

		OutputSyncState.SetTransforms_WorldSpace( FinalLocation,
												  FinalRotation,
												  EffectiveVelocity,
												  AngularVelocityDegrees,
												  MovementBaseInfo.MovementBase.Get(), MovementBaseInfo.BoneName);
	}
	else
	{
		OutputSyncState.SetTransforms_WorldSpace( FinalLocation,
												  FinalRotation,
												  EffectiveVelocity,
												  AngularVelocityDegrees,
												  nullptr); // no movement base
	}
}
