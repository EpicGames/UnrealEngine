// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultMovementSet/Modes/AsyncFlyingMode.h"
#include "MoveLibrary/AirMovementUtils.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/GroundMovementUtils.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoveLibrary/AsyncMovementUtils.h"
#include "MoverComponent.h"
#include "DefaultMovementSet/Settings/CommonLegacyMovementSettings.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncFlyingMode)


UAsyncFlyingMode::UAsyncFlyingMode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SharedSettingsClasses.Add(UCommonLegacyMovementSettings::StaticClass());
	
	GameplayTags.AddTag(Mover_IsInAir);
	GameplayTags.AddTag(Mover_IsFlying);
}

void UAsyncFlyingMode::GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const
{
	const UMoverComponent* MoverComp = GetMoverComponent();
	const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(StartingSyncState);

	const float DeltaSeconds = TimeStep.StepMs * 0.001f;

	FFreeMoveParams Params;
	if (CharacterInputs)
	{
		Params.MoveInputType = CharacterInputs->GetMoveInputType();
		const bool bMaintainInputMagnitude = true;
		Params.MoveInput = UPlanarConstraintUtils::ConstrainDirectionToPlane(MoverComp->GetPlanarConstraint(), CharacterInputs->GetMoveInput_WorldSpace(), bMaintainInputMagnitude);
	}
	else
	{
		Params.MoveInputType = EMoveInputType::None;
		Params.MoveInput = FVector::ZeroVector;
	}

	FRotator IntendedOrientation_WorldSpace;
	// If there's no intent from input to change orientation, use the current orientation
	if (!CharacterInputs || CharacterInputs->OrientationIntent.IsNearlyZero())
	{
		IntendedOrientation_WorldSpace = StartingSyncState->GetOrientation_WorldSpace();
	}
	else
	{
		IntendedOrientation_WorldSpace = CharacterInputs->GetOrientationIntentDir_WorldSpace().ToOrientationRotator();
	}

	IntendedOrientation_WorldSpace = UMovementUtils::ApplyGravityToOrientationIntent(IntendedOrientation_WorldSpace, MoverComp->GetWorldToGravityTransform(), CommonLegacySettings->bShouldRemainVertical);
	
	Params.OrientationIntent = IntendedOrientation_WorldSpace;
	Params.PriorVelocity = StartingSyncState->GetVelocity_WorldSpace();
	Params.PriorOrientation = StartingSyncState->GetOrientation_WorldSpace();
	Params.TurningRate = CommonLegacySettings->TurningRate;
	Params.TurningBoost = CommonLegacySettings->TurningBoost;
	Params.MaxSpeed = CommonLegacySettings->MaxSpeed;
	Params.Acceleration = CommonLegacySettings->Acceleration;
	Params.Deceleration = CommonLegacySettings->Deceleration;
	Params.DeltaSeconds = DeltaSeconds;
	Params.WorldToGravityQuat = MoverComp->GetWorldToGravityTransform();
	Params.bUseAccelerationForVelocityMove = CommonLegacySettings->bUseAccelerationForVelocityMove;
	
	OutProposedMove = UAirMovementUtils::ComputeControlledFreeMove(Params);
}

void UAsyncFlyingMode::SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState)
{
	const UMoverComponent* MoverComp = GetMoverComponent();
	const FMoverTickStartData& StartState = Params.StartState;
	USceneComponent* UpdatedComponent = Params.MovingComps.UpdatedComponent.Get();
	const FProposedMove& ProposedMove = Params.ProposedMove;

	const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(StartingSyncState);

	FMoverDefaultSyncState& OutputSyncState = OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();

	const float DeltaSeconds = Params.TimeStep.StepMs * 0.001f;

	FMovementRecord MoveRecord;
	MoveRecord.SetDeltaSeconds(DeltaSeconds);

	UMoverBlackboard* SimBlackboard = MoverComp->GetSimBlackboard_Mutable();
	SimBlackboard->Invalidate(CommonBlackboard::LastFloorResult);	// flying = no valid floor
	SimBlackboard->Invalidate(CommonBlackboard::LastFoundDynamicMovementBase);

	OutputSyncState.MoveDirectionIntent = (ProposedMove.bHasDirIntent ? ProposedMove.DirectionIntent : FVector::ZeroVector);

	// Use the orientation intent directly. If no intent is provided, use last frame's orientation. Note that we are assuming rotation changes can't fail. 
	const FRotator StartingOrient = StartingSyncState->GetOrientation_WorldSpace();
	const FRotator TargetOrient = UMovementUtils::ApplyAngularVelocityToRotator(StartingOrient, ProposedMove.AngularVelocityDegrees, DeltaSeconds);
	
	const FVector StartLocation = StartingSyncState->GetLocation_WorldSpace();
	const FVector TargetLocation = StartLocation + (ProposedMove.LinearVelocity * DeltaSeconds);

	const FQuat StartRotation = StartingOrient.Quaternion();
	FQuat TargetRotation = TargetOrient.Quaternion();
	if (CommonLegacySettings->bShouldRemainVertical)
	{
		TargetRotation = FRotationMatrix::MakeFromZX(MoverComp->GetUpDirection(), TargetRotation.GetForwardVector()).ToQuat();
	}

	FHitResult SweepHit(1.f);
	FMovementRecord SweepRecord;
	SweepRecord.SetDeltaSeconds(DeltaSeconds);
	FVector LocationInProgress = StartLocation;
	FQuat   RotationInProgress = StartRotation;

	const bool bWouldMove = UAsyncMovementUtils::TestDepenetratingMove(Params.MovingComps, StartLocation, TargetLocation, StartRotation, TargetRotation, /* bShouldSweep */ true, OUT SweepHit, SweepRecord);

	LocationInProgress = StartLocation + ((TargetLocation - StartLocation) * SweepHit.Time);
	RotationInProgress = FQuat::Slerp(StartRotation, TargetRotation, SweepHit.Time);

	if (SweepHit.IsValidBlockingHit())
	{
		const float PctOfTimeUsedForSliding = UAsyncMovementUtils::TestSlidingMoveAlongHitSurface(Params.MovingComps, TargetLocation - StartLocation, LocationInProgress, TargetRotation, SweepHit, SweepRecord);

		if (PctOfTimeUsedForSliding > 0.f)
		{
			LocationInProgress = SweepHit.TraceStart + ((SweepHit.TraceEnd - SweepHit.TraceStart) * PctOfTimeUsedForSliding);
			RotationInProgress = FQuat::Slerp(RotationInProgress, TargetRotation, PctOfTimeUsedForSliding);
		}
	}

	if (bRespectDistanceOverWalkableSurfaces)
	{
		// If we are very close to a walkable surface, make sure we maintain a small gap over it
		FFloorCheckResult FloorUnderActor;

		FFloorCheckSettings FloorCheckSettings;
		FloorCheckSettings.FloorSweepDistance = CommonLegacySettings->FloorSweepDistance;
		FloorCheckSettings.MaxWalkSlopeCosine = CommonLegacySettings->MaxWalkSlopeCosine;
		FloorCheckSettings.bUseFlatBaseForFloorChecks = CommonLegacySettings->bUseFlatBaseForFloorChecks;
		FloorCheckSettings.PerchRadiusThreshold = CommonLegacySettings->PerchRadiusThreshold;

		UFloorQueryUtils::FindFloor(Params.MovingComps, FloorCheckSettings, LocationInProgress, FloorUnderActor);

		if (FloorUnderActor.IsWalkableFloor())
		{
			LocationInProgress = UGroundMovementUtils::TestMoveToKeepMinHeightAboveFloor(Params.MovingComps, LocationInProgress, RotationInProgress, CommonLegacySettings->MaxWalkSlopeCosine, IN OUT FloorUnderActor, MoveRecord);
		}
	}

	OutputSyncState.SetTransforms_WorldSpace(
		LocationInProgress,
		RotationInProgress.Rotator(),
		SweepRecord.GetRelevantVelocity(),
		ProposedMove.AngularVelocityDegrees,
		nullptr); // no movement base
}


void UAsyncFlyingMode::OnRegistered(const FName ModeName, const FMoverSimContext& SimContext)
{
	Super::OnRegistered(ModeName, SimContext);

	CommonLegacySettings = GetMoverComponent()->FindSharedSettings<UCommonLegacyMovementSettings>();
	ensureMsgf(CommonLegacySettings, TEXT("Failed to find instance of CommonLegacyMovementSettings on %s. Movement may not function properly."), *GetPathNameSafe(this));
}


void UAsyncFlyingMode::OnUnregistered(const FMoverSimContext& SimContext)
{
	CommonLegacySettings = nullptr;

	Super::OnUnregistered(SimContext);
}