// Copyright Epic Games, Inc. All Rights Reserved.


#include "DefaultMovementSet/Modes/AsyncNavWalkingMode.h"
#include "MoverComponent.h"
#include "AI/NavigationSystemBase.h"
#include "AI/Navigation/NavigationDataInterface.h"
#include "Components/ShapeComponent.h"
#include "DefaultMovementSet/NavMoverComponent.h"
#include "DefaultMovementSet/Settings/CommonLegacyMovementSettings.h"
#include "PhysicsEngine/PhysicsBodyInstanceOwnerInterface.h"
#include "MoveLibrary/AsyncMovementUtils.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/GroundMovementUtils.h"
#include "MoveLibrary/ModularMovement.h"
#include "MoveLibrary/MovementUtils.h"
#include "DrawDebugHelpers.h"
#include "NavigationSystem.h"
#include "AI/Navigation/PathFollowingAgentInterface.h"
#include "MoveLibrary/NavMovementUtils.h"
#include "NavMesh/RecastNavMesh.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncNavWalkingMode)

#if UE_DEBUG_RECORDING_ENABLED
static const FName AsyncWalkModeLogCategory = FName("AsyncNavWalkingMode");
#endif

namespace AsyncWalkingModeCVars
{
	float OverrideRaycastInterval = -1.0f;
	FAutoConsoleVariableRef CVarOverrideNavProjectionInterval(TEXT("Mover.AsyncNav.OverrideRaycastInterval"), OverrideRaycastInterval, TEXT(""));

	bool bUseNavMeshNormal = false;
	FAutoConsoleVariableRef CVarUseNavMeshNormal(TEXT("Mover.AsyncNav.UseNavMeshNormal"), bUseNavMeshNormal, TEXT(""));
}

static const FName MoveWithoutNavMeshSubstepName = "MoveWithoutNavMesh";

UAsyncNavWalkingMode::UAsyncNavWalkingMode()
	: bSweepWhileNavWalking(true)
	, bProjectNavMeshWalking(false)
	, NavMeshProjectionHeightScaleUp(0.67f)
	, NavMeshProjectionHeightScaleDown(1.0f)
	, NavMeshProjectionInterval(0.1f)
	, NavMeshProjectionInterpSpeed(12.f)
	, NavMeshProjectionTimer(0)
	, NavMoverComponent(nullptr)
	, NavDataInterface(nullptr)
	, bProjectNavMeshOnBothWorldChannels(true)
{
	SharedSettingsClasses.Add(UCommonLegacyMovementSettings::StaticClass());
	
	GameplayTags.AddTag(Mover_IsOnGround);
	GameplayTags.AddTag(Mover_IsNavWalking);
}

void UAsyncNavWalkingMode::GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const
{
	if (!CommonLegacySettings.IsValid())
	{
		return;
	}

	const UMoverComponent* MoverComp = GetMoverComponent();
	const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(StartingSyncState);

	const float DeltaSeconds = TimeStep.StepMs * 0.001f;
	FFloorCheckResult LastFloorResult;
	FVector MovementNormal;
	FVector UpDirection = MoverComp->GetUpDirection();

	UMoverBlackboard* SimBlackboard = MoverComp->GetSimBlackboard_Mutable();

	// Try to use the floor as the basis for the intended move direction (i.e. try to walk along slopes, rather than into them)
	if (SimBlackboard && SimBlackboard->TryGet(CommonBlackboard::LastFloorResult, LastFloorResult) && LastFloorResult.IsWalkableFloor())
	{
		MovementNormal = LastFloorResult.HitResult.ImpactNormal;
	}
	else
	{
		MovementNormal = MoverComp->GetUpDirection();
	}

	FRotator IntendedOrientation_WorldSpace;
	// If there's no intent from input to change orientation, use the current orientation
	if (!CharacterInputs || CharacterInputs->OrientationIntent.IsNearlyZero())
	{
		IntendedOrientation_WorldSpace = StartingSyncState->GetOrientation_WorldSpace();
	}
	else
	{
		IntendedOrientation_WorldSpace = CharacterInputs->GetOrientationIntentDir_WorldSpace().ToOrientationRotator();
	}

	IntendedOrientation_WorldSpace = UMovementUtils::ApplyGravityToOrientationIntent(IntendedOrientation_WorldSpace, MoverComp->GetWorldToGravityTransform(), CommonLegacySettings->bShouldRemainVertical);

	FGroundMoveParams Params;

	if (CharacterInputs)
	{
		Params.MoveInputType = CharacterInputs->GetMoveInputType();
		Params.MoveInput = CharacterInputs->GetMoveInput_WorldSpace();
	}
	else
	{
		Params.MoveInputType = EMoveInputType::None;
		Params.MoveInput = FVector::ZeroVector;
	}

	Params.OrientationIntent = IntendedOrientation_WorldSpace;
	Params.PriorVelocity = FVector::VectorPlaneProject(StartingSyncState->GetVelocity_WorldSpace(), MovementNormal);
	Params.PriorOrientation = StartingSyncState->GetOrientation_WorldSpace();
	Params.GroundNormal = MovementNormal;
	Params.TurningRate = CommonLegacySettings->TurningRate;
	Params.TurningBoost = CommonLegacySettings->TurningBoost;
	Params.MaxSpeed = CommonLegacySettings->MaxSpeed;
	Params.Acceleration = CommonLegacySettings->Acceleration;
	Params.Deceleration = CommonLegacySettings->Deceleration;
	Params.DeltaSeconds = DeltaSeconds;
	Params.WorldToGravityQuat = MoverComp->GetWorldToGravityTransform();
	Params.UpDirection = UpDirection;
	Params.bUseAccelerationForVelocityMove = CommonLegacySettings->bUseAccelerationForVelocityMove;

	if (Params.MoveInput.SizeSquared() > 0.f && !UMovementUtils::IsExceedingMaxSpeed(Params.PriorVelocity, CommonLegacySettings->MaxSpeed))
	{
		Params.Friction = CommonLegacySettings->GroundFriction;
	}
	else
	{
		Params.Friction = CommonLegacySettings->bUseSeparateBrakingFriction ? CommonLegacySettings->BrakingFriction : CommonLegacySettings->GroundFriction;
		Params.Friction *= CommonLegacySettings->BrakingFrictionFactor;
	}

	OutProposedMove = UGroundMovementUtils::ComputeControlledGroundMove(Params);

	if (TurnGenerator)
	{
		OutProposedMove.AngularVelocityDegrees = ITurnGeneratorInterface::Execute_GetTurn(TurnGenerator, IntendedOrientation_WorldSpace, StartState, *StartingSyncState, TimeStep, OutProposedMove, SimBlackboard);
	}
}

void UAsyncNavWalkingMode::SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState)
{
	SCOPED_NAMED_EVENT(AsyncNavWalkingMode_SimulationTick, FColor::Yellow);

	const UMoverComponent* MoverComp = Cast<UMoverComponent>(GetMoverComponent());
	if (!MoverComp || !NavMoverComponent.IsValid() || !CommonLegacySettings.IsValid())
	{
		OutputState.MovementEndState.bEndedWithNoChanges = true;
		return;
	}

	const FMoverTickStartData& StartState = Params.StartState;
	const FProposedMove& ProposedMove = Params.ProposedMove;
	FVector UpDirection = MoverComp->GetUpDirection();

	const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(StartingSyncState);

	FMoverDefaultSyncState& OutputSyncState = OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();

	const float DeltaSeconds = Params.TimeStep.StepMs * 0.001f;
	const FVector OrigMoveDelta = ProposedMove.LinearVelocity * DeltaSeconds;

	FMovementRecord MoveRecord;
	MoveRecord.SetDeltaSeconds(DeltaSeconds);

	OutputSyncState.MoveDirectionIntent = (ProposedMove.bHasDirIntent ? ProposedMove.DirectionIntent : FVector::ZeroVector);

	const FRotator StartingOrient = StartingSyncState->GetOrientation_WorldSpace();
	
	const FRotator TargetOrient = UMovementUtils::ApplyAngularVelocityToRotator(StartingOrient, ProposedMove.AngularVelocityDegrees, DeltaSeconds);

	const FVector StartingFeetLocation = NavMoverComponent->GetFeetLocationAt(StartingSyncState->GetLocation_WorldSpace());
	FVector TargetFeetLocation = StartingFeetLocation + (ProposedMove.LinearVelocity * DeltaSeconds);

	const FVector StartLocation = StartingSyncState->GetLocation_WorldSpace();

	const FQuat StartRotation = StartingOrient.Quaternion();

	FQuat TargetRotation = TargetOrient.Quaternion();
	if (CommonLegacySettings->bShouldRemainVertical)
	{
		TargetRotation = FRotationMatrix::MakeFromZX(UpDirection, TargetRotation.GetForwardVector()).ToQuat();
	}

	FVector LocationInProgress = StartLocation;
	FQuat RotationInProgress = StartRotation;

	const bool bDeltaMoveNearlyZero = OrigMoveDelta.IsNearlyZero();
	FNavLocation DestNavLocation;

	float SimpleRadius = 0;
	float SimpleHalfHeight = 0;
	NavMoverComponent->GetSimpleCollisionCylinder(OUT SimpleRadius, OUT SimpleHalfHeight);

	if (!NavDataInterface.IsValid())
	{
		NavDataInterface = GetNavData();
	}
	
	bool bSameNavLocation = false;
	if (CachedNavLocation.HasNodeRef())
	{
		if (bProjectNavMeshWalking)
		{
			const float DistSq = UMovementUtils::ProjectToGravityFloor(StartingFeetLocation - CachedNavLocation.Location, UpDirection).SizeSquared();
			const float DistDot = FMath::Abs((StartingFeetLocation - CachedNavLocation.Location).Dot(UpDirection));

			const float TotalCapsuleHeight = SimpleHalfHeight * 2.0f;
			const float ProjectionScale = (StartingFeetLocation.Dot(UpDirection) > CachedNavLocation.Location.Dot(UpDirection)) ? NavMeshProjectionHeightScaleUp : NavMeshProjectionHeightScaleDown;
			const float DistThr = TotalCapsuleHeight * FMath::Max(0.f, ProjectionScale);

			bSameNavLocation = (DistSq <= UE_KINDA_SMALL_NUMBER) && (DistDot < DistThr);
		}
		else
		{
			bSameNavLocation = CachedNavLocation.Location.Equals(StartingFeetLocation);
		}

		if (bDeltaMoveNearlyZero && bSameNavLocation)
		{
			if (NavDataInterface.IsValid())
			{
				if (!NavDataInterface->IsNodeRefValid(CachedNavLocation.NodeRef))
				{
					CachedNavLocation.NodeRef = INVALID_NAVNODEREF;
					bSameNavLocation = false;
				}
			}
		}
	}

	if (bDeltaMoveNearlyZero && bSameNavLocation)
	{
		DestNavLocation = CachedNavLocation;
		UE_LOGF(LogMover, VeryVerbose, "%ls using cached navmesh location! (bProjectNavMeshWalking = %d)", *GetNameSafe(GetMoverComponent()->GetOwner()), bProjectNavMeshWalking);
	}
	else
	{
		// Start the trace from the vertical location of the last valid trace.
		// Otherwise if we are projecting our location to the underlying geometry and it's far above or below the navmesh,
		// we'll follow that geometry's plane out of range of valid navigation.
		if (bSameNavLocation && bProjectNavMeshWalking)
		{
			UMovementUtils::SetGravityVerticalComponent(TargetFeetLocation, CachedNavLocation.Location.Dot(UpDirection), UpDirection);
		}
		
		// Find the point on the NavMesh
		bool bFoundPointOnNavMesh = false;

		if (NavDataInterface.IsValid())
		{
			const IPathFollowingAgentInterface* PathFollowingAgent = NavMoverComponent->GetPathFollowingAgent();
			const bool bIsOnNavLink = PathFollowingAgent && PathFollowingAgent->IsFollowingNavLink();
			
			if (bSlideAlongNavMeshEdge && !bIsOnNavLink)
			{
				FNavLocation StartingNavFloorLocation;
				bool bHasValidCachedNavLocation = NavDataInterface->IsNodeRefValid(CachedNavLocation.NodeRef);

				// If we don't have a valid CachedNavLocation lets try finding the NavFloor where we're currently at and use that otherwise we can just use our CachedNavLocation
				if (!bHasValidCachedNavLocation)
				{
					bHasValidCachedNavLocation = FindNavFloor(StartingFeetLocation, OUT StartingNavFloorLocation, NavDataInterface.Get());
				}
				else
				{
					StartingNavFloorLocation = CachedNavLocation;
				}

				if (bHasValidCachedNavLocation)
				{
					bFoundPointOnNavMesh = NavDataInterface->FindMoveAlongSurface(StartingNavFloorLocation, TargetFeetLocation, OUT DestNavLocation);

					if (bFoundPointOnNavMesh)
					{
						TargetFeetLocation = UMovementUtils::ProjectToGravityFloor(DestNavLocation.Location, UpDirection) + UMovementUtils::GetGravityVerticalComponent(TargetFeetLocation, UpDirection);
					}
				}
			}
			else
			{
				bFoundPointOnNavMesh = FindNavFloor(TargetFeetLocation, OUT DestNavLocation, NavDataInterface.Get());
			}
		}

		if (!bFoundPointOnNavMesh)
		{
			// Can't find nav mesh at this location, so we need to do something else
			switch (BehaviorOffNavMesh)
			{
				default:	// fall through
				case EOffNavMeshBehavior::SwitchToWalking:
					UE_LOGF(LogMover, Verbose, "%ls could not find valid navigation data at location %ls. Switching to walking mode.", *GetNameSafe(MoverComp->GetOwner()), *TargetFeetLocation.ToCompactString());
					OutputState.MovementEndState.NextModeName = DefaultModeNames::Walking;
					OutputState.MovementEndState.RemainingMs = Params.TimeStep.StepMs;
					MoveRecord.SetDeltaSeconds(0.0f);
					break;

				case EOffNavMeshBehavior::MoveWithoutNavMesh:
					// allow the full move to occur 
					LocationInProgress = StartingSyncState->GetLocation_WorldSpace() + (ProposedMove.LinearVelocity* DeltaSeconds);
					RotationInProgress = TargetRotation;
					MoveRecord.Append(FMovementSubstep(MoveWithoutNavMeshSubstepName, ProposedMove.LinearVelocity * DeltaSeconds, true));
					break;

				case EOffNavMeshBehavior::DoNotMove:
					UE_LOGF(LogMover, Verbose, "%ls could not find valid navigation data at location %ls. Cannot move.", *GetNameSafe(MoverComp->GetOwner()), *TargetFeetLocation.ToCompactString());
					// nothing to be done
					break;

				case EOffNavMeshBehavior::RotateOnly:
					RotationInProgress = TargetRotation;
					break;
			}

			CaptureOutputState(*StartingSyncState, LocationInProgress, RotationInProgress.Rotator(), MoveRecord, ProposedMove.AngularVelocityDegrees, OutputSyncState, OutputState);
			return;
		}

		CachedNavLocation = DestNavLocation;
	}

	if (DestNavLocation.NodeRef != INVALID_NAVNODEREF)
	{
		FVector NewFeetLocation = UMovementUtils::ProjectToGravityFloor(TargetFeetLocation, UpDirection) + UMovementUtils::GetGravityVerticalComponent(DestNavLocation.Location, UpDirection);
		if (bProjectNavMeshWalking)
		{
			const float TotalCapsuleHeight = SimpleHalfHeight * 2.0f;
			const float UpOffset = TotalCapsuleHeight * FMath::Max(0.f, NavMeshProjectionHeightScaleUp);
			const float DownOffset = TotalCapsuleHeight * FMath::Max(0.f, NavMeshProjectionHeightScaleDown);
			NewFeetLocation = ProjectLocationFromNavMesh(DeltaSeconds, StartingFeetLocation, NewFeetLocation, UpOffset, DownOffset);
		}
		else
		{
			if (UMoverBlackboard* SimBlackboard = MoverComp->GetSimBlackboard_Mutable())
			{
				const FFloorCheckResult EmptyFloorCheckResult;
				SimBlackboard->Set(CommonBlackboard::LastFloorResult, EmptyFloorCheckResult);
			}
		}

		FVector AdjustedDelta = NewFeetLocation - StartingFeetLocation;

		if (!AdjustedDelta.IsNearlyZero())
		{
			FHitResult MoveHitResult;

			FMoverCollisionParams CollisionParams(Params.MovingComps.UpdatedComponent.Get());

			// Ignore all world geometry while moving on nav mesh
			CollisionParams.ResponseParams.CollisionResponse.SetResponse(ECC_WorldStatic, ECR_Ignore);
			CollisionParams.ResponseParams.CollisionResponse.SetResponse(ECC_WorldDynamic, ECR_Ignore);

			if (UAsyncMovementUtils::TestDepenetratingMove(Params.MovingComps, StartLocation, StartLocation + AdjustedDelta, StartRotation, TargetRotation, bSweepWhileNavWalking, CollisionParams, OUT MoveHitResult, IN OUT MoveRecord))
			{
				LocationInProgress = StartLocation + (AdjustedDelta * MoveHitResult.Time);
				RotationInProgress = FQuat::Slerp(StartRotation, TargetRotation, MoveHitResult.Time);
			}
		}
		else
		{
			// not moving, but let's allow the full rotation
			RotationInProgress = TargetRotation;
		}
	}
	else
	{
		OutputState.MovementEndState.NextModeName = CommonLegacySettings->AirMovementModeName;
		OutputState.MovementEndState.RemainingMs = Params.TimeStep.StepMs;
		MoveRecord.SetDeltaSeconds((Params.TimeStep.StepMs - OutputState.MovementEndState.RemainingMs) * 0.001f);
	}

	CaptureOutputState(*StartingSyncState, LocationInProgress, RotationInProgress.Rotator(), MoveRecord, ProposedMove.AngularVelocityDegrees, OutputSyncState, OutputState);
}

bool UAsyncNavWalkingMode::FindNavFloor(const FVector& TestLocation, FNavLocation& OutNavFloorLocation, const INavigationDataInterface* NavData) const
{
	if (NavData == nullptr || !NavMoverComponent.IsValid())
	{
		return false;
	}

	const FNavAgentProperties& AgentProps = NavMoverComponent->GetNavAgentPropertiesRef();
	const float SearchRadius = AgentProps.AgentRadius * 2.0f;
	const float SearchHeight = AgentProps.AgentHeight * AgentProps.NavWalkingSearchHeightScale;

	return NavData->ProjectPoint(TestLocation, OutNavFloorLocation, FVector(SearchRadius, SearchRadius, SearchHeight));
}

UObject* UAsyncNavWalkingMode::GetTurnGenerator()
{
	return TurnGenerator;
}

void UAsyncNavWalkingMode::SetTurnGeneratorClass(TSubclassOf<UObject> TurnGeneratorClass)
{
	if (TurnGeneratorClass)
	{
		TurnGenerator = NewObject<UObject>(this, TurnGeneratorClass);
	}
	else
	{
		TurnGenerator = nullptr; // Clearing the turn generator is valid - will go back to the default turn generation
	}
}

void UAsyncNavWalkingMode::Activate(const FMoverEventContext& Context, FName PrevModeName, const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, FMoverSyncState* OutSyncState, FMoverAuxStateContext* OutAuxState)
{
	Super::Activate(Context, PrevModeName, SimContext, StartState, OutSyncState, OutAuxState);
	
	if (UMoverComponent* MoverComp = GetMoverComponent())
	{
		if (UMoverBlackboard* SimBlackboard = MoverComp->GetSimBlackboard_Mutable())
		{
			SimBlackboard->Invalidate(CommonBlackboard::LastFloorResult);
		}
	}

	NavDataInterface = GetNavData();
	
	NavMeshProjectionTimer = (NavMeshProjectionInterval > 0.f) ? FMath::FRandRange(-NavMeshProjectionInterval, 0.f) : 0.f;
}


const INavigationDataInterface* UAsyncNavWalkingMode::GetNavData() const
{
	ANavigationData* NavData = nullptr;

	if (const UWorld* World = GetWorld())
	{
		if (const UNavigationSystemV1* NavSys = Cast<UNavigationSystemV1>(World->GetNavigationSystem()))
		{
			if (NavMoverComponent.IsValid())
			{
				const FNavAgentProperties& AgentProps = NavMoverComponent->GetNavAgentPropertiesRef();
				NavData = NavSys->GetNavDataForProps(AgentProps, NavMoverComponent->GetNavLocation());
			}
		}
		else
		{
			NavWalkingUtils::CheckClientSideNavigationSettings(GetMoverComponent());
		}
	}

	return NavData;
}

void UAsyncNavWalkingMode::FindBestNavMeshLocation(const FVector& TraceStart, const FVector& TraceEnd, const FVector& CurrentFeetLocation, const FVector& TargetNavLocation, FHitResult& OutHitResult) const
{
	// raycast to underlying mesh to allow us to more closely follow geometry
	// we use static objects here as a best approximation to accept only objects that
	// influence navmesh generation
	FCollisionQueryParams Params(SCENE_QUERY_STAT(ProjectLocation), false);
	Params.AddIgnoredActor(GetMoverComponent()->GetOwner());
	
	// blocked by world static and optionally world dynamic
	FCollisionResponseParams ResponseParams(ECR_Ignore);
	ResponseParams.CollisionResponse.SetResponse(ECC_WorldStatic, ECR_Overlap);
	ResponseParams.CollisionResponse.SetResponse(ECC_WorldDynamic, bProjectNavMeshOnBothWorldChannels ? ECR_Overlap : ECR_Ignore);

	TArray<FHitResult> MultiTraceHits;
	GetWorld()->LineTraceMultiByChannel(MultiTraceHits, TraceStart, TraceEnd, ECC_WorldStatic, Params, ResponseParams);

	struct FRemoveNotBlockingResponseNavMeshTrace
	{
		FRemoveNotBlockingResponseNavMeshTrace(bool bInCheckOnlyWorldStatic) : bCheckOnlyWorldStatic(bInCheckOnlyWorldStatic) {}

		FORCEINLINE bool operator()(const FHitResult& TestHit) const
		{
			const IPhysicsBodyInstanceOwner* BodyInstanceOwner = IPhysicsBodyInstanceOwner::GetPhysicsBodyInstanceOwnerFromHitResult(TestHit);
			const bool bBlockOnWorldStatic = BodyInstanceOwner && (BodyInstanceOwner->GetCollisionResponseToChannel(ECC_WorldStatic) == ECR_Block);
			const bool bBlockOnWorldDynamic = BodyInstanceOwner && (BodyInstanceOwner->GetCollisionResponseToChannel(ECC_WorldDynamic) == ECR_Block);
			return !bBlockOnWorldStatic && (!bBlockOnWorldDynamic || bCheckOnlyWorldStatic);
		}

		bool bCheckOnlyWorldStatic;
	};

	MultiTraceHits.RemoveAllSwap(FRemoveNotBlockingResponseNavMeshTrace(!bProjectNavMeshOnBothWorldChannels), EAllowShrinking::No);
	if (MultiTraceHits.Num() > 0)
	{
		if (ProjectNavMeshToGeometryBehavior == EProjectNavMeshToGeometryBehavior::UseTopGeometry)
		{
			FVector UpDirection = (TraceStart - TraceEnd).GetSafeNormal();
			MultiTraceHits.Sort([UpDirection](const FHitResult& A, const FHitResult& B)
			{
				return A.ImpactPoint.Dot(UpDirection) > B.ImpactPoint.Dot(UpDirection);
			});
		}
		else if (ProjectNavMeshToGeometryBehavior == EProjectNavMeshToGeometryBehavior::UseClosestGeometry)
		{
			// Sort the hits by the closest to our origin.
			MultiTraceHits.Sort([TargetNavLocation](const FHitResult& A, const FHitResult& B)
			{
				const float ADistSqr = (TargetNavLocation - A.ImpactPoint).SizeSquared();
				const float BDistSqr = (TargetNavLocation - B.ImpactPoint).SizeSquared();

				return (ADistSqr < BDistSqr);	
			});
		}

		OutHitResult = MultiTraceHits[0];
		OutHitResult.bBlockingHit = true;
	}

#if UE_DEBUG_RECORDING_ENABLED
	const float DebugNormalLength = 50.0f;
	UE_VLOG_SEGMENT(GetMoverComponent(), AsyncWalkModeLogCategory, Display, TraceStart, TraceEnd, FColor::Green, TEXT(""));
	for (int i = 0; i < MultiTraceHits.Num(); i++)
	{
		UE_VLOG_SPHERE(GetMoverComponent(), "AsyncNavWalkingMode", Display, MultiTraceHits[i].ImpactPoint, i == 0 ? 5.0f : 2.5f, i == 0 ? FColor::Red : FColor::Yellow, TEXT("%i"), i);
		UE_VLOG_ARROW(GetMoverComponent(), "AsyncNavWalkingMode", Display, MultiTraceHits[i].ImpactPoint, (MultiTraceHits[i].ImpactPoint + DebugNormalLength * MultiTraceHits[i].ImpactNormal), i == 0 ? FColor::Red : FColor::Yellow, TEXT(""));
	}
#endif // UE_DEBUG_RECORDING_ENABLED

	if (AsyncWalkingModeCVars::bUseNavMeshNormal && CachedNavLocation.HasNodeRef() && MultiTraceHits.Num() > 0)
	{
		FVector NavMeshNormal;
		if (NavMovementUtils::CalculateNavMeshNormal(CachedNavLocation, NavMeshNormal, NavDataInterface.Get(), GetMoverComponent()))
		{
			OutHitResult.ImpactNormal = NavMeshNormal;
			OutHitResult.Normal = NavMeshNormal;
			UE_VLOG_ARROW(GetMoverComponent(), "AsyncNavWalkingMode", Display, OutHitResult.ImpactPoint, (OutHitResult.ImpactPoint + DebugNormalLength * OutHitResult.ImpactNormal), FColor::Magenta, TEXT("NavMeshNormal"));
		}
	}
}

FVector UAsyncNavWalkingMode::ProjectLocationFromNavMesh(float DeltaSeconds, const FVector& CurrentFeetLocation, const FVector& TargetNavLocation, float UpOffset, float DownOffset)
{
	FVector NewLocation = TargetNavLocation;

	const float VerticalOffset = -(DownOffset + UpOffset);
	if (VerticalOffset > -UE_SMALL_NUMBER)
	{
		return NewLocation;
	}

	const UMoverComponent* MoverComp = GetMoverComponent();
	const FVector UpDirection = MoverComp->GetUpDirection();

	const FVector TraceStart = TargetNavLocation + UpOffset * UpDirection;
	const FVector TraceEnd = TargetNavLocation + DownOffset * -UpDirection;

	FFloorCheckResult CachedFloorCheckResult;
	UMoverBlackboard* SimBlackboard = MoverComp->GetSimBlackboard_Mutable();
	bool bHasValidFloorResult = SimBlackboard->TryGet(CommonBlackboard::LastFloorResult, CachedFloorCheckResult);
	FHitResult CachedProjectedNavMeshHitResult = CachedFloorCheckResult.HitResult;

	// We can skip this trace if we are checking at the same location as the last trace (ie, we haven't moved).
	const bool bCachedLocationStillValid = (CachedProjectedNavMeshHitResult.bBlockingHit &&
		CachedProjectedNavMeshHitResult.TraceStart == TraceStart &&
		CachedProjectedNavMeshHitResult.TraceEnd == TraceEnd);

	// Check periodically or if we have no information about our last floor result
	UE_VLOG_SPHERE(GetMoverComponent(), AsyncWalkModeLogCategory, Display, TargetNavLocation, 5.0f, FColor::Blue, TEXT("TargetNavLocation"));
	NavMeshProjectionTimer -= DeltaSeconds;
	if (NavMeshProjectionTimer <= 0.0f || !bHasValidFloorResult)
	{
		if (!bCachedLocationStillValid)
		{
			UE_LOGF(LogMover, VeryVerbose, "ProjectLocationFromNavMesh(): %ls interval: %.3f [SKIP TRACE]", *GetNameSafe(GetMoverComponent()->GetOwner()), NavMeshProjectionInterval);

			FHitResult HitResult;
			FindBestNavMeshLocation(TraceStart, TraceEnd, CurrentFeetLocation, TargetNavLocation, HitResult);

			// discard result if we were already inside something
			if (HitResult.bStartPenetrating || !HitResult.bBlockingHit)
			{
				CachedProjectedNavMeshHitResult.Reset();
				const FFloorCheckResult EmptyFloorCheckResult;
				SimBlackboard->Set(CommonBlackboard::LastFloorResult, EmptyFloorCheckResult);
			}
			else
			{
				CachedProjectedNavMeshHitResult = HitResult;
				
				FFloorCheckResult FloorCheckResult;
				FloorCheckResult.bBlockingHit = HitResult.bBlockingHit;
				FloorCheckResult.bLineTrace = true;
				FloorCheckResult.bWalkableFloor = true;
				FloorCheckResult.LineDist = FMath::Abs((CurrentFeetLocation - CachedProjectedNavMeshHitResult.ImpactPoint).Dot(UpDirection));
				FloorCheckResult.FloorDist = FloorCheckResult.LineDist; // This is usually set from a sweep trace but it doesn't really hurt setting it. 
				FloorCheckResult.HitResult = CachedProjectedNavMeshHitResult;
				SimBlackboard->Set(CommonBlackboard::LastFloorResult, FloorCheckResult);
			}
		}
		else
		{
			UE_LOGF(LogMover, VeryVerbose, "ProjectLocationFromNavMesh(): %ls interval: %.3f [SKIP TRACE]", *GetNameSafe(GetMoverComponent()->GetOwner()), NavMeshProjectionInterval);
		}

		// Wrap around to maintain same relative offset to tick time changes.
		// Prevents large framerate spikes from aligning multiple characters to the same frame (if they start staggered, they will now remain staggered).
		float ModTime = 0.f;
		float Interval = AsyncWalkingModeCVars::OverrideRaycastInterval >= 0.0f ? AsyncWalkingModeCVars::OverrideRaycastInterval : NavMeshProjectionInterval;
		if (Interval > UE_SMALL_NUMBER)
		{
			ModTime = FMath::Fmod(-NavMeshProjectionTimer, Interval);
		}

		NavMeshProjectionTimer = Interval - ModTime;
	}

	// Project to last plane we found.
	if (CachedProjectedNavMeshHitResult.bBlockingHit)
	{
		if (bCachedLocationStillValid && FMath::IsNearlyEqual(CurrentFeetLocation.Dot(UpDirection), CachedProjectedNavMeshHitResult.ImpactPoint.Dot(UpDirection), (FVector::FReal)0.01f))
		{
			// Already at destination.
			UMovementUtils::SetGravityVerticalComponent(NewLocation, CurrentFeetLocation.Dot(UpDirection), UpDirection);
		}
		else
		{
			const FVector ProjectedPoint = FMath::LinePlaneIntersection(TraceStart, TraceEnd, CachedProjectedNavMeshHitResult.ImpactPoint, CachedProjectedNavMeshHitResult.ImpactNormal);
			UE_VLOG_SPHERE(GetMoverComponent(), "AsyncNavWalkingMode", Display, ProjectedPoint, 2.5f, FColor::Orange, TEXT("ProjectedPoint"));
			FVector::FReal ProjectedVertical = ProjectedPoint.Dot(UpDirection);

			// Limit to not be too far above or below NavMesh location
			const FVector::FReal VertTraceStart = TraceStart.Dot(UpDirection);
			const FVector::FReal VertTraceEnd = TraceEnd.Dot(UpDirection);
			const FVector::FReal TraceMin = FMath::Min(VertTraceStart, VertTraceEnd);
			const FVector::FReal TraceMax = FMath::Max(VertTraceStart, VertTraceEnd);
			ProjectedVertical = FMath::Clamp(ProjectedVertical, TraceMin, TraceMax);

			// Interp for smoother updates (less "pop" when trace hits something new). 0 interp speed is instant.
			const FVector::FReal InterpSpeed = FMath::Max<FVector::FReal>(0.f, NavMeshProjectionInterpSpeed);
			ProjectedVertical = FMath::FInterpTo(CurrentFeetLocation.Dot(UpDirection), ProjectedVertical, (FVector::FReal)DeltaSeconds, InterpSpeed);
			ProjectedVertical = FMath::Clamp(ProjectedVertical, TraceMin, TraceMax);

			// Final result
			UMovementUtils::SetGravityVerticalComponent(NewLocation, ProjectedVertical, UpDirection);
		}
	}

	return NewLocation;
}

void UAsyncNavWalkingMode::OnRegistered(const FName ModeName, const FMoverSimContext& SimContext)
{
	Super::OnRegistered(ModeName, SimContext);

	UMoverComponent* MoverComponent = GetMoverComponent();
	CommonLegacySettings = MoverComponent->FindSharedSettings<UCommonLegacyMovementSettings>();
	ensureMsgf(CommonLegacySettings.IsValid(), TEXT("Failed to find instance of CommonLegacyMovementSettings on %s. Movement may not function properly."), *GetPathNameSafe(this));

	if (const AActor* MoverCompOwner = MoverComponent->GetOwner())
	{
		NavMoverComponent = MoverCompOwner->FindComponentByClass<UNavMoverComponent>();
	}

	if (!NavMoverComponent.IsValid())
	{
		UE_LOGF(LogMover, Warning, "NavWalkingMode on %ls could not find a valid NavMoverComponent and will not function properly.", *GetNameSafe(GetMoverComponent()->GetOwner()));
	}
}

void UAsyncNavWalkingMode::OnUnregistered(const FMoverSimContext& SimContext)
{
	CommonLegacySettings = nullptr;
	NavMoverComponent = nullptr;
	NavDataInterface = nullptr;

	Super::OnUnregistered(SimContext);
}

void UAsyncNavWalkingMode::CaptureOutputState(const FMoverDefaultSyncState& StartSyncState, const FVector& FinalLocation, const FRotator& FinalRotation, const FMovementRecord& Record, const FVector& AngularVelocityDegrees, FMoverDefaultSyncState& OutputSyncState, FMoverTickEndData& TickEndData) const
{
	UMoverBlackboard* SimBlackboard = GetMoverComponent()->GetSimBlackboard_Mutable();

	SimBlackboard->Invalidate(CommonBlackboard::LastFoundDynamicMovementBase);

	OutputSyncState.SetTransforms_WorldSpace(FinalLocation,
		FinalRotation,
		Record.GetRelevantVelocity(),
		AngularVelocityDegrees,
		nullptr);	// no movement base

	TickEndData.MovementEndState.bEndedWithNoChanges = OutputSyncState.IsNearlyEqual(StartSyncState);
}


