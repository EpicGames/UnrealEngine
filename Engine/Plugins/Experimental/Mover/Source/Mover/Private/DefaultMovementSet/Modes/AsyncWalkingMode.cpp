// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultMovementSet/Modes/AsyncWalkingMode.h"
#include "Components/SkeletalMeshComponent.h"
#include "GameFramework/Pawn.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoveLibrary/AsyncMovementUtils.h"
#include "MoveLibrary/BasedMovementUtils.h"
#include "MoveLibrary/ModularMovement.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/GroundMovementUtils.h"
#include "MoverComponent.h"
#include "DefaultMovementSet/Settings/CommonLegacyMovementSettings.h"
#include "MoverLog.h"
#include "DefaultMovementSet/InstantMovementEffects/BasicInstantMovementEffects.h"
#include "DrawDebugHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncWalkingMode)


UAsyncWalkingMode::UAsyncWalkingMode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SharedSettingsClasses.Add(UCommonLegacyMovementSettings::StaticClass());

	GameplayTags.AddTag(Mover_IsOnGround);
}

void UAsyncWalkingMode::Activate(const FMoverEventContext& Context, FName PrevModeName, const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, FMoverSyncState* OutSyncState, FMoverAuxStateContext* OutAuxState)
{
	Super::Activate(Context, PrevModeName, SimContext, StartState, OutSyncState, OutAuxState);

	// Make sure we test the floor ourselves upon activation
	if (UMoverBlackboard* SimBlackboard = GetMoverComponent()->GetSimBlackboard_Mutable())
	{
		SimBlackboard->Invalidate(CommonBlackboard::LastFloorResult);
	}
}

void UAsyncWalkingMode::GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const
{
	const UMoverComponent* MoverComp = GetMoverComponent();
	const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(StartingSyncState);

	if (!CommonLegacySettings.IsValid())
	{
		return;
	}

	const float DeltaSeconds = TimeStep.StepMs * 0.001f;
	FFloorCheckResult LastFloorResult;
	FVector MovementNormal;

	UMoverBlackboard* SimBlackboard = MoverComp->GetSimBlackboard_Mutable();
	FVector UpDirection = MoverComp->GetUpDirection();

	// Try to use the floor as the basis for the intended move direction (i.e. try to walk along slopes, rather than into them)
	if (SimBlackboard && SimBlackboard->TryGet(CommonBlackboard::LastFloorResult, LastFloorResult) && LastFloorResult.IsWalkableFloor())
	{
		MovementNormal = LastFloorResult.HitResult.ImpactNormal;
	}
	else
	{
		MovementNormal = UpDirection;
	}

	FRotator IntendedOrientation_WorldSpace;
	// If there's no intent from input to change orientation, use the current orientation
	if (!CharacterInputs || CharacterInputs->OrientationIntent.IsNearlyZero())
	{
		IntendedOrientation_WorldSpace = StartingSyncState->GetOrientation_WorldSpace();
	}
	else
	{
		IntendedOrientation_WorldSpace = CharacterInputs->GetOrientationIntentDir_WorldSpace().ToOrientationRotator();
	}

	IntendedOrientation_WorldSpace = UMovementUtils::ApplyGravityToOrientationIntent(IntendedOrientation_WorldSpace, MoverComp->GetWorldToGravityTransform(), CommonLegacySettings->bShouldRemainVertical);
	
	FGroundMoveParams Params;

	if (CharacterInputs)
	{
		Params.MoveInputType = CharacterInputs->GetMoveInputType();

		const bool bMaintainInputMagnitude = true;
		Params.MoveInput = UPlanarConstraintUtils::ConstrainDirectionToPlane(MoverComp->GetPlanarConstraint(), CharacterInputs->GetMoveInput_WorldSpace(), bMaintainInputMagnitude);
	}
	else
	{
		Params.MoveInputType = EMoveInputType::None;
		Params.MoveInput = FVector::ZeroVector;
	}

	Params.OrientationIntent = IntendedOrientation_WorldSpace;
	Params.PriorVelocity = FVector::VectorPlaneProject(StartingSyncState->GetVelocity_WorldSpace(), MovementNormal);
	Params.PriorOrientation = StartingSyncState->GetOrientation_WorldSpace();
	Params.GroundNormal = MovementNormal;
	Params.TurningRate = CommonLegacySettings->TurningRate;
	Params.TurningBoost = CommonLegacySettings->TurningBoost;
	Params.MaxSpeed = CommonLegacySettings->MaxSpeed;
	Params.Acceleration = CommonLegacySettings->Acceleration;
	Params.Deceleration = CommonLegacySettings->Deceleration;
	Params.DeltaSeconds = DeltaSeconds;
	Params.WorldToGravityQuat = MoverComp->GetWorldToGravityTransform();
	Params.UpDirection = UpDirection;
	Params.bUseAccelerationForVelocityMove = CommonLegacySettings->bUseAccelerationForVelocityMove;
	
	if (Params.MoveInput.SizeSquared() > 0.f && !UMovementUtils::IsExceedingMaxSpeed(Params.PriorVelocity, CommonLegacySettings->MaxSpeed))
	{
		Params.Friction = CommonLegacySettings->GroundFriction;
	}
	else
	{
		Params.Friction = CommonLegacySettings->bUseSeparateBrakingFriction ? CommonLegacySettings->BrakingFriction : CommonLegacySettings->GroundFriction;
		Params.Friction *= CommonLegacySettings->BrakingFrictionFactor;
	}

	OutProposedMove = UGroundMovementUtils::ComputeControlledGroundMove(Params);

	if (TurnGenerator)
	{
		OutProposedMove.AngularVelocityDegrees = ITurnGeneratorInterface::Execute_GetTurn(TurnGenerator, IntendedOrientation_WorldSpace, StartState, *StartingSyncState, TimeStep, OutProposedMove, SimBlackboard);
	}
}

void UAsyncWalkingMode::SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState)
{
	if (!CommonLegacySettings.IsValid())
	{
		return;
	}

	UMoverComponent* MoverComp = GetMoverComponent();
	const FMoverTickStartData& StartState = Params.StartState;
	FProposedMove ProposedMove = Params.ProposedMove;

	const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(StartingSyncState);

	FMoverDefaultSyncState& OutputSyncState = OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();


	const float DeltaSeconds = Params.TimeStep.StepMs * 0.001f;

	const FVector OrigMoveDelta = ProposedMove.LinearVelocity * DeltaSeconds;

	const FVector StartLocation = StartingSyncState->GetLocation_WorldSpace();
	const FVector TargetLocation = StartLocation + OrigMoveDelta;


	FMovementRecord MoveRecord;
	MoveRecord.SetDeltaSeconds(DeltaSeconds);

	FFloorCheckResult CurrentFloor;
	UMoverBlackboard* SimBlackboard = MoverComp->GetSimBlackboard_Mutable();

	FVector UpDirection = MoverComp->GetUpDirection();

	FFloorCheckSettings FloorCheckSettings;
	FloorCheckSettings.FloorSweepDistance = CommonLegacySettings->FloorSweepDistance;
	FloorCheckSettings.MaxWalkSlopeCosine = CommonLegacySettings->MaxWalkSlopeCosine;
	FloorCheckSettings.bUseFlatBaseForFloorChecks = CommonLegacySettings->bUseFlatBaseForFloorChecks;
	FloorCheckSettings.PerchRadiusThreshold = CommonLegacySettings->PerchRadiusThreshold;
	
	// If we don't have cached floor information, we need to search for it again
	if (!SimBlackboard->TryGet(CommonBlackboard::LastFloorResult, CurrentFloor))
	{
		UFloorQueryUtils::FindFloor(Params.MovingComps, FloorCheckSettings, StartLocation, CurrentFloor);
	}

	OutputSyncState.MoveDirectionIntent = (ProposedMove.bHasDirIntent ? ProposedMove.DirectionIntent : FVector::ZeroVector);

	const FRotator StartingOrient = StartingSyncState->GetOrientation_WorldSpace();
	const FRotator TargetOrient = UMovementUtils::ApplyAngularVelocityToRotator(StartingOrient, ProposedMove.AngularVelocityDegrees, DeltaSeconds);
	const bool bIsOrientationChanging = !StartingOrient.Equals(TargetOrient);

	const FQuat StartRotation = StartingOrient.Quaternion();

	FQuat TargetRotation = TargetOrient.Quaternion();
	if (CommonLegacySettings->bShouldRemainVertical)
	{
		TargetRotation = FRotationMatrix::MakeFromZX(UpDirection, TargetRotation.GetForwardVector()).ToQuat();
	}

	FVector LocationInProgress = StartLocation;
	FQuat   RotationInProgress = StartRotation;

	FHitResult MoveHitResult(1.f);
	
	FVector CurMoveDelta = OrigMoveDelta;

	bool bDidAttemptMovement = false;

	float PercentTimeAppliedSoFar = MoveHitResult.Time;
	bool bWasFirstMoveBlocked = false;

	if (!CurMoveDelta.IsNearlyZero() || bIsOrientationChanging)
	{
		// Attempt to move the full amount first
		bDidAttemptMovement = true;

		const bool bWouldMove = UAsyncMovementUtils::TestDepenetratingMove(Params.MovingComps, StartLocation, TargetLocation, StartRotation, TargetRotation, /* bShouldSweep */ true, OUT MoveHitResult, MoveRecord);

		float LastMoveSeconds = DeltaSeconds;

		LocationInProgress = StartLocation + ((TargetLocation - StartLocation) * MoveHitResult.Time);
		RotationInProgress = FQuat::Slerp(StartRotation, TargetRotation, MoveHitResult.Time);

		if (MoveHitResult.bStartPenetrating)
		{
			// We started by being stuck in geometry and need to resolve it first
			// TODO: try to resolve starting stuck in geometry
		}
		else if (MoveHitResult.IsValidBlockingHit())
		{
			// We impacted something (possibly a ramp, possibly a barrier)
			PercentTimeAppliedSoFar = MoveHitResult.Time;		

			// Check if the blockage is a walkable ramp rising in front of us
			if ((MoveHitResult.Time > 0.f) && (MoveHitResult.Normal.Dot(UpDirection) > UE_KINDA_SMALL_NUMBER) && 
			    UFloorQueryUtils::IsHitSurfaceWalkable(MoveHitResult, UpDirection, CommonLegacySettings->MaxWalkSlopeCosine))
			{
				// It's a walkable ramp, so cut up the move and attempt to move the remainder of it along the ramp's surface, possibly generating another hit
				const float PercentTimeRemaining = 1.f - PercentTimeAppliedSoFar;
				CurMoveDelta = UGroundMovementUtils::ComputeDeflectedMoveOntoRamp(CurMoveDelta * PercentTimeRemaining, UpDirection, MoveHitResult, CommonLegacySettings->MaxWalkSlopeCosine, CurrentFloor.bLineTrace);

				UAsyncMovementUtils::TestDepenetratingMove(Params.MovingComps, LocationInProgress, LocationInProgress+CurMoveDelta, RotationInProgress, TargetRotation, /*bShouldSweep=*/ true, OUT MoveHitResult, MoveRecord);

				LastMoveSeconds = PercentTimeRemaining * LastMoveSeconds;

				LocationInProgress = LocationInProgress + ((MoveHitResult.TraceEnd - MoveHitResult.TraceStart) * MoveHitResult.Time);
				RotationInProgress = FQuat::Slerp(RotationInProgress, TargetRotation, MoveHitResult.Time);

				const float SecondHitPercent = MoveHitResult.Time * PercentTimeRemaining;
				PercentTimeAppliedSoFar = FMath::Clamp(PercentTimeAppliedSoFar + SecondHitPercent, 0.f, 1.f);
			}

			if (MoveHitResult.IsValidBlockingHit())
			{
	
				// If still blocked, try to step up onto the blocking object OR slide along it
				// TODO: Take movement bases into account
				if (UGroundMovementUtils::CanStepUpOnHitSurface(MoveHitResult)) // || (CharacterOwner->GetMovementBase() != nullptr && Hit.HitObjectHandle == CharacterOwner->GetMovementBase()->GetOwner()))
				{
					// hit a barrier or unwalkable surface, try to step up and onto it
					const FVector PreStepUpLocation = LocationInProgress;
					const FVector DownwardDir = -UpDirection;

					FOptionalFloorCheckResult StepUpFloorResult;	// passed to sub-operations, so we can use their final floor results if they did a test
					FVector PostStepUpLocation; // Valid if step-up succeeded

					float StepUpMovePercentage = UGroundMovementUtils::TestMoveToStepOver(Params.MovingComps, DownwardDir, CommonLegacySettings->MaxStepHeight, FloorCheckSettings, OrigMoveDelta * (1.f - PercentTimeAppliedSoFar), RotationInProgress, MoveHitResult, CurrentFloor, false, &StepUpFloorResult, PostStepUpLocation, MoveRecord);
					PercentTimeAppliedSoFar = FMath::Clamp(PercentTimeAppliedSoFar + (StepUpMovePercentage * (1.f - PercentTimeAppliedSoFar)), 0.f, 1.f);
					if (StepUpMovePercentage > 0)
					{
						LocationInProgress = PostStepUpLocation;
						RotationInProgress = TargetRotation;
						PercentTimeAppliedSoFar = 1.0f;
					}
					else
					{
						FMoverOnImpactParams ImpactParams(DefaultModeNames::Walking, MoveHitResult, OrigMoveDelta);
						MoverComp->HandleImpact(ImpactParams);
						float PercentAvailableToSlide = 1.f - PercentTimeAppliedSoFar;

						float SlideAmount = UGroundMovementUtils::TestGroundedMoveAlongHitSurface(Params.MovingComps, OrigMoveDelta, LocationInProgress, TargetRotation, /*bHandleImpact=*/true, CommonLegacySettings->MaxStepHeight, CommonLegacySettings->MaxWalkSlopeCosine, IN OUT MoveHitResult, IN OUT MoveRecord);

						LocationInProgress = LocationInProgress + ((MoveHitResult.TraceEnd - MoveHitResult.TraceStart) * SlideAmount);
						RotationInProgress = FQuat::Slerp(RotationInProgress, TargetRotation, SlideAmount);
						PercentTimeAppliedSoFar += PercentAvailableToSlide * SlideAmount;
					}
				}
				else if (MoveHitResult.Component.IsValid() && !MoveHitResult.Component.Get()->CanCharacterStepUp(Cast<APawn>(MoveHitResult.GetActor())))
				{
					FMoverOnImpactParams ImpactParams(DefaultModeNames::Walking, MoveHitResult, OrigMoveDelta);
					MoverComp->HandleImpact(ImpactParams);
					float PercentAvailableToSlide = 1.f - PercentTimeAppliedSoFar;
					
					float SlideAmount = UGroundMovementUtils::TestGroundedMoveAlongHitSurface(Params.MovingComps, OrigMoveDelta, LocationInProgress, TargetRotation, /*bHandleImpact=*/true, CommonLegacySettings->MaxStepHeight, CommonLegacySettings->MaxWalkSlopeCosine, IN OUT MoveHitResult, IN OUT MoveRecord);

					LocationInProgress = LocationInProgress + ((MoveHitResult.TraceEnd - MoveHitResult.TraceStart) * SlideAmount);
					RotationInProgress = FQuat::Slerp(RotationInProgress, TargetRotation, SlideAmount);


					PercentTimeAppliedSoFar += PercentAvailableToSlide * SlideAmount;
				}
			}
		}

		// Search for the floor we've ended up on
		UFloorQueryUtils::FindFloor(Params.MovingComps, FloorCheckSettings, LocationInProgress, CurrentFloor);

		if (CurrentFloor.IsWalkableFloor())
		{
			LocationInProgress = UGroundMovementUtils::TestMoveToAdjustToFloor(Params.MovingComps, LocationInProgress, RotationInProgress, CommonLegacySettings->MaxWalkSlopeCosine, IN OUT CurrentFloor, MoveRecord);
		}
    
		if (!CurrentFloor.IsWalkableFloor() && !CurrentFloor.HitResult.bStartPenetrating)
		{
			// No floor or not walkable, so let's let the airborne movement mode deal with it
			OutputState.MovementEndState.NextModeName = CommonLegacySettings->AirMovementModeName;
			OutputState.MovementEndState.RemainingMs = Params.TimeStep.StepMs - (Params.TimeStep.StepMs * PercentTimeAppliedSoFar);
			MoveRecord.SetDeltaSeconds((Params.TimeStep.StepMs - OutputState.MovementEndState.RemainingMs) * 0.001f);
			CaptureFinalState(StartingSyncState, LocationInProgress, RotationInProgress.Rotator(), bDidAttemptMovement, CurrentFloor, MoveRecord, ProposedMove.AngularVelocityDegrees, OutputSyncState);
			return;
		}
	}
	else
	{
		/* TODO: Support option to perform floor checks even when not moving
		// If the actor isn't moving we still may need to check if they have a valid floor, such as if they're on an elevator platform moving up/down
		if ((FloorCheckPolicy == EStaticFloorCheckPolicy::Always) || 
		    (FloorCheckPolicy == EStaticFloorCheckPolicy::OnDynamicBaseOnly && StartingSyncState->GetMovementBase()))
		{
			UFloorQueryUtils::FindFloor(Params.MovingComps, FloorCheckSettings, LocationInProgress, CurrentFloor);
		
			FHitResult Hit(CurrentFloor.HitResult);
			if (Hit.bStartPenetrating)
			{
				// The floor check failed because it started in penetration
				// We do not want to try to move downward because the downward sweep failed, rather we'd like to try to pop out of the floor.
				Hit.TraceEnd = Hit.TraceStart + UpDirection * 2.4;
				FVector RequestedAdjustment = UMovementUtils::ComputePenetrationAdjustment(Hit);
			
				const EMoveComponentFlags IncludeBlockingOverlapsWithoutEvents = (MOVECOMP_NeverIgnoreBlockingOverlaps | MOVECOMP_DisableBlockingOverlapDispatch);
				EMoveComponentFlags MoveComponentFlags = MOVECOMP_NoFlags;
				MoveComponentFlags = (MoveComponentFlags | IncludeBlockingOverlapsWithoutEvents);
				UMovementUtils::TryMoveToResolvePenetration(Params.MovingComps, MoveComponentFlags, RequestedAdjustment, Hit, RotationInProgress, MoveRecord);

				//TODO: Update Location/RotationInProgress
			}
		
			if (!CurrentFloor.IsWalkableFloor() && !Hit.bStartPenetrating)
			{
				// No floor or not walkable, so let's let the airborne movement mode deal with it
				OutputState.MovementEndState.NextModeName = CommonLegacySettings->AirMovementModeName;
				OutputState.MovementEndState.RemainingMs = Params.TimeStep.StepMs;
				MoveRecord.SetDeltaSeconds((Params.TimeStep.StepMs - OutputState.MovementEndState.RemainingMs) * 0.001f);
				CaptureFinalState(StartingSyncState, LocationInProgress, RotationInProgress.Rotator(), bDidAttemptMovement, CurrentFloor, MoveRecord, OutputSyncState);
				return;
			}
		}
		*/
	}

	CaptureFinalState(StartingSyncState, LocationInProgress, RotationInProgress.Rotator(), bDidAttemptMovement, CurrentFloor, MoveRecord, ProposedMove.AngularVelocityDegrees, OutputSyncState);

}

UObject* UAsyncWalkingMode::GetTurnGenerator()
{
	return TurnGenerator;
}

void UAsyncWalkingMode::SetTurnGeneratorClass(TSubclassOf<UObject> TurnGeneratorClass)
{
	if (TurnGeneratorClass)
	{
		TurnGenerator = NewObject<UObject>(this, TurnGeneratorClass);
	}
	else
	{
		TurnGenerator = nullptr; // Clearing the turn generator is valid - will go back to the default turn generation
	}
}


void UAsyncWalkingMode::OnRegistered(const FName ModeName, const FMoverSimContext& SimContext)
{
	Super::OnRegistered(ModeName, SimContext);

	CommonLegacySettings = GetMoverComponent()->FindSharedSettings<UCommonLegacyMovementSettings>();
	ensureMsgf(CommonLegacySettings.IsValid(), TEXT("Failed to find instance of CommonLegacyMovementSettings on %s. Movement may not function properly."), *GetPathNameSafe(this));
}

void UAsyncWalkingMode::OnUnregistered(const FMoverSimContext& SimContext)
{
	CommonLegacySettings = nullptr;

	Super::OnUnregistered(SimContext);
}

void UAsyncWalkingMode::CaptureFinalState(const FMoverDefaultSyncState* StartingState, const FVector FinalLocation, const FRotator FinalRotation, bool bDidAttemptMovement, const FFloorCheckResult& FloorResult, const FMovementRecord& Record, const FVector& AngularVelocityDegrees, FMoverDefaultSyncState& OutputSyncState) const
{
	FRelativeBaseInfo PriorBaseInfo;

	const UMoverComponent* MoverComp = GetMoverComponent();
	UMoverBlackboard* SimBlackboard = MoverComp->GetSimBlackboard_Mutable();

	const bool bHasPriorBaseInfo = SimBlackboard->TryGet(CommonBlackboard::LastFoundDynamicMovementBase, PriorBaseInfo);

	FRelativeBaseInfo CurrentBaseInfo = UpdateFloorAndBaseInfo(FloorResult, FinalLocation);

	// If we're on a dynamic base and we're not trying to move, keep using the same relative actor location. This prevents slow relative 
	//  drifting that can occur from repeated floor sampling as the base moves through the world.
	if (CurrentBaseInfo.HasRelativeInfo() 
		&& bHasPriorBaseInfo && !bDidAttemptMovement 
		&& PriorBaseInfo.UsesSameBase(CurrentBaseInfo))
	{
		CurrentBaseInfo.ContactLocalPosition = PriorBaseInfo.ContactLocalPosition;
		CurrentBaseInfo.WorldspaceOffsetFromContactPos = PriorBaseInfo.WorldspaceOffsetFromContactPos;
	}
	
	if (CurrentBaseInfo.HasRelativeInfo())
	{
		SimBlackboard->Set(CommonBlackboard::LastFoundDynamicMovementBase, CurrentBaseInfo);

		OutputSyncState.SetTransforms_WorldSpace( FinalLocation,
                                                  FinalRotation,
                                                  Record.GetRelevantVelocity(),
                                                  AngularVelocityDegrees,
                                                  CurrentBaseInfo.MovementBase.Get(), CurrentBaseInfo.BoneName);
	}
	else
	{
		SimBlackboard->Invalidate(CommonBlackboard::LastFoundDynamicMovementBase);

		FVector NewWorldVelocity = Record.GetRelevantVelocity();

		// If we're walking off a movement base, keep the base's velocity so we carry momentum
		if (const UPrimitiveComponent* BaseComp = StartingState->GetMovementBase())
		{
			FVector DepartingBaseVelocity;
			if (UBasedMovementUtils::GetMovementBaseVelocityAtPoint(BaseComp, FinalLocation, OUT DepartingBaseVelocity))
			{
				NewWorldVelocity += DepartingBaseVelocity;
			}
		}

		OutputSyncState.SetTransforms_WorldSpace( FinalLocation,
                                                  FinalRotation,
												  NewWorldVelocity,
                                                  AngularVelocityDegrees,
                                                  nullptr);  // no movement base
	}
}


FRelativeBaseInfo UAsyncWalkingMode::UpdateFloorAndBaseInfo(const FFloorCheckResult& FloorResult, FVector CurrentLocation) const
{
	FRelativeBaseInfo ReturnBaseInfo;

	const UMoverComponent* MoverComp = GetMoverComponent();
	UMoverBlackboard* SimBlackboard = MoverComp->GetSimBlackboard_Mutable();

	SimBlackboard->Set(CommonBlackboard::LastFloorResult, FloorResult);

	if (FloorResult.IsWalkableFloor() && UBasedMovementUtils::IsADynamicBase(FloorResult.HitResult.GetComponent()))
	{
		ReturnBaseInfo.SetFromFloorResult(FloorResult, CurrentLocation, MoverComp->GetUpDirection());
	}

	return ReturnBaseInfo;
}
