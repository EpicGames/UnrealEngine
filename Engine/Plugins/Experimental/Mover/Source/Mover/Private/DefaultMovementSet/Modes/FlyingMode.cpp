// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultMovementSet/Modes/FlyingMode.h"
#include "MoveLibrary/AirMovementUtils.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/GroundMovementUtils.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoverComponent.h"
#include "DefaultMovementSet/Settings/CommonLegacyMovementSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FlyingMode)


UFlyingMode::UFlyingMode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SharedSettingsClasses.Add(UCommonLegacyMovementSettings::StaticClass());
	
	GameplayTags.AddTag(Mover_IsInAir);
	GameplayTags.AddTag(Mover_IsFlying);
}

void UFlyingMode::GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const
{
	const UMoverComponent* MoverComp = GetMoverComponent();
	const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(StartingSyncState);

	const float DeltaSeconds = TimeStep.StepMs * 0.001f;

	FFreeMoveParams Params;
	if (CharacterInputs)
	{
		Params.MoveInputType = CharacterInputs->GetMoveInputType();
		const bool bMaintainInputMagnitude = true;
		Params.MoveInput = UPlanarConstraintUtils::ConstrainDirectionToPlane(MoverComp->GetPlanarConstraint(), CharacterInputs->GetMoveInput_WorldSpace(), bMaintainInputMagnitude);
	}
	else
	{
		Params.MoveInputType = EMoveInputType::None;
		Params.MoveInput = FVector::ZeroVector;
	}

	FRotator IntendedOrientation_WorldSpace;
	// If there's no intent from input to change orientation, use the current orientation
	if (!CharacterInputs || CharacterInputs->OrientationIntent.IsNearlyZero())
	{
		IntendedOrientation_WorldSpace = StartingSyncState->GetOrientation_WorldSpace();
	}
	else
	{
		IntendedOrientation_WorldSpace = CharacterInputs->GetOrientationIntentDir_WorldSpace().ToOrientationRotator();
	}

	IntendedOrientation_WorldSpace = UMovementUtils::ApplyGravityToOrientationIntent(IntendedOrientation_WorldSpace, MoverComp->GetWorldToGravityTransform(), CommonLegacySettings->bShouldRemainVertical);
	
	Params.OrientationIntent = IntendedOrientation_WorldSpace;
	Params.PriorVelocity = StartingSyncState->GetVelocity_WorldSpace();
	Params.PriorOrientation = StartingSyncState->GetOrientation_WorldSpace();
	Params.TurningRate = CommonLegacySettings->TurningRate;
	Params.TurningBoost = CommonLegacySettings->TurningBoost;
	Params.MaxSpeed = CommonLegacySettings->MaxSpeed;
	Params.Acceleration = CommonLegacySettings->Acceleration;
	Params.Deceleration = CommonLegacySettings->Deceleration;
	Params.DeltaSeconds = DeltaSeconds;
	Params.WorldToGravityQuat = MoverComp->GetWorldToGravityTransform();
	Params.bUseAccelerationForVelocityMove = CommonLegacySettings->bUseAccelerationForVelocityMove;
	
	OutProposedMove = UAirMovementUtils::ComputeControlledFreeMove(Params);
}

void UFlyingMode::SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState)
{
	UMoverComponent* MoverComp = GetMoverComponent();
	const FMoverTickStartData& StartState = Params.StartState;
	USceneComponent* UpdatedComponent = Params.MovingComps.UpdatedComponent.Get();
	FProposedMove ProposedMove = Params.ProposedMove;

	if (!UpdatedComponent)
	{
		return;
	}

	const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(StartingSyncState);

	FMoverDefaultSyncState& OutputSyncState = OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();

	const float DeltaSeconds = Params.TimeStep.StepMs * 0.001f;

	FMovementRecord MoveRecord;
	MoveRecord.SetDeltaSeconds(DeltaSeconds);

	UMoverBlackboard* SimBlackboard = MoverComp->GetSimBlackboard_Mutable();

	SimBlackboard->Invalidate(CommonBlackboard::LastFloorResult);	// flying = no valid floor
	SimBlackboard->Invalidate(CommonBlackboard::LastFoundDynamicMovementBase);

	OutputSyncState.MoveDirectionIntent = (ProposedMove.bHasDirIntent ? ProposedMove.DirectionIntent : FVector::ZeroVector);

	// Use the orientation intent directly. If no intent is provided, use last frame's orientation. Note that we are assuming rotation changes can't fail. 
	const FRotator StartingOrient = StartingSyncState->GetOrientation_WorldSpace();

	const FRotator TargetOrient = UMovementUtils::ApplyAngularVelocityToRotator(StartingOrient, ProposedMove.AngularVelocityDegrees, DeltaSeconds);
	const bool bIsOrientationChanging = !StartingOrient.Equals(TargetOrient);
	
	FVector MoveDelta = ProposedMove.LinearVelocity * DeltaSeconds;

	FQuat TargetOrientQuat = TargetOrient.Quaternion();
	if (CommonLegacySettings->bShouldRemainVertical)
	{
		TargetOrientQuat = FRotationMatrix::MakeFromZX(MoverComp->GetUpDirection(), TargetOrientQuat.GetForwardVector()).ToQuat();
	}

	FHitResult Hit(1.f);

	if (!MoveDelta.IsNearlyZero() || bIsOrientationChanging)
	{
		UMovementUtils::TrySafeMoveUpdatedComponent(Params.MovingComps, MoveDelta, TargetOrientQuat, true, Hit, ETeleportType::None, MoveRecord);
	}

	if (Hit.IsValidBlockingHit())
	{
		FMoverOnImpactParams ImpactParams(DefaultModeNames::Flying, Hit, MoveDelta);
		MoverComp->HandleImpact(ImpactParams);

		// Try to slide the remaining distance along the surface.
		UMovementUtils::TryMoveToSlideAlongSurface(FMovingComponentSet(MoverComp), MoveDelta, 1.f - Hit.Time, TargetOrientQuat, Hit.Normal, Hit, true, MoveRecord);
	}

	if (bRespectDistanceOverWalkableSurfaces)
	{
		// If we are very close to a walkable surface, make sure we maintain a small gap over it
		FFloorCheckResult FloorUnderActor;

		FFloorCheckSettings FloorCheckSettings;
		FloorCheckSettings.FloorSweepDistance = CommonLegacySettings->FloorSweepDistance;
		FloorCheckSettings.MaxWalkSlopeCosine = CommonLegacySettings->MaxWalkSlopeCosine;
		FloorCheckSettings.bUseFlatBaseForFloorChecks = CommonLegacySettings->bUseFlatBaseForFloorChecks;
		FloorCheckSettings.PerchRadiusThreshold = CommonLegacySettings->PerchRadiusThreshold;

		UFloorQueryUtils::FindFloor(Params.MovingComps, FloorCheckSettings, UpdatedComponent->GetComponentLocation(), OUT FloorUnderActor);

		if (FloorUnderActor.IsWalkableFloor())
		{
			UGroundMovementUtils::TryMoveToKeepMinHeightAboveFloor(MoverComp, FloorUnderActor, CommonLegacySettings->MaxWalkSlopeCosine, MoveRecord);
		}
	}

	CaptureFinalState(UpdatedComponent, MoveRecord, *StartingSyncState, ProposedMove.AngularVelocityDegrees, OutputSyncState, DeltaSeconds);
}

// TODO: replace this function with simply looking at/collapsing the MovementRecord
void UFlyingMode::CaptureFinalState(USceneComponent* UpdatedComponent, FMovementRecord& Record, const FMoverDefaultSyncState& StartSyncState, const FVector& AngularVelocityDegrees, FMoverDefaultSyncState& OutputSyncState, const float DeltaSeconds) const
{
	const FVector FinalLocation = UpdatedComponent->GetComponentLocation();
	const FVector FinalVelocity = Record.GetRelevantVelocity();
	
	// TODO: Update Main/large movement record with substeps from our local record

	OutputSyncState.SetTransforms_WorldSpace(FinalLocation,
											  UpdatedComponent->GetComponentRotation(),
											  FinalVelocity,
											  AngularVelocityDegrees,
											  nullptr); // no movement base

	UpdatedComponent->ComponentVelocity = FinalVelocity;
}

void UFlyingMode::OnRegistered(const FName ModeName, const FMoverSimContext& SimContext)
{
	Super::OnRegistered(ModeName, SimContext);

	CommonLegacySettings = GetMoverComponent()->FindSharedSettings<UCommonLegacyMovementSettings>();
	ensureMsgf(CommonLegacySettings, TEXT("Failed to find instance of CommonLegacyMovementSettings on %s. Movement may not function properly."), *GetPathNameSafe(this));
}


void UFlyingMode::OnUnregistered(const FMoverSimContext& SimContext)
{
	CommonLegacySettings = nullptr;

	Super::OnUnregistered(SimContext);
}
