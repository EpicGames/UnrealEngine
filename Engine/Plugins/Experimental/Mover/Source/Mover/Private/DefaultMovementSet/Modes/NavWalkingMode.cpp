// Copyright Epic Games, Inc. All Rights Reserved.


#include "DefaultMovementSet/Modes/NavWalkingMode.h"
#include "MoverComponent.h"
#include "NavigationSystem.h"
#include "AI/NavigationSystemBase.h"
#include "AI/Navigation/NavigationDataInterface.h"
#include "AI/Navigation/PathFollowingAgentInterface.h"
#include "Components/ShapeComponent.h"
#include "DefaultMovementSet/NavMoverComponent.h"
#include "DefaultMovementSet/Settings/CommonLegacyMovementSettings.h"
#include "Misc/ConfigCacheIni.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/GroundMovementUtils.h"
#include "MoveLibrary/ModularMovement.h"
#include "MoveLibrary/MovementUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavWalkingMode)


void NavWalkingUtils::CheckClientSideNavigationSettings(const UMoverComponent* MoverComponent)
{
	if (MoverComponent && MoverComponent->GetNetMode() == NM_Client)
	{
		bool bClientSideNavigationAllowed = false;
		GConfig->GetBool(TEXT("/Script/NavigationSystem"), TEXT("bAllowClientSideNavigation"), bClientSideNavigationAllowed, GEngineIni);
				
		ensureMsgf(bClientSideNavigationAllowed, TEXT("NavWalking Mode on %s expected a valid navigation system on clients but bAllowClientSideNavigation wasn't enabled. Enable bAllowClientSideNavigation if NavWalking needs to run on Clients."), *GetNameSafe(MoverComponent->GetOwner()));
	}
}

UNavWalkingMode::UNavWalkingMode()
	: bSweepWhileNavWalking(true)
	, bProjectNavMeshWalking(false)
	, NavMeshProjectionHeightScaleUp(0.67f)
	, NavMeshProjectionHeightScaleDown(1.0f)
	, NavMeshProjectionInterval(0.1f)
	, NavMeshProjectionInterpSpeed(12.f)
	, NavMeshProjectionTimer(0)
	, NavMoverComponent(nullptr)
	, NavDataInterface(nullptr)
	, bProjectNavMeshOnBothWorldChannels(true)
{
	SharedSettingsClasses.Add(UCommonLegacyMovementSettings::StaticClass());
	
	GameplayTags.AddTag(Mover_IsOnGround);
	GameplayTags.AddTag(Mover_IsNavWalking);
}

void UNavWalkingMode::GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const
{
	const UMoverComponent* MoverComp = GetMoverComponent();
	const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(StartingSyncState);

	const float DeltaSeconds = TimeStep.StepMs * 0.001f;
	FFloorCheckResult LastFloorResult;
	FVector MovementNormal;
	FVector UpDirection = MoverComp->GetUpDirection();

	UMoverBlackboard* SimBlackboard = MoverComp->GetSimBlackboard_Mutable();

	// Try to use the floor as the basis for the intended move direction (i.e. try to walk along slopes, rather than into them)
	if (SimBlackboard && SimBlackboard->TryGet(CommonBlackboard::LastFloorResult, LastFloorResult) && LastFloorResult.IsWalkableFloor())
	{
		MovementNormal = LastFloorResult.HitResult.ImpactNormal;
	}
	else
	{
		MovementNormal = MoverComp->GetUpDirection();
	}
	
	FRotator IntendedOrientation_WorldSpace;
	// If there's no intent from input to change orientation, use the current orientation
	if (!CharacterInputs || CharacterInputs->OrientationIntent.IsNearlyZero())
	{
		IntendedOrientation_WorldSpace = StartingSyncState->GetOrientation_WorldSpace();
	}
	else
	{
		IntendedOrientation_WorldSpace = CharacterInputs->GetOrientationIntentDir_WorldSpace().ToOrientationRotator();
	}

	IntendedOrientation_WorldSpace = UMovementUtils::ApplyGravityToOrientationIntent(IntendedOrientation_WorldSpace, MoverComp->GetWorldToGravityTransform(), CommonLegacySettings->bShouldRemainVertical);
	
	FGroundMoveParams Params;

	if (CharacterInputs)
	{
		Params.MoveInputType = CharacterInputs->GetMoveInputType();
		Params.MoveInput = CharacterInputs->GetMoveInput_WorldSpace();
	}
	else
	{
		Params.MoveInputType = EMoveInputType::None;
		Params.MoveInput = FVector::ZeroVector;
	}

	Params.OrientationIntent = IntendedOrientation_WorldSpace;
	Params.PriorVelocity = FVector::VectorPlaneProject(StartingSyncState->GetVelocity_WorldSpace(), MovementNormal);
	Params.PriorOrientation = StartingSyncState->GetOrientation_WorldSpace();
	Params.GroundNormal = MovementNormal;
	Params.TurningRate = CommonLegacySettings->TurningRate;
	Params.TurningBoost = CommonLegacySettings->TurningBoost;
	Params.MaxSpeed = CommonLegacySettings->MaxSpeed;
	Params.Acceleration = CommonLegacySettings->Acceleration;
	Params.Deceleration = CommonLegacySettings->Deceleration;
	Params.DeltaSeconds = DeltaSeconds;
	Params.WorldToGravityQuat = MoverComp->GetWorldToGravityTransform();
	Params.UpDirection = UpDirection;
	Params.bUseAccelerationForVelocityMove = CommonLegacySettings->bUseAccelerationForVelocityMove;

	if (Params.MoveInput.SizeSquared() > 0.f && !UMovementUtils::IsExceedingMaxSpeed(Params.PriorVelocity, CommonLegacySettings->MaxSpeed))
	{
		Params.Friction = CommonLegacySettings->GroundFriction;
	}
	else
	{
		Params.Friction = CommonLegacySettings->bUseSeparateBrakingFriction ? CommonLegacySettings->BrakingFriction : CommonLegacySettings->GroundFriction;
		Params.Friction *= CommonLegacySettings->BrakingFrictionFactor;
	}

	OutProposedMove = UGroundMovementUtils::ComputeControlledGroundMove(Params);

	if (TurnGenerator)
	{
		OutProposedMove.AngularVelocityDegrees = ITurnGeneratorInterface::Execute_GetTurn(TurnGenerator, IntendedOrientation_WorldSpace, StartState, *StartingSyncState, TimeStep, OutProposedMove, SimBlackboard);
	}
}

void UNavWalkingMode::SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState)
{
	const UMoverComponent* MoverComp = Cast<UMoverComponent>(GetMoverComponent());
	if (!ensureMsgf(MoverComp, TEXT("Nav Walking Mode couldn't find a valid MoverComponent!")))
	{
		return;
	}
	
	const FMoverTickStartData& StartState = Params.StartState;
	USceneComponent* UpdatedComponent = Params.MovingComps.UpdatedComponent.Get();
	UPrimitiveComponent* UpdatedPrimitive = Params.MovingComps.UpdatedPrimitive.Get();
	const FProposedMove& ProposedMove = Params.ProposedMove;
	const FVector UpDirection = MoverComp->GetUpDirection();

	if (!UpdatedComponent || !UpdatedPrimitive)
	{
		return;
	}

	const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(StartingSyncState);

	FMoverDefaultSyncState& OutputSyncState = OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();
	
	const float DeltaSeconds = Params.TimeStep.StepMs * 0.001f;
	const FVector OrigMoveDelta = ProposedMove.LinearVelocity * DeltaSeconds;

	TObjectPtr<AActor> OwnerActor = UpdatedComponent->GetOwner();
	check(OwnerActor);

	FMovementRecord MoveRecord;
	MoveRecord.SetDeltaSeconds(DeltaSeconds);

	OutputSyncState.MoveDirectionIntent = (ProposedMove.bHasDirIntent ? ProposedMove.DirectionIntent : FVector::ZeroVector);

	const FRotator StartingOrient = StartingSyncState->GetOrientation_WorldSpace();
	const FRotator TargetOrient = UMovementUtils::ApplyAngularVelocityToRotator(StartingOrient, ProposedMove.AngularVelocityDegrees, DeltaSeconds);
	const bool bIsOrientationChanging = !StartingOrient.Equals(TargetOrient);
	
	FQuat TargetOrientQuat = TargetOrient.Quaternion();
	if (CommonLegacySettings->bShouldRemainVertical)
	{
		TargetOrientQuat = FRotationMatrix::MakeFromZX(UpDirection, TargetOrientQuat.GetForwardVector()).ToQuat();
	}
	
	FVector StartingLocation = NavMoverComponent->GetFeetLocation();
	FVector AdjustedDest = StartingLocation + OrigMoveDelta;

	const bool bDeltaMoveNearlyZero = OrigMoveDelta.IsNearlyZero();
	FNavLocation DestNavLocation;

	float SimpleRadius = 0;
	float SimpleHalfHeight = 0;
	NavMoverComponent->GetSimpleCollisionCylinder(SimpleRadius, SimpleHalfHeight);
	
	if (!NavDataInterface.IsValid())
	{
		NavDataInterface = GetNavData();
	}
	
	bool bSameNavLocation = false;
	if (CachedNavLocation.NodeRef != INVALID_NAVNODEREF)
	{
		if (bProjectNavMeshWalking)
		{
			const float DistSq = UMovementUtils::ProjectToGravityFloor(StartingLocation - CachedNavLocation.Location, UpDirection).SizeSquared();
			const float DistDot = FMath::Abs((StartingLocation - CachedNavLocation.Location).Dot(UpDirection));
			
			const float TotalCapsuleHeight = SimpleHalfHeight * 2.0f;
			const float ProjectionScale = (StartingLocation.Dot(UpDirection) > CachedNavLocation.Location.Dot(UpDirection)) ? NavMeshProjectionHeightScaleUp : NavMeshProjectionHeightScaleDown;
			const float DistThr = TotalCapsuleHeight * FMath::Max(0.f, ProjectionScale);

			bSameNavLocation = (DistSq <= UE_KINDA_SMALL_NUMBER) && (DistDot < DistThr);
		}
		else
		{
			bSameNavLocation = CachedNavLocation.Location.Equals(StartingLocation);
		}

		if (bDeltaMoveNearlyZero && bSameNavLocation)
		{
			if (NavDataInterface.IsValid())
			{
				if (!NavDataInterface->IsNodeRefValid(CachedNavLocation.NodeRef))
				{
					CachedNavLocation.NodeRef = INVALID_NAVNODEREF;
					bSameNavLocation = false;
				}
			}
		}
	}

	if (bDeltaMoveNearlyZero && bSameNavLocation)
	{
		DestNavLocation = CachedNavLocation;
		UE_LOGF(LogMover, VeryVerbose, "%ls using cached navmesh location! (bProjectNavMeshWalking = %d)", *GetNameSafe(GetMoverComponent()->GetOwner()), bProjectNavMeshWalking);
	}
	else
	{
		// Start the trace from the vertical location of the last valid trace.
		// Otherwise if we are projecting our location to the underlying geometry and it's far above or below the navmesh,
		// we'll follow that geometry's plane out of range of valid navigation.
		if (bSameNavLocation && bProjectNavMeshWalking)
		{
			UMovementUtils::SetGravityVerticalComponent(AdjustedDest, CachedNavLocation.Location.Dot(UpDirection), UpDirection);
		}
		
		// Find the point on the NavMesh
		bool bFoundPointOnNavMesh = false;

		if (NavDataInterface.IsValid())
		{
			const IPathFollowingAgentInterface* PathFollowingAgent = NavMoverComponent->GetPathFollowingAgent();
			const bool bIsOnNavLink = PathFollowingAgent && PathFollowingAgent->IsFollowingNavLink();
			
			if (bSlideAlongNavMeshEdge && !bIsOnNavLink)
			{
				FNavLocation StartingNavFloorLocation;
				bool bHasValidCachedNavLocation = NavDataInterface->IsNodeRefValid(CachedNavLocation.NodeRef);

				// If we don't have a valid CachedNavLocation lets try finding the NavFloor where we're currently at and use that otherwise we can just use our CachedNavLocation
				if (!bHasValidCachedNavLocation)
				{
					bHasValidCachedNavLocation = FindNavFloor(StartingLocation, OUT StartingNavFloorLocation, NavDataInterface.Get());
				}
				else
				{
					StartingNavFloorLocation = CachedNavLocation;
				}

				if (bHasValidCachedNavLocation)
				{
					bFoundPointOnNavMesh = NavDataInterface->FindMoveAlongSurface(StartingNavFloorLocation, AdjustedDest, OUT DestNavLocation);

					if (bFoundPointOnNavMesh)
					{
						AdjustedDest = UMovementUtils::ProjectToGravityFloor(DestNavLocation.Location, UpDirection) + UMovementUtils::GetGravityVerticalComponent(AdjustedDest, UpDirection);
					}
				}
			}
			else
			{
				bFoundPointOnNavMesh = FindNavFloor(AdjustedDest, DestNavLocation, NavDataInterface.Get());
			}
		}


		if (!bFoundPointOnNavMesh)
		{
			FHitResult MoveHitResult;
			// Can't find nav mesh at this location, so we need to do something else
			switch (BehaviorOffNavMesh)
			{
				default:	// fall through
				case EOffNavMeshBehavior::SwitchToWalking:
					UE_LOGF(LogMover, Verbose, "%ls could not find valid navigation data at location %ls. Switching to walking mode.", *GetNameSafe(MoverComp->GetOwner()), *AdjustedDest.ToCompactString());
					OutputState.MovementEndState.NextModeName = DefaultModeNames::Walking;
					OutputState.MovementEndState.RemainingMs = Params.TimeStep.StepMs;
					MoveRecord.SetDeltaSeconds(0.0f);
					break;

				case EOffNavMeshBehavior::MoveWithoutNavMesh:
					// allow the full move to occur 
					UMovementUtils::TrySafeMoveUpdatedComponent(Params.MovingComps, OrigMoveDelta, TargetOrientQuat, /*bSweep?*/ false, MoveHitResult, ETeleportType::None, MoveRecord);
					break;

				case EOffNavMeshBehavior::DoNotMove:
					UE_LOGF(LogMover, Verbose, "%ls could not find valid navigation data at location %ls. Cannot move.", *GetNameSafe(MoverComp->GetOwner()), *AdjustedDest.ToCompactString());
					// nothing to be done
					break;

				case EOffNavMeshBehavior::RotateOnly:
					UMovementUtils::TrySafeMoveUpdatedComponent(Params.MovingComps, FVector::ZeroVector, TargetOrientQuat, /*bSweep?*/ false, MoveHitResult, ETeleportType::None, MoveRecord);
					break;
			}

			CaptureFinalState(UpdatedComponent, MoveRecord, ProposedMove.AngularVelocityDegrees, OutputSyncState);
			return;
		}

		CachedNavLocation = DestNavLocation;
	}
	
	if (DestNavLocation.NodeRef != INVALID_NAVNODEREF)
	{
		FVector NewLocation = UMovementUtils::ProjectToGravityFloor(AdjustedDest, UpDirection) + UMovementUtils::GetGravityVerticalComponent(DestNavLocation.Location, UpDirection);
		if (bProjectNavMeshWalking)
		{
			const float TotalCapsuleHeight = SimpleHalfHeight * 2.0f;
			const float UpOffset = TotalCapsuleHeight * FMath::Max(0.f, NavMeshProjectionHeightScaleUp);
			const float DownOffset = TotalCapsuleHeight * FMath::Max(0.f, NavMeshProjectionHeightScaleDown);
			NewLocation = ProjectLocationFromNavMesh(DeltaSeconds, StartingLocation, NewLocation, UpOffset, DownOffset);
		}
		else
		{
			if (UMoverBlackboard* SimBlackboard = MoverComp->GetSimBlackboard_Mutable())
			{
				const FFloorCheckResult EmptyFloorCheckResult;
				SimBlackboard->Set(CommonBlackboard::LastFloorResult, EmptyFloorCheckResult);
			}	
		}

		FVector AdjustedDelta = NewLocation - StartingLocation;
		
		if (!AdjustedDelta.IsNearlyZero() || bIsOrientationChanging)
		{
			FHitResult MoveHitResult;
			UMovementUtils::TrySafeMoveUpdatedComponent(Params.MovingComps, AdjustedDelta, TargetOrientQuat, bSweepWhileNavWalking, MoveHitResult, ETeleportType::None, MoveRecord);
		}
	}
	else
	{
		// Can't find nav destination, so revert to a different mode and let it process the intended movement
		OutputState.MovementEndState.NextModeName = CommonLegacySettings->AirMovementModeName;
		OutputState.MovementEndState.RemainingMs = Params.TimeStep.StepMs;
		MoveRecord.SetDeltaSeconds(0.0f);
	}

	CaptureFinalState(UpdatedComponent, MoveRecord, ProposedMove.AngularVelocityDegrees, OutputSyncState);
}

bool UNavWalkingMode::FindNavFloor(const FVector& TestLocation, FNavLocation& OutNavFloorLocation, const INavigationDataInterface* NavData) const
{
	if (NavData == nullptr || NavMoverComponent == nullptr)
	{
		return false;
	}

	const FNavAgentProperties& AgentProps = NavMoverComponent->GetNavAgentPropertiesRef();
	const float SearchRadius = AgentProps.AgentRadius * 2.0f;
	const float SearchHeight = AgentProps.AgentHeight * AgentProps.NavWalkingSearchHeightScale;

	return NavData->ProjectPoint(TestLocation, OutNavFloorLocation, FVector(SearchRadius, SearchRadius, SearchHeight));
}

UObject* UNavWalkingMode::GetTurnGenerator()
{
	return TurnGenerator;
}

void UNavWalkingMode::SetTurnGeneratorClass(TSubclassOf<UObject> TurnGeneratorClass)
{
	if (TurnGeneratorClass)
	{
		TurnGenerator = NewObject<UObject>(this, TurnGeneratorClass);
	}
	else
	{
		TurnGenerator = nullptr; // Clearing the turn generator is valid - will go back to the default turn generation
	}
}

void UNavWalkingMode::SetCollisionForNavWalking(bool bEnable)
{
	if (const UMoverComponent* MoverComponent = GetMoverComponent())
	{
		if (UPrimitiveComponent* UpdatedCompAsPrimitive = Cast<UPrimitiveComponent>(MoverComponent->GetUpdatedComponent()))
		{
			if (bEnable)
			{
				CollideVsWorldStatic = UpdatedCompAsPrimitive->GetCollisionResponseToChannel(ECC_WorldStatic);
				CollideVsWorldDynamic = UpdatedCompAsPrimitive->GetCollisionResponseToChannel(ECC_WorldDynamic);

				// TODO NS: Eventually might want make the new collision response an option so they trigger certain things while still not colliding 
				UpdatedCompAsPrimitive->SetCollisionResponseToChannel(ECC_WorldStatic, ECR_Ignore);
				UpdatedCompAsPrimitive->SetCollisionResponseToChannel(ECC_WorldDynamic, ECR_Ignore);

				if (UNavWalkingMode* NavWalkingMode = Cast<UNavWalkingMode>(MoverComponent->FindMovementMode(UNavWalkingMode::StaticClass())))
				{
					if (UMoverBlackboard* SimBlackboard = MoverComponent->GetSimBlackboard_Mutable())
					{
						const FFloorCheckResult EmptyFloorCheckResult;
						SimBlackboard->Set(CommonBlackboard::LastFloorResult, EmptyFloorCheckResult);
					}

					// Stagger timed updates so many different characters spawned at the same time don't update on the same frame.
					// Initially we want an immediate update though, so set time to a negative randomized range.
					NavWalkingMode->NavMeshProjectionTimer = (NavWalkingMode->NavMeshProjectionInterval > 0.f) ? FMath::FRandRange(-NavWalkingMode->NavMeshProjectionInterval, 0.f) : 0.f;
				}
			}
			else
			{
				// Grabbing the original shape settings and reverting to our old collision responses
				if (const UShapeComponent* OriginalShapeComp = UMovementUtils::GetOriginalComponentType<UShapeComponent>(MoverComponent->GetOwner()))
				{
					CollideVsWorldStatic = OriginalShapeComp->GetCollisionResponseToChannel(ECC_WorldStatic);
					CollideVsWorldDynamic = OriginalShapeComp->GetCollisionResponseToChannel(ECC_WorldDynamic);
				}

				UpdatedCompAsPrimitive->SetCollisionResponseToChannel(ECC_WorldStatic, CollideVsWorldStatic);
				UpdatedCompAsPrimitive->SetCollisionResponseToChannel(ECC_WorldDynamic, CollideVsWorldDynamic);
			}
		}
	}
}

void UNavWalkingMode::Activate(const FMoverEventContext& Context, FName PrevModeName, const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, FMoverSyncState* OutSyncState, FMoverAuxStateContext* OutAuxState)
{
	Super::Activate(Context, PrevModeName, SimContext, StartState, OutSyncState, OutAuxState);
	SetCollisionForNavWalking(true);

	NavDataInterface = GetNavData();
}

void UNavWalkingMode::Deactivate(const FMoverEventContext& Context, FName NextModeName, const FMoverSimContext& SimContext)
{
	SetCollisionForNavWalking(false);
	Super::Deactivate(Context, NextModeName, SimContext);
}

const INavigationDataInterface* UNavWalkingMode::GetNavData() const
{
	ANavigationData* NavData = nullptr;
	
	if (const UWorld* World = GetWorld())
	{
		if (const UNavigationSystemV1* NavSys = Cast<UNavigationSystemV1>(World->GetNavigationSystem()))
		{
			if (NavMoverComponent)
			{
				const FNavAgentProperties& AgentProps = NavMoverComponent->GetNavAgentPropertiesRef();
				NavData = NavSys->GetNavDataForProps(AgentProps, NavMoverComponent->GetNavLocation());
			}
		}
		else
		{
			NavWalkingUtils::CheckClientSideNavigationSettings(GetMoverComponent());
		}
	}
	
	return NavData;
}

void UNavWalkingMode::FindBestNavMeshLocation(const FVector& TraceStart, const FVector& TraceEnd, const FVector& CurrentFeetLocation, const FVector& TargetNavLocation, FHitResult& OutHitResult) const
{
	// raycast to underlying mesh to allow us to more closely follow geometry
	// we use static objects here as a best approximation to accept only objects that
	// influence navmesh generation
	FCollisionQueryParams Params(SCENE_QUERY_STAT(ProjectLocation), false);

	// blocked by world static and optionally world dynamic
	FCollisionResponseParams ResponseParams(ECR_Ignore);
	ResponseParams.CollisionResponse.SetResponse(ECC_WorldStatic, ECR_Overlap);
	ResponseParams.CollisionResponse.SetResponse(ECC_WorldDynamic, bProjectNavMeshOnBothWorldChannels ? ECR_Overlap : ECR_Ignore);

	TArray<FHitResult> MultiTraceHits;
	GetWorld()->LineTraceMultiByChannel(MultiTraceHits, TraceStart, TraceEnd, ECC_WorldStatic, Params, ResponseParams);

	struct FRemoveNotBlockingResponseNavMeshTrace
	{
		FRemoveNotBlockingResponseNavMeshTrace(bool bInCheckOnlyWorldStatic) : bCheckOnlyWorldStatic(bInCheckOnlyWorldStatic) {}

		FORCEINLINE bool operator()(const FHitResult& TestHit) const
		{
			UPrimitiveComponent* PrimComp = TestHit.GetComponent();
			const bool bBlockOnWorldStatic = PrimComp && (PrimComp->GetCollisionResponseToChannel(ECC_WorldStatic) == ECR_Block);
			const bool bBlockOnWorldDynamic = PrimComp && (PrimComp->GetCollisionResponseToChannel(ECC_WorldDynamic) == ECR_Block);

			return !bBlockOnWorldStatic && (!bBlockOnWorldDynamic || bCheckOnlyWorldStatic);
		}

		bool bCheckOnlyWorldStatic;
	};

	MultiTraceHits.RemoveAllSwap(FRemoveNotBlockingResponseNavMeshTrace(!bProjectNavMeshOnBothWorldChannels), EAllowShrinking::No);
	if (MultiTraceHits.Num() > 0)
	{
		if (ProjectNavMeshToGeometryBehavior == EProjectNavMeshToGeometryBehavior::UseTopGeometry)
		{
			FVector UpDirection = (TraceStart - TraceEnd).GetSafeNormal();
			MultiTraceHits.Sort([UpDirection](const FHitResult& A, const FHitResult& B)
			{
				return A.ImpactPoint.Dot(UpDirection) > B.ImpactPoint.Dot(UpDirection);
			});
		}
		else if (ProjectNavMeshToGeometryBehavior == EProjectNavMeshToGeometryBehavior::UseClosestGeometry)
		{
			// Sort the hits by the closest to our origin.
			MultiTraceHits.Sort([TargetNavLocation](const FHitResult& A, const FHitResult& B)
			{
				const float ADistSqr = (TargetNavLocation - A.ImpactPoint).SizeSquared();
				const float BDistSqr = (TargetNavLocation - B.ImpactPoint).SizeSquared();

				return (ADistSqr < BDistSqr);	
			});
		}

		OutHitResult = MultiTraceHits[0];
		OutHitResult.bBlockingHit = true;
	}
}

FVector UNavWalkingMode::ProjectLocationFromNavMesh(float DeltaSeconds, const FVector& CurrentFeetLocation, const FVector& TargetNavLocation, float UpOffset, float DownOffset)
{
	FVector NewLocation = TargetNavLocation;

	const float VerticalOffset = -(DownOffset + UpOffset);
	if (VerticalOffset > -UE_SMALL_NUMBER)
	{
		return NewLocation;
	}

	const UMoverComponent* MoverComp = GetMoverComponent();
	const FVector UpDirection = MoverComp->GetUpDirection();
	
	const FVector TraceStart = TargetNavLocation + UpOffset * UpDirection;
	const FVector TraceEnd   = TargetNavLocation + DownOffset * -UpDirection;

	FFloorCheckResult CachedFloorCheckResult;
	UMoverBlackboard* SimBlackboard = MoverComp->GetSimBlackboard_Mutable();
	bool bHasValidFloorResult = SimBlackboard->TryGet(CommonBlackboard::LastFloorResult, CachedFloorCheckResult);
	FHitResult CachedProjectedNavMeshHitResult = CachedFloorCheckResult.HitResult;
	
	// We can skip this trace if we are checking at the same location as the last trace (ie, we haven't moved).
	const bool bCachedLocationStillValid = (CachedProjectedNavMeshHitResult.bBlockingHit &&
											CachedProjectedNavMeshHitResult.TraceStart == TraceStart &&
											CachedProjectedNavMeshHitResult.TraceEnd == TraceEnd);

	// Check periodically or if we have no information about our last floor result
	NavMeshProjectionTimer -= DeltaSeconds;
	if (NavMeshProjectionTimer <= 0.0f || !bHasValidFloorResult)
	{
		if (!bCachedLocationStillValid)
		{
			UE_LOGF(LogMover, VeryVerbose, "ProjectLocationFromNavMesh(): %ls interval: %.3f [SKIP TRACE]", *GetNameSafe(GetMoverComponent()->GetOwner()), NavMeshProjectionInterval);

			FHitResult HitResult;
			FindBestNavMeshLocation(TraceStart, TraceEnd, CurrentFeetLocation, TargetNavLocation, HitResult);

			// discard result if we were already inside something
			if (HitResult.bStartPenetrating || !HitResult.bBlockingHit)
			{
				CachedProjectedNavMeshHitResult.Reset();
				const FFloorCheckResult EmptyFloorCheckResult;
				SimBlackboard->Set(CommonBlackboard::LastFloorResult, EmptyFloorCheckResult);
			}
			else
			{
				CachedProjectedNavMeshHitResult = HitResult;
				
				FFloorCheckResult FloorCheckResult;
				FloorCheckResult.bBlockingHit = HitResult.bBlockingHit;
				FloorCheckResult.bLineTrace = true;
				FloorCheckResult.bWalkableFloor = true;
				FloorCheckResult.LineDist = FMath::Abs((CurrentFeetLocation - CachedProjectedNavMeshHitResult.ImpactPoint).Dot(UpDirection));
				FloorCheckResult.FloorDist = FloorCheckResult.LineDist; // This is usually set from a sweep trace but it doesn't really hurt setting it. 
				FloorCheckResult.HitResult = HitResult;
				SimBlackboard->Set(CommonBlackboard::LastFloorResult, FloorCheckResult);
			}
		}
		else
		{
			UE_LOGF(LogMover, VeryVerbose, "ProjectLocationFromNavMesh(): %ls interval: %.3f [SKIP TRACE]", *GetNameSafe(GetMoverComponent()->GetOwner()), NavMeshProjectionInterval);
		}

		// Wrap around to maintain same relative offset to tick time changes.
		// Prevents large framerate spikes from aligning multiple characters to the same frame (if they start staggered, they will now remain staggered).
		float ModTime = 0.f;
		if (NavMeshProjectionInterval > UE_SMALL_NUMBER)
		{
			ModTime = FMath::Fmod(-NavMeshProjectionTimer, NavMeshProjectionInterval);
		}

		NavMeshProjectionTimer = NavMeshProjectionInterval - ModTime;
	}
	
	// Project to last plane we found.
	if (CachedProjectedNavMeshHitResult.bBlockingHit)
	{
		if (bCachedLocationStillValid && FMath::IsNearlyEqual(CurrentFeetLocation.Dot(UpDirection), CachedProjectedNavMeshHitResult.ImpactPoint.Dot(UpDirection), (FVector::FReal)0.01f))
		{
			// Already at destination.
			UMovementUtils::SetGravityVerticalComponent(NewLocation, CurrentFeetLocation.Dot(UpDirection), UpDirection);
		}
		else
		{
			const FVector ProjectedPoint = FMath::LinePlaneIntersection(TraceStart, TraceEnd, CachedProjectedNavMeshHitResult.ImpactPoint, CachedProjectedNavMeshHitResult.Normal);
			FVector::FReal ProjectedVertical = ProjectedPoint.Dot(UpDirection);
			
			// Limit to not be too far above or below NavMesh location
			const FVector::FReal VertTraceStart = TraceStart.Dot(UpDirection);
			const FVector::FReal VertTraceEnd = TraceEnd.Dot(UpDirection);
			const FVector::FReal TraceMin = FMath::Min(VertTraceStart, VertTraceEnd);
			const FVector::FReal TraceMax = FMath::Max(VertTraceStart, VertTraceEnd);
			ProjectedVertical = FMath::Clamp(ProjectedVertical, TraceMin, TraceMax);

			// Interp for smoother updates (less "pop" when trace hits something new). 0 interp speed is instant.
			const FVector::FReal InterpSpeed = FMath::Max<FVector::FReal>(0.f, NavMeshProjectionInterpSpeed);
			ProjectedVertical = FMath::FInterpTo(CurrentFeetLocation.Dot(UpDirection), ProjectedVertical, (FVector::FReal)DeltaSeconds, InterpSpeed);
			ProjectedVertical = FMath::Clamp(ProjectedVertical, TraceMin, TraceMax);
			
			// Final result
			UMovementUtils::SetGravityVerticalComponent(NewLocation, ProjectedVertical, UpDirection);
		}
	}

	return NewLocation;
}

void UNavWalkingMode::OnRegistered(const FName ModeName, const FMoverSimContext& SimContext)
{
	Super::OnRegistered(ModeName, SimContext);

	UMoverComponent* MoverComponent = GetMoverComponent();
	CommonLegacySettings = MoverComponent->FindSharedSettings<UCommonLegacyMovementSettings>();
	ensureMsgf(CommonLegacySettings, TEXT("Failed to find instance of CommonLegacyMovementSettings on %s. Movement may not function properly."), *GetPathNameSafe(this));
	
	if (const AActor* MoverCompOwner = MoverComponent->GetOwner())
	{
		NavMoverComponent = MoverCompOwner->FindComponentByClass<UNavMoverComponent>();
	}
	
	if (!NavMoverComponent)
	{
		UE_LOGF(LogMover, Warning, "NavWalkingMode on %ls could not find a valid NavMoverComponent and will not function properly.", *GetNameSafe(GetMoverComponent()->GetOwner()));
	}
}

void UNavWalkingMode::OnUnregistered(const FMoverSimContext& SimContext)
{
	CommonLegacySettings = nullptr;
	NavDataInterface = nullptr;

	Super::OnUnregistered(SimContext);
}

void UNavWalkingMode::CaptureFinalState(USceneComponent* UpdatedComponent, const FMovementRecord& Record, const FVector& AngularVelocityDegrees, FMoverDefaultSyncState& OutputSyncState) const
{
	UMoverBlackboard* SimBlackboard = GetMoverComponent()->GetSimBlackboard_Mutable();

	SimBlackboard->Invalidate(CommonBlackboard::LastFoundDynamicMovementBase);

	OutputSyncState.SetTransforms_WorldSpace(UpdatedComponent->GetComponentLocation(),
		UpdatedComponent->GetComponentRotation(),
		Record.GetRelevantVelocity(),
		AngularVelocityDegrees,
		nullptr);	// no movement base

	UpdatedComponent->ComponentVelocity = OutputSyncState.GetVelocity_WorldSpace();
}



