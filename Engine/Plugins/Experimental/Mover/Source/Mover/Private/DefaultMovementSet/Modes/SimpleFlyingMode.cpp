// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultMovementSet/Modes/SimpleFlyingMode.h"

#include "MoverComponent.h"
#include "DefaultMovementSet/Settings/CommonLegacyMovementSettings.h"
#include "MoveLibrary/AirMovementUtils.h"
#include "MoveLibrary/MovementUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimpleFlyingMode)

void USimpleFlyingMode::GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep,
                                                   FProposedMove& OutProposedMove) const
{
	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();

	if (CommonLegacySettings.Get() == nullptr || !StartingSyncState)
	{
		return;
	}

	const float DeltaSeconds = TimeStep.StepMs * 0.001f;
	if (DeltaSeconds <= FLT_EPSILON)
	{
		return;
	}

	// Get input
	FVector DesiredVelocity;
	EMoveInputType MoveInputType;
	FVector DesiredFacingDir;
	 
	if (const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>())
	{
		DesiredVelocity = CharacterInputs->GetMoveInput_WorldSpace();
		MoveInputType = CharacterInputs->GetMoveInputType();
		DesiredFacingDir = CharacterInputs->GetOrientationIntentDir_WorldSpace();
	}
	else
	{
		// If no input found, most likely a networked sim proxy
		// Try to deduce an input from the sync state
		DesiredVelocity = StartingSyncState->GetIntent_WorldSpace();
		MoveInputType = EMoveInputType::DirectionalIntent;
		DesiredFacingDir = StartingSyncState->GetOrientation_WorldSpace().Quaternion().GetForwardVector();
	}
	float MaxMoveSpeed = MaxSpeedOverride >= 0.0f ? MaxSpeedOverride : CommonLegacySettings->MaxSpeed;

	switch (MoveInputType)
	{
	case EMoveInputType::DirectionalIntent:
		{
			OutProposedMove.DirectionIntent = DesiredVelocity;	// here, DesiredVelocity is already in "intent space" (unit length for "max intent") so we can use it directly
			DesiredVelocity *= MaxMoveSpeed;
		}
		break;
	case EMoveInputType::Velocity:
		{
			// Clamp to max move speed
			DesiredVelocity = DesiredVelocity.GetClampedToMaxSize(MaxMoveSpeed);
			OutProposedMove.DirectionIntent = MaxMoveSpeed > UE_KINDA_SMALL_NUMBER ? DesiredVelocity / MaxMoveSpeed : FVector::ZeroVector; // here, DesiredVelocity is converted to "intent space"
		}
		break;
	case EMoveInputType::None:
	case EMoveInputType::Invalid:
	default:
		{
			UE_LOGF(LogMover, Warning, "Unhandled MoveInputType %i in USimpleFlyingMode", EnumToUnderlyingType(MoveInputType));
			DesiredVelocity = FVector::ZeroVector;
			OutProposedMove.DirectionIntent = FVector::ZeroVector;
		}
	break;
	}

	OutProposedMove.bHasDirIntent = !OutProposedMove.DirectionIntent.IsNearlyZero();
	DesiredFacingDir -= DesiredFacingDir.ProjectOnTo(GetMoverComponent()->GetUpDirection());
	FQuat CurrentFacing = StartingSyncState->GetOrientation_WorldSpace().Quaternion();
	FQuat DesiredFacing = CurrentFacing;
	
	if (DesiredFacingDir.Normalize())
	{
		DesiredFacing = FQuat::FindBetween(FVector::ForwardVector, DesiredFacingDir);
	}

	OutProposedMove.LinearVelocity = StartingSyncState->GetVelocity_WorldSpace();
	FVector AngularVelocityDegrees = StartingSyncState->GetAngularVelocityDegrees_WorldSpace();
	
	// Hack const_cast stuff
	// Why is this needed?
	// Because some modes want to mutate their data inside the generate move
	USimpleFlyingMode* MutableSimpleMode  = const_cast<USimpleFlyingMode*>(this);
	MutableSimpleMode->GenerateFlyingMove(const_cast<FMoverTickStartData&>(StartState), DeltaSeconds, DesiredVelocity, DesiredFacing, CurrentFacing, AngularVelocityDegrees, OutProposedMove.LinearVelocity);

	// Calc angular velocity from final facing
	OutProposedMove.AngularVelocityDegrees = AngularVelocityDegrees;
}

void USimpleFlyingMode::GenerateFlyingMove_Implementation(FMoverTickStartData& StartState, float DeltaSeconds, const FVector& DesiredVelocity,
	const FQuat& DesiredFacing, const FQuat& CurrentFacing, FVector& InOutAngularVelocityDegrees, FVector& InOutVelocity)
{
	InOutVelocity = DesiredVelocity;
	FQuat ToFacing = CurrentFacing.Inverse() * DesiredFacing;
	InOutAngularVelocityDegrees = DeltaSeconds > 0.0f ? FMath::RadiansToDegrees(ToFacing.ToRotationVector() / DeltaSeconds) : FVector::ZeroVector;
}

