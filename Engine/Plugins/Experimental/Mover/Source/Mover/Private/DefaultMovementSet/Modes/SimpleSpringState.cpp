// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimpleSpringState.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimpleSpringState)

UScriptStruct* FSimpleSpringState::GetScriptStruct() const
{ 
	return StaticStruct(); 
}

FMoverDataStructBase* FSimpleSpringState::Clone() const
{ 
	return new FSimpleSpringState(*this); 
}

bool FSimpleSpringState::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	bool bSuccess = Super::NetSerialize(Ar, Map, bOutSuccess);

	// Could be quantized to save bandwidth
	Ar << CurrentAccel;

	return bSuccess;
}

void FSimpleSpringState::ToString(FAnsiStringBuilderBase& Out) const
{
	Super::ToString(Out);

	Out.Appendf("Accel=%s\n", *CurrentAccel.ToCompactString());
}

bool FSimpleSpringState::ShouldReconcile(const FMoverDataStructBase& AuthorityState) const
{
	const FSimpleSpringState* AuthoritySpringState = static_cast<const FSimpleSpringState*>(&AuthorityState);

	return !(CurrentAccel - AuthoritySpringState->CurrentAccel).IsNearlyZero();
		
}


void FSimpleSpringState::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct)
{
	const FSimpleSpringState* FromState = static_cast<const FSimpleSpringState*>(&From);
	const FSimpleSpringState* ToState   = static_cast<const FSimpleSpringState*>(&To);

	CurrentAccel           = FMath::Lerp(FromState->CurrentAccel, ToState->CurrentAccel, Pct);
}
