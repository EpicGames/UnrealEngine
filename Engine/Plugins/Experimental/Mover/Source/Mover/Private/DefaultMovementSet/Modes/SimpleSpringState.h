// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovementMode.h"
#include "MoverTypes.h"
#include "SimpleSpringState.generated.h"

/**
* Internal state data for the SimpleSpringWalkingMode
*/
USTRUCT()
struct FSimpleSpringState : public FMoverDataStructBase
{
	GENERATED_BODY()
	
	virtual UScriptStruct* GetScriptStruct() const override;
	virtual FMoverDataStructBase* Clone() const override;
	virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;
	virtual void ToString(FAnsiStringBuilderBase& Out) const override;
	virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override;
	virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override;

	// Acceleration of internal spring model
	UPROPERTY(BlueprintReadOnly, Category = "Mover|Experimental")
	FVector CurrentAccel = FVector::ZeroVector;
};
