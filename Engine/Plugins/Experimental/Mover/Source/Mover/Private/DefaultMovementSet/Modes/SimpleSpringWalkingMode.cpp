// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultMovementSet/Modes/SimpleSpringWalkingMode.h"
#include "DefaultMovementSet/Modes/SimpleSpringState.h"
#include "MoverComponent.h"
#include "DefaultMovementSet/Settings/CommonLegacyMovementSettings.h"
#include "Math/SpringMath.h"
#include "MoveLibrary/RollbackBlackboard.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(SimpleSpringWalkingMode)



void USimpleSpringWalkingMode::SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState)
{
	Super::SimulationTick_Implementation(Params, OutputState);

	// If we've created or updated the spring state during GenerateMove, we need to copy it into the output simulation state.
	if (const FSimpleSpringState* InSpringState = Params.StartState.SyncState.SyncStateCollection.FindDataByType<FSimpleSpringState>())
	{
		FSimpleSpringState& OutputSpringState = OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FSimpleSpringState>();
		OutputSpringState = *InSpringState;
	}
}

void USimpleSpringWalkingMode::GenerateWalkMove_Implementation(FMoverTickStartData& StartState, float DeltaSeconds, const FMoverSimContext& SimContext, const FVector& DesiredVelocity,
	const FQuat& DesiredFacing, const FQuat& CurrentFacing, FVector& InOutAngularVelocityDegrees, FVector& InOutVelocity)
{
	bool bIsSpringStateNew = false;
	FSimpleSpringState& SpringState = StartState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FSimpleSpringState>(bIsSpringStateNew);


	// If the spring state is stale, we need to reset its internal state
	if (!bIsSpringStateNew)
	{
		bool bDidGenerateMoveSinceLastFrame = false;
		SimContext.Blackboard.TryGet(DidGenerateMoveEntry, bDidGenerateMoveSinceLastFrame);
		if (!bDidGenerateMoveSinceLastFrame)
		{
			SpringState = FSimpleSpringState();
		}
	}

	// Linear //
	
	SpringMath::CriticalSpringDamper(InOutVelocity, SpringState.CurrentAccel, DesiredVelocity, VelocitySmoothingTime, DeltaSeconds);
	
	// Angular //
	
	FVector CurrentAngularVelocityRad = FMath::DegreesToRadians(InOutAngularVelocityDegrees);
	FQuat UpdatedFacing = CurrentFacing;
	SpringMath::CriticalSpringDamperQuat(UpdatedFacing, CurrentAngularVelocityRad, DesiredFacing, FacingSmoothingTime, DeltaSeconds);
	InOutAngularVelocityDegrees = FMath::RadiansToDegrees(CurrentAngularVelocityRad);
}

