// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultMovementSet/Modes/SimpleWalkingMode.h"

#include "MoverComponent.h"
#include "DefaultMovementSet/Settings/CommonLegacyMovementSettings.h"
#include "MoveLibrary/RollbackBlackboardLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimpleWalkingMode)

const FName USimpleWalkingMode::DidGenerateMoveEntry = TEXT("DidGenerateMove");


void USimpleWalkingMode::OnRegistered(const FName ModeName, const FMoverSimContext& SimContext)
{
	Super::OnRegistered(ModeName, SimContext);

	URollbackBlackboard::EntrySettings DidGenerateMoveEntrySettings = URollbackBlackboardLibrary::MakeSingleFrameEntrySettings();
	DidGenerateMoveEntrySettings.PersistencePolicy = EBlackboardPersistencePolicy::ThroughNextFrame;

	SimContext.Blackboard.CreateEntry<bool>(DidGenerateMoveEntry, DidGenerateMoveEntrySettings);
}


void USimpleWalkingMode::GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep,
                                                   FProposedMove& OutProposedMove) const
{
	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();

	if (!StartingSyncState)
	{
		return;
	}

	const float DeltaSeconds = TimeStep.StepMs * 0.001f;
	if (DeltaSeconds <= FLT_EPSILON)
	{
		return;
	}

	// Get input
	FVector DesiredVelocity;
	EMoveInputType MoveInputType;
	FVector DesiredFacingDir;
	 
	if (const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>())
	{
		DesiredVelocity = CharacterInputs->GetMoveInput_WorldSpace();
		MoveInputType = CharacterInputs->GetMoveInputType();
		DesiredFacingDir = CharacterInputs->GetOrientationIntentDir_WorldSpace();
	}
	else
	{
		// If no input found, most likely a networked sim proxy
		// Try to deduce an input from the sync state
		DesiredVelocity = StartingSyncState->GetIntent_WorldSpace();
		MoveInputType = EMoveInputType::DirectionalIntent;
		DesiredFacingDir = StartingSyncState->GetOrientation_WorldSpace().Quaternion().GetForwardVector();
	}

	const bool bHasMaxMoveSpeed = MaxSpeedOverride >= 0.0f || CommonLegacySettings;
	float MaxMoveSpeed = MaxSpeedOverride >= 0.0f ? MaxSpeedOverride : (CommonLegacySettings ? CommonLegacySettings->MaxSpeed : 0.0f);

	const UMoverComponent* MoverComponent = GetMoverComponent();
	const FVector UpVector = MoverComponent ? MoverComponent->GetUpDirection() : FVector::UpVector;

	// Subtract vertical component but keep same magnitude
	float DesiredVelMag = DesiredVelocity.Length();
	DesiredVelocity -= DesiredVelocity.ProjectOnTo(UpVector);
	float DesiredVel2DSquaredLength = DesiredVelocity.SquaredLength(); 
	if (DesiredVel2DSquaredLength > 0.0f)
	{
		DesiredVelocity *= DesiredVelMag / FMath::Sqrt(DesiredVel2DSquaredLength); 
	}

	const float DefaultDirectionalIntentSpeed = 100.0f;

	switch (MoveInputType)
	{
	case EMoveInputType::DirectionalIntent:
		{
			OutProposedMove.DirectionIntent = DesiredVelocity;	// here, DesiredVelocity is already in "intent space" (unit length for "max intent") so we can use it directly
			DesiredVelocity = bHasMaxMoveSpeed ? DesiredVelocity * MaxMoveSpeed : DesiredVelocity * DefaultDirectionalIntentSpeed;
		}
		break;
	case EMoveInputType::Velocity:
		{
			// Clamp to max move speed
			DesiredVelocity = bHasMaxMoveSpeed ? DesiredVelocity.GetClampedToMaxSize(MaxMoveSpeed) : DesiredVelocity;
			OutProposedMove.DirectionIntent = MaxMoveSpeed > UE_KINDA_SMALL_NUMBER ? DesiredVelocity / MaxMoveSpeed : FVector::ZeroVector; // here, DesiredVelocity is converted to "intent space"
		}
		break;
	case EMoveInputType::None:
	case EMoveInputType::Invalid:
	default:
		{
			UE_LOGF(LogMover, Warning, "Unhandled MoveInputType %i in USimpleWalkingMode", EnumToUnderlyingType(MoveInputType));
			DesiredVelocity = FVector::ZeroVector;
			OutProposedMove.DirectionIntent = FVector::ZeroVector;
		}
	break;
	}

	OutProposedMove.bHasDirIntent = !OutProposedMove.DirectionIntent.IsNearlyZero();
	DesiredFacingDir -= DesiredFacingDir.ProjectOnTo(UpVector);
	FQuat CurrentFacing = StartingSyncState->GetOrientation_WorldSpace().Quaternion();
	FQuat DesiredFacing = CurrentFacing;
	
	if (DesiredFacingDir.Normalize())
	{
		DesiredFacing = FQuat::FindBetween(FVector::ForwardVector, DesiredFacingDir);
	}

	OutProposedMove.LinearVelocity = StartingSyncState->GetVelocity_WorldSpace();
	OutProposedMove.AngularVelocityDegrees = StartingSyncState->GetAngularVelocityDegrees_WorldSpace();
	
	// Hack const_cast stuff
	// Why is this needed?
	// Because some modes want to mutate their data inside the generate walk move
	USimpleWalkingMode* MutableSimpleWalkMode  = const_cast<USimpleWalkingMode*>(this);
	MutableSimpleWalkMode->GenerateWalkMove(const_cast<FMoverTickStartData&>(StartState), DeltaSeconds, SimContext, DesiredVelocity, DesiredFacing, CurrentFacing, OutProposedMove.AngularVelocityDegrees, OutProposedMove.LinearVelocity);

	SimContext.Blackboard.TrySet(DidGenerateMoveEntry, true);
}

void USimpleWalkingMode::GenerateWalkMove_Implementation(FMoverTickStartData& StartState, float DeltaSeconds, const FMoverSimContext& SimContext, const FVector& DesiredVelocity,
	const FQuat& DesiredFacing, const FQuat& CurrentFacing, FVector& InOutAngularVelocityDegrees, FVector& InOutVelocity)
{
	InOutVelocity = DesiredVelocity;

	FQuat ToFacing = CurrentFacing.Inverse() * DesiredFacing;
	InOutAngularVelocityDegrees = DeltaSeconds > 0.0f ? FMath::RadiansToDegrees(ToFacing.ToRotationVector() / DeltaSeconds) : FVector::ZeroVector;
}

