// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultMovementSet/Modes/SmoothWalkingMode.h"
#include "DefaultMovementSet/Modes/SmoothWalkingState.h"

#include "MoverComponent.h"
#include "DefaultMovementSet/Settings/CommonLegacyMovementSettings.h"
#include "Math/SpringMath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SmoothWalkingMode)

void USmoothWalkingMode::SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState)
{
	Super::SimulationTick_Implementation(Params, OutputState);

	// If we've created or updated the spring state during GenerateWalkMove, we need to copy it into the output simulation state. 
	if (const FSmoothWalkingState* InSpringState = Params.StartState.SyncState.SyncStateCollection.FindDataByType<FSmoothWalkingState>())
	{
		FSmoothWalkingState& OutputSpringState = OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FSmoothWalkingState>();
		OutputSpringState = *InSpringState;
	}
}

void USmoothWalkingMode::GenerateWalkMove_Implementation(FMoverTickStartData& StartState, float DeltaSeconds, const FMoverSimContext& SimContext, const FVector& DesiredVelocity,
	const FQuat& DesiredFacing, const FQuat& CurrentFacing, FVector& InOutAngularVelocityDegrees, FVector& InOutVelocity) 
{
	if (DeltaSeconds <= FLT_EPSILON)
	{
		return;
	}
	
	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	if (!ensure(StartingSyncState))
	{
		return;
	}

	// Find or add a FSmoothWalkingState to the SyncState
	bool bIsSmoothWalkingStateNew = false;
	FSmoothWalkingState& SpringState = StartState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FSmoothWalkingState>(bIsSmoothWalkingStateNew);

	bool bDidGenerateMoveSinceLastFrame = false;
	SimContext.Blackboard.TryGet(DidGenerateMoveEntry, bDidGenerateMoveSinceLastFrame);

	// If the state was not there already or is stale, we need to initialize some of the intermediate state to whatever we have as the current state to avoid 
	// a discontinuity. Unfortunately there is no way currently to initialize the angular velocities or accelerations right now as these are not 
	// carried between movement modes in FMoverDefaultSyncState.
	if (bIsSmoothWalkingStateNew || !bDidGenerateMoveSinceLastFrame)
	{
		SpringState.SpringVelocity = InOutVelocity;
		SpringState.SpringAcceleration = FVector::ZeroVector;
		SpringState.IntermediateVelocity = InOutVelocity;
		SpringState.IntermediateFacing = CurrentFacing;
		SpringState.IntermediateAngularVelocity = FVector::ZeroVector;
	}

	// We can compute the degree to which the internal velocity matches the actual velocity by projecting the spring velocity onto the actual velocity 
	// which we moved last frame. This gives a number between 0 and 1 which says how different our velocity was from what we expected.
	const float VelocityMatch = FMath::Clamp(SpringState.SpringVelocity.Dot(InOutVelocity) / 
		FMath::Max(InOutVelocity.Length() * SpringState.SpringVelocity.Length(), UE_SMALL_NUMBER), 0.0f, 1.0f);

	// If our velocity was very different from what we expected then we can effectively "reset" the intermediate velocity in a smooth way towards it. 
	// This removes any velocity we may have built-up in the intermediate spring that is different from our current velocity.
	FMath::ExponentialSmoothingApprox(SpringState.IntermediateVelocity, InOutVelocity, DeltaSeconds, 
		(OutsideInfluenceSmoothingTime + UE_KINDA_SMALL_NUMBER) / (1.0f - VelocityMatch));

	// Update spring velocity based on real velocity
	SpringState.SpringVelocity = InOutVelocity;

	// Rotate the intermediate velocity towards the target direction using the TurningStrength
	if (TurningStrength > 0.0f)
	{
		if (!DesiredVelocity.IsNearlyZero())
		{
			FMath::ExponentialSmoothingApprox(
				SpringState.IntermediateVelocity,
				DesiredVelocity.GetSafeNormal() * SpringState.IntermediateVelocity.Length(),
				DeltaSeconds,
				SpringMath::StrengthToSmoothingTime(TurningStrength));
		}
	}

	// Check if we are acceleration or decelerating and work out how much lateral vs directional acceleration to apply. Note that even when the
	// DirectionalAccelerationFactor is high Deceleration is always applied laterally. This is similar to how the default walking mode behaves.
	const bool bIsAccelerating = (1.01f * DesiredVelocity.SquaredLength()) > SpringState.SpringVelocity.SquaredLength();
	const float LateralAccelerationMagnitude =  bIsAccelerating ? (1.0f - DirectionalAccelerationFactor) * Acceleration : Deceleration;
	const float DirectionalAccelerationMagnitude = bIsAccelerating ? DirectionalAccelerationFactor * Acceleration : 0.0f;

	// Record the previous velocity length
	const float PreviousVelocityLength = SpringState.IntermediateVelocity.Length();
	
	// Compute the desired difference in velocity
	const FVector VelocityDifference = DesiredVelocity - SpringState.IntermediateVelocity;

	// Compute the lateral acceleration moving directly toward the desired velocity
	const FVector LateralAccelerationVector = VelocityDifference.GetSafeNormal() * FMath::Min(LateralAccelerationMagnitude, VelocityDifference.Length() / FMath::Max(DeltaSeconds, UE_SMALL_NUMBER));

	// Compute the directional acceleration moving in the desired direction. This emulates how acceleration is applied in the default movement mode.
	const FVector DirectionalAccelerationVector = DesiredVelocity.GetSafeNormal() * DirectionalAccelerationMagnitude;

	// Find the combined desired acceleration as the addition of the two previous accelerations
	const FVector DesiredAcceleration = LateralAccelerationVector + DirectionalAccelerationVector;

	// Integrate the desired acceleration to estimate the next velocity, and future velocity to track. Here we don't want to over-shoot the desired
	// velocity target so we just snap to that if we are within enough distance.
	// 
	// We then clamp this velocity to make it no larger than the previous intermediate velocity or the total desired velocity this stops the 
	// DirectionalAcceleration adding velocity to the system and infinitely speeding up the character
	FVector NextVelocity = VelocityDifference.Dot(DesiredAcceleration * DeltaSeconds) < VelocityDifference.SquaredLength() ?
		SpringState.IntermediateVelocity + DesiredAcceleration * DeltaSeconds : DesiredVelocity;

	NextVelocity = NextVelocity.GetClampedToMaxSize(FMath::Max(PreviousVelocityLength, DesiredVelocity.Length()));

	// Compute the smoothing time based on acceleration or deceleration
	const float VelocitySmoothingTime = bIsAccelerating ? AccelerationSmoothingTime : DecelerationSmoothingTime;

	// Compute the smoothing compensation based on acceleration or deceleration
	const float VelocitySmoothingCompensation = bIsAccelerating ? AccelerationSmoothingCompensation : DecelerationSmoothingCompensation;

	// Get the lag associated with the current velocity smoothing time and perform the same process to estimate the velocity we need to track to
	// avoid lagging behind
	const float LagSeconds = DeltaSeconds + (VelocitySmoothingCompensation * VelocitySmoothingTime);

	FVector TrackVelocity = VelocityDifference.Dot(DesiredAcceleration * LagSeconds) < VelocityDifference.SquaredLength() ?
		SpringState.IntermediateVelocity + DesiredAcceleration * LagSeconds : DesiredVelocity;

	TrackVelocity = TrackVelocity.GetClampedToMaxSize(FMath::Max(PreviousVelocityLength, DesiredVelocity.Length()));

	// Apply the smoothing to the track velocity - effectively tracking the intermediate velocity at the appropriate time in the future
	SpringMath::CriticalSpringDamper(SpringState.SpringVelocity, SpringState.SpringAcceleration, TrackVelocity, VelocitySmoothingTime, DeltaSeconds);

	// Snap the velocity to the desired velocity based on the dead-zone
	if ((DesiredVelocity - SpringState.SpringVelocity).SquaredLength() < FMath::Square(VelocityDeadzoneThreshold))
	{
		// We reached our target
		SpringState.SpringVelocity = DesiredVelocity;
	
		// If we've reached our target then also snap the acceleration to zero once it is close enough
		if (SpringState.SpringAcceleration.SquaredLength() < FMath::Square(AccelerationDeadzoneThreshold))
		{
			SpringState.SpringAcceleration = FVector::ZeroVector;
		}
	}

	// Update the target velocity
	InOutVelocity = SpringState.SpringVelocity;

	// Update the intermediate velocity
	SpringState.IntermediateVelocity = NextVelocity;

	FVector CurrentAngularVelocityRadians = FMath::DegreesToRadians(InOutAngularVelocityDegrees);
	FQuat UpdatedFacing = CurrentFacing;

	// For controlling the facing direction we use either a spring or a double spring
	if (bSmoothFacingWithDoubleSpring)
	{
		SpringMath::CriticalSpringDamperQuat(SpringState.IntermediateFacing, SpringState.IntermediateAngularVelocity, DesiredFacing, FacingSmoothingTime / 2.0f, DeltaSeconds);
		SpringMath::CriticalSpringDamperQuat(UpdatedFacing, CurrentAngularVelocityRadians, SpringState.IntermediateFacing, FacingSmoothingTime / 2.0f, DeltaSeconds);
	}
	else
	{
		SpringState.IntermediateFacing = DesiredFacing;
		SpringState.IntermediateAngularVelocity = CurrentAngularVelocityRadians;
		SpringMath::CriticalSpringDamperQuat(UpdatedFacing, CurrentAngularVelocityRadians, DesiredFacing, FacingSmoothingTime, DeltaSeconds);
	}

	// Snap the facing to the desired facing based on the dead-zone
	if (DesiredFacing.AngularDistance(UpdatedFacing) < FMath::DegreesToRadians(FacingDeadzoneThreshold))
	{
		// We reached our target
		// Ensure the output angular velocity will snap perfectly to the target
		// Note we don't do this normally because its better to have a consistent angular velocity
		// If we output the angular velocity based on the updated facing every frame it can cause errors at low dt due to inaccuracy in the inverse
		// exponential approximation inside the spring damper
		CurrentAngularVelocityRadians = DeltaSeconds > 0.0f ? ((CurrentFacing.Inverse() * UpdatedFacing).GetShortestArcWith(FQuat::Identity)).ToRotationVector() / DeltaSeconds : FVector::ZeroVector;
		SpringState.IntermediateFacing = DesiredFacing;

		// If we've reached our target then also snap the angular velocity to zero once it is close enough
		if (CurrentAngularVelocityRadians.SquaredLength() < FMath::Square(FMath::DegreesToRadians(AngularVelocityDeadzoneThreshold)))
		{
			SpringState.IntermediateAngularVelocity = FVector::ZeroVector;
		}
	}

	InOutAngularVelocityDegrees = FMath::RadiansToDegrees(CurrentAngularVelocityRadians);
}


