// Copyright Epic Games, Inc. All Rights Reserved.

#include "SmoothWalkingState.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SmoothWalkingState)

namespace SmoothWalkingStateErrorTolerance
{
	constexpr float VelocityErrorTolerance = 10.f;
	constexpr float AngularVelocityErrorTolerance = 10.f;
	constexpr float AccelerationErrorTolerance = 50.f;
	constexpr float FacingDegreeErrorTolerance = 10.0f;
}

UScriptStruct* FSmoothWalkingState::GetScriptStruct() const
{ 
	return StaticStruct(); 
}

FMoverDataStructBase* FSmoothWalkingState::Clone() const
{ 
	return new FSmoothWalkingState(*this); 
}

bool FSmoothWalkingState::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	bool bSuccess = Super::NetSerialize(Ar, Map, bOutSuccess);

	// Could be quantized to save bandwidth
	Ar << SpringVelocity;
	Ar << SpringAcceleration;
	Ar << IntermediateVelocity;
	Ar << IntermediateFacing;
	Ar << IntermediateAngularVelocity;

	return bSuccess;
}

void FSmoothWalkingState::ToString(FAnsiStringBuilderBase& Out) const
{
	Super::ToString(Out);

	Out.Appendf("SpringVelocity=%s SpringAcceleration=%s IntVel=%s IntFac=%s IntAng=%s\n",
		*SpringVelocity.ToCompactString(),
		*SpringAcceleration.ToCompactString(),
		*IntermediateVelocity.ToCompactString(),
		*IntermediateFacing.ToString(), 
		*IntermediateAngularVelocity.ToString());
}

bool FSmoothWalkingState::ShouldReconcile(const FMoverDataStructBase& AuthorityState) const
{
	const FSmoothWalkingState* AuthoritySpringState = static_cast<const FSmoothWalkingState*>(&AuthorityState);

	return (!(SpringVelocity - AuthoritySpringState->SpringVelocity).IsNearlyZero(SmoothWalkingStateErrorTolerance::VelocityErrorTolerance) ||
			!(SpringAcceleration - AuthoritySpringState->SpringAcceleration).IsNearlyZero(SmoothWalkingStateErrorTolerance::AccelerationErrorTolerance) ||
			!(IntermediateVelocity - AuthoritySpringState->IntermediateVelocity).IsNearlyZero(SmoothWalkingStateErrorTolerance::VelocityErrorTolerance) ||
			 (FMath::RadiansToDegrees(IntermediateFacing.AngularDistance(AuthoritySpringState->IntermediateFacing)) > SmoothWalkingStateErrorTolerance::FacingDegreeErrorTolerance ||
			!(IntermediateAngularVelocity - AuthoritySpringState->IntermediateAngularVelocity).IsNearlyZero(SmoothWalkingStateErrorTolerance::AngularVelocityErrorTolerance)));
}


void FSmoothWalkingState::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct)
{
	const FSmoothWalkingState* FromState = static_cast<const FSmoothWalkingState*>(&From);
	const FSmoothWalkingState* ToState   = static_cast<const FSmoothWalkingState*>(&To);

	SpringVelocity = FMath::Lerp(FromState->SpringVelocity, ToState->SpringVelocity, Pct);
	SpringAcceleration = FMath::Lerp(FromState->SpringAcceleration, ToState->SpringAcceleration, Pct);
	IntermediateVelocity = FMath::Lerp(FromState->IntermediateVelocity, ToState->IntermediateVelocity, Pct);
	IntermediateFacing = FQuat::Slerp(FromState->IntermediateFacing, ToState->IntermediateFacing, Pct);
	IntermediateAngularVelocity = FMath::Lerp(FromState->IntermediateAngularVelocity, ToState->IntermediateAngularVelocity, Pct);
}
