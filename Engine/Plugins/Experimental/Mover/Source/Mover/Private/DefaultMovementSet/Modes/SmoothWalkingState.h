// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovementMode.h"
#include "MoverTypes.h"
#include "SmoothWalkingState.generated.h"

/**
* Internal state data for the SmoothWalkingMode
*/
USTRUCT()
struct FSmoothWalkingState : public FMoverDataStructBase
{
	GENERATED_BODY()
	
	virtual UScriptStruct* GetScriptStruct() const override;
	virtual FMoverDataStructBase* Clone() const override;
	virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;
	virtual void ToString(FAnsiStringBuilderBase& Out) const override;
	virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override;
	virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override;

	// Velocity of internal velocity spring
	UPROPERTY(BlueprintReadOnly, Category = "Mover|Experimental")
	FVector SpringVelocity = FVector::ZeroVector;

	// Acceleration of internal velocity spring
	UPROPERTY(BlueprintReadOnly, Category = "Mover|Experimental")
	FVector SpringAcceleration = FVector::ZeroVector;

	// Intermediate velocity which the velocity spring tracks as a target
	UPROPERTY(BlueprintReadOnly, Category = "Mover|Experimental")
	FVector IntermediateVelocity = FVector::ZeroVector;

	// Intermediate facing direction when using a double spring
	UPROPERTY(BlueprintReadOnly, Category = "Mover|Experimental")
	FQuat IntermediateFacing = FQuat::Identity;

	// Angular velocity of the intermediate spring when using a double spring
	UPROPERTY(BlueprintReadOnly, Category = "Mover|Experimental")
	FVector IntermediateAngularVelocity = FVector::ZeroVector;
};
