// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultMovementSet/Modes/SwimmingMode.h"
#include "Components/SkeletalMeshComponent.h"
#include "GameFramework/Pawn.h"
#include "MoveLibrary/WaterMovementUtils.h"
#include "DefaultMovementSet/InstantMovementEffects/BasicInstantMovementEffects.h"
#include "MoverComponent.h"
#include "DefaultMovementSet/Settings/CommonLegacyMovementSettings.h"
#include "MoverLog.h"
#include "MoveLibrary/MovementUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SwimmingMode)


USwimmingMode::USwimmingMode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SharedSettingsClasses.Add(UCommonLegacyMovementSettings::StaticClass());

	GameplayTags.AddTag(Mover_IsSwimming);
}

void USwimmingMode::GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const
{
	const UMoverComponent* MoverComp = GetMoverComponent();
	const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(StartingSyncState);
	
	const float DeltaSeconds = TimeStep.StepMs * 0.001f;
	
	UMoverBlackboard* SimBlackboard = MoverComp->GetSimBlackboard_Mutable();
	FWaterCheckResult LastWaterResult;
	
	const float CapsuleHalfHeight = MoverComp->GetOwner()->GetSimpleCollisionHalfHeight();

	if (SimBlackboard)
	{
		SimBlackboard->TryGet(CommonBlackboard::LastWaterResult, LastWaterResult);
	}
	
	FVector Velocity = StartingSyncState->GetVelocity_WorldSpace();
	
	// Buoyancy Bobbing
	{
		const FWaterFlowSplineData& WaterData = LastWaterResult.WaterSplineData;
		
		const float ReciprocalCapsuleHeight = 0.5f / CapsuleHalfHeight;
		const float ReciprocalOriginalCapsuleHeight = 0.5f / OriginalCapsuleHalfHeight;
		const FVector CapsuleLocation = StartingSyncState->GetLocation_WorldSpace();

		const float ImmersionDepth = WaterData.ImmersionDepth + CapsuleHalfHeight;
		const float ImmersionPercent = FMath::Clamp(ImmersionDepth * ReciprocalCapsuleHeight, UE_KINDA_SMALL_NUMBER, 1.f);
		const float IdealDepth = CommonLegacySettings->SwimmingIdealImmersionDepth + CapsuleHalfHeight;
		const float IdealImmersionPercent = FMath::Clamp(IdealDepth * ReciprocalOriginalCapsuleHeight, UE_KINDA_SMALL_NUMBER, 1.f);

		const float GravityForce = GetMoverComponent()->GetGravityAcceleration().Z;

		
		// Using IdealDepth, we compute the Buoyancy such that (Buoyancy * IdealImmersionPercent) = -G force
		const float BuoyancyForce = -GravityForce / IdealImmersionPercent;
	
		// Now calculate the actual force in the correct ratio
		float BobbingForce = (BuoyancyForce * ImmersionPercent) + GravityForce;
		const float MaxForce = SurfaceSwimmingWaterControlSettings.BobbingMaxForce;
		BobbingForce = FMath::Clamp(BobbingForce, -MaxForce, MaxForce);

		Velocity.Z += BobbingForce * DeltaSeconds;

		// Vertical fluid friction for bobbing
		if (!FMath::IsNearlyZero(Velocity.Z, (FVector::FReal)0.1f))
		{
			const float IdealDepthTolerance = SurfaceSwimmingWaterControlSettings.BobbingIdealDepthTolerance;
			if (FMath::Sign(Velocity.Z) != FMath::Sign(BobbingForce) || FMath::IsNearlyEqual(ImmersionDepth, IdealDepth, IdealDepthTolerance))
			{
				float FluidFriction = 0.f;
				float ExpDrag = 0.f;
				const bool bFullySubmerged = ImmersionPercent > 1.f;
				if (Velocity.Z > 0)
				{
					FluidFriction = SurfaceSwimmingWaterControlSettings.BobbingFrictionUp;
					ExpDrag = SurfaceSwimmingWaterControlSettings.BobbingExpDragUp;
				}
				else
				{
					// Different drag when fully immersed and moving down (mainly controls how far you go when falling in fast)
					FluidFriction = bFullySubmerged ? SurfaceSwimmingWaterControlSettings.BobbingFrictionDownSubmerged : SurfaceSwimmingWaterControlSettings.BobbingFrictionDown;
					ExpDrag = bFullySubmerged ? SurfaceSwimmingWaterControlSettings.BobbingExpDragDownSubmerged : SurfaceSwimmingWaterControlSettings.BobbingExpDragDown;
				}
	
				FluidFriction *= SurfaceSwimmingWaterControlSettings.BobbingFrictionMultiplier;
				ExpDrag *= SurfaceSwimmingWaterControlSettings.BobbingExpDragMultiplier;
	
				Velocity.Z = Velocity.Z * (1.f - FMath::Min(FluidFriction * DeltaSeconds, 1.f));
				Velocity.Z = Velocity.Z * (1.f - FMath::Min(FMath::Abs(Velocity.Z) * FMath::Square(ExpDrag) * DeltaSeconds, 1.f));
			}
		}
	}

	// Vertical speed limit in Water
	{
		const float MaxVerticalWaterSpeedUp = SurfaceSwimmingWaterControlSettings.MaxSpeedUp;
		const float MaxVerticalWaterSpeedDown = SurfaceSwimmingWaterControlSettings.MaxSpeedDown;
	
		Velocity.Z = FMath::Clamp(Velocity.Z, -FMath::Abs(MaxVerticalWaterSpeedDown), FMath::Abs(MaxVerticalWaterSpeedUp));
	}

	// Calculate and apply the requested move here
	{
		// Force from Water flow's WaterVelocity 
		const float MaxWaterForce = SurfaceSwimmingWaterControlSettings.MaxWaterForce;
		const float WaterForceMultiplier = SurfaceSwimmingWaterControlSettings.WaterForceMultiplier * SurfaceSwimmingWaterControlSettings.WaterForceSecondMultiplier;
		const FVector WaterVelocity = LastWaterResult.WaterSplineData.WaterVelocity;
		const FVector WaterAcceleration = (WaterVelocity * WaterForceMultiplier).GetClampedToMaxSize(MaxWaterForce);
		const float WaterSpeed = WaterVelocity.Size();
	
	    // Consider player input
		FRotator IntendedOrientation_WorldSpace;
		if (!CharacterInputs || CharacterInputs->OrientationIntent.IsNearlyZero())
		{
			IntendedOrientation_WorldSpace = StartingSyncState->GetOrientation_WorldSpace();
		}
		else
		{
			IntendedOrientation_WorldSpace = CharacterInputs->GetOrientationIntentDir_WorldSpace().ToOrientationRotator();
		}

		IntendedOrientation_WorldSpace = UMovementUtils::ApplyGravityToOrientationIntent(IntendedOrientation_WorldSpace, MoverComp->GetWorldToGravityTransform(), CommonLegacySettings->bShouldRemainVertical);
		
		FWaterMoveParams Params;
		if (CharacterInputs)
		{
			Params.MoveInputType = CharacterInputs->GetMoveInputType();

			const bool bMaintainInputMagnitude = true;
			Params.MoveInput = UPlanarConstraintUtils::ConstrainDirectionToPlane(MoverComp->GetPlanarConstraint(), CharacterInputs->GetMoveInput_WorldSpace(), bMaintainInputMagnitude);
		}
		else
		{
			Params.MoveInputType = EMoveInputType::None;
			Params.MoveInput = FVector::ZeroVector;
		}
		Params.OrientationIntent = IntendedOrientation_WorldSpace;
		Params.PriorVelocity = StartingSyncState->GetVelocity_WorldSpace();
		Params.PriorOrientation = StartingSyncState->GetOrientation_WorldSpace();
		Params.TurningRate = CommonLegacySettings->TurningRate;
		Params.TurningBoost = CommonLegacySettings->TurningBoost;
		Params.MaxSpeed = CommonLegacySettings->MaxSpeed;
		Params.Acceleration = CommonLegacySettings->Acceleration;
		Params.Deceleration = CommonLegacySettings->Deceleration;
		Params.DeltaSeconds = DeltaSeconds;
		Params.MoveSpeed = WaterSpeed;
		Params.MoveAcceleration = WaterAcceleration;
		Params.WorldToGravityQuat = MoverComp->GetWorldToGravityTransform();

		// Calculate the move
		OutProposedMove = UWaterMovementUtils::ComputeControlledWaterMove(Params);
		
		// Use Z Velocity calculated earlier (Buoyancy, Friction and Terminal Velocity) for the move's Z component
		OutProposedMove.LinearVelocity.Z = Velocity.Z;
	}
}

void USwimmingMode::SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState)
{
}

void USwimmingMode::OnRegistered(const FName ModeName, const FMoverSimContext& SimContext)
{
	Super::OnRegistered(ModeName, SimContext);

	if (UMoverComponent* MoverComp = GetMoverComponent())
	{
		CommonLegacySettings = MoverComp->FindSharedSettings<UCommonLegacyMovementSettings>();
		OriginalCapsuleHalfHeight = MoverComp->GetOwner()->GetSimpleCollisionHalfHeight();
	}
	ensureMsgf(CommonLegacySettings, TEXT("Failed to find instance of CommonLegacyMovementSettings on %s. Movement may not function properly."), *GetPathNameSafe(this));
}

void USwimmingMode::OnUnregistered(const FMoverSimContext& SimContext)
{
	CommonLegacySettings = nullptr;

	Super::OnUnregistered(SimContext);
}

bool USwimmingMode::AttemptJump(const FSimulationTickParams& Params, float UpwardsSpeed, FMoverTickEndData& OutputState)
{
	// TODO: This should check if a jump is even allowed
	TSharedPtr<FJumpImpulseEffect> JumpMove = MakeShared<FJumpImpulseEffect>();
	JumpMove->UpwardsSpeed = UpwardsSpeed;

	// TODO: Use UMoverSimulation to queue instant movement effects when it is passed as part of FSimulationTickParams
	GetMoverComponent()->QueueInstantMovementEffect_Internal(Params.TimeStep, JumpMove);
	
	return true;
}
