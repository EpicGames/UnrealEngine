// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultMovementSet/Modes/WalkingMode.h"
#include "Components/SkeletalMeshComponent.h"
#include "GameFramework/Pawn.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoveLibrary/BasedMovementUtils.h"
#include "MoveLibrary/ModularMovement.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/GroundMovementUtils.h"
#include "MoverComponent.h"
#include "DefaultMovementSet/Settings/CommonLegacyMovementSettings.h"
#include "MoverLog.h"
#include "DefaultMovementSet/InstantMovementEffects/BasicInstantMovementEffects.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WalkingMode)


UWalkingMode::UWalkingMode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SharedSettingsClasses.Add(UCommonLegacyMovementSettings::StaticClass());

	GameplayTags.AddTag(Mover_IsOnGround);
}

void UWalkingMode::Activate(const FMoverEventContext& Context, FName PrevModeName, const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, FMoverSyncState* OutSyncState, FMoverAuxStateContext* OutAuxState)
{
	Super::Activate(Context, PrevModeName, SimContext, StartState, OutSyncState, OutAuxState);

	// Make sure we test the floor ourselves upon activation
	if (UMoverBlackboard* SimBlackboard = GetMoverComponent()->GetSimBlackboard_Mutable())
	{
		SimBlackboard->Invalidate(CommonBlackboard::LastFloorResult);
	}
}

void UWalkingMode::GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const
{
	const UMoverComponent* MoverComp = GetMoverComponent();
	const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(StartingSyncState);


	const float DeltaSeconds = TimeStep.StepMs * 0.001f;
	FFloorCheckResult LastFloorResult;
	FVector MovementNormal;

	UMoverBlackboard* SimBlackboard = MoverComp->GetSimBlackboard_Mutable();
	FVector UpDirection = MoverComp->GetUpDirection();

	// Try to use the floor as the basis for the intended move direction (i.e. try to walk along slopes, rather than into them)
	if (SimBlackboard && SimBlackboard->TryGet(CommonBlackboard::LastFloorResult, LastFloorResult) && LastFloorResult.IsWalkableFloor())
	{
		MovementNormal = LastFloorResult.HitResult.ImpactNormal;
	}
	else
	{
		MovementNormal = UpDirection;
	}

	FRotator IntendedOrientation_WorldSpace;
	// If there's no intent from input to change orientation, use the current orientation
	if (!CharacterInputs || CharacterInputs->OrientationIntent.IsNearlyZero())
	{
		IntendedOrientation_WorldSpace = StartingSyncState->GetOrientation_WorldSpace();
	}
	else
	{
		IntendedOrientation_WorldSpace = CharacterInputs->GetOrientationIntentDir_WorldSpace().ToOrientationRotator();
	}

	IntendedOrientation_WorldSpace = UMovementUtils::ApplyGravityToOrientationIntent(IntendedOrientation_WorldSpace, MoverComp->GetWorldToGravityTransform(), CommonLegacySettings->bShouldRemainVertical);
	
	FGroundMoveParams Params;

	if (CharacterInputs)
	{
		Params.MoveInputType = CharacterInputs->GetMoveInputType();

		const bool bMaintainInputMagnitude = true;
		Params.MoveInput = UPlanarConstraintUtils::ConstrainDirectionToPlane(MoverComp->GetPlanarConstraint(), CharacterInputs->GetMoveInput_WorldSpace(), bMaintainInputMagnitude);
	}
	else
	{
		Params.MoveInputType = EMoveInputType::None;
		Params.MoveInput = FVector::ZeroVector;
	}

	Params.OrientationIntent = IntendedOrientation_WorldSpace;
	Params.PriorVelocity = FVector::VectorPlaneProject(StartingSyncState->GetVelocity_WorldSpace(), MovementNormal);
	Params.PriorOrientation = StartingSyncState->GetOrientation_WorldSpace();
	Params.GroundNormal = MovementNormal;
	Params.TurningRate = CommonLegacySettings->TurningRate;
	Params.TurningBoost = CommonLegacySettings->TurningBoost;
	Params.MaxSpeed = CommonLegacySettings->MaxSpeed;
	Params.Acceleration = CommonLegacySettings->Acceleration;
	Params.Deceleration = CommonLegacySettings->Deceleration;
	Params.DeltaSeconds = DeltaSeconds;
	Params.WorldToGravityQuat = MoverComp->GetWorldToGravityTransform();
	Params.UpDirection = UpDirection;
	Params.bUseAccelerationForVelocityMove = CommonLegacySettings->bUseAccelerationForVelocityMove;
	
	if (Params.MoveInput.SizeSquared() > 0.f && !UMovementUtils::IsExceedingMaxSpeed(Params.PriorVelocity, CommonLegacySettings->MaxSpeed))
	{
		Params.Friction = CommonLegacySettings->GroundFriction;
	}
	else
	{
		Params.Friction = CommonLegacySettings->bUseSeparateBrakingFriction ? CommonLegacySettings->BrakingFriction : CommonLegacySettings->GroundFriction;
		Params.Friction *= CommonLegacySettings->BrakingFrictionFactor;
	}

	OutProposedMove = UGroundMovementUtils::ComputeControlledGroundMove(Params);

	if (TurnGenerator)
	{
		OutProposedMove.AngularVelocityDegrees = ITurnGeneratorInterface::Execute_GetTurn(TurnGenerator, IntendedOrientation_WorldSpace, StartState, *StartingSyncState, TimeStep, OutProposedMove, SimBlackboard);
	}
}

void UWalkingMode::SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState)
{
	UMoverComponent* MoverComp = GetMoverComponent();
	const FMoverTickStartData& StartState = Params.StartState;
	USceneComponent* UpdatedComponent = Params.MovingComps.UpdatedComponent.Get();
	FProposedMove ProposedMove = Params.ProposedMove;

	if (!UpdatedComponent)
	{
		return;
	}

	const FCharacterDefaultInputs* CharacterInputs = StartState.InputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>();
	const FMoverDefaultSyncState* StartingSyncState = StartState.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	check(StartingSyncState);

	FMoverDefaultSyncState& OutputSyncState = OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();

	UMoverBlackboard* SimBlackboard = MoverComp->GetSimBlackboard_Mutable();

	FRelativeBaseInfo CurrentBaseInfo;

	if (StartingSyncState->GetMovementBase() && SimBlackboard->TryGet(CommonBlackboard::LastFoundDynamicMovementBase, CurrentBaseInfo))
	{
		if (CurrentBaseInfo.HasRelativeInfo())
		{
			CurrentBaseInfo = UBasedMovementUtils::ApplyMovementBaseChange(MoverComp, CurrentBaseInfo, CommonLegacySettings->bIgnoreBaseRotation);
			SimBlackboard->Set(CommonBlackboard::LastFoundDynamicMovementBase, CurrentBaseInfo);
		}
	}

	const float DeltaSeconds = Params.TimeStep.StepMs * 0.001f;

	const FVector OrigMoveDelta = ProposedMove.LinearVelocity * DeltaSeconds;

	FMovementRecord MoveRecord;
	MoveRecord.SetDeltaSeconds(DeltaSeconds);

	FFloorCheckResult CurrentFloor;

	const FVector UpDirection = MoverComp->GetUpDirection();
	FMovingComponentSet MovingComponents(MoverComp);

	FFloorCheckSettings FloorCheckSettings;
	FloorCheckSettings.FloorSweepDistance = CommonLegacySettings->FloorSweepDistance;
	FloorCheckSettings.MaxWalkSlopeCosine = CommonLegacySettings->MaxWalkSlopeCosine;
	FloorCheckSettings.bUseFlatBaseForFloorChecks = CommonLegacySettings->bUseFlatBaseForFloorChecks;
	FloorCheckSettings.PerchRadiusThreshold = CommonLegacySettings->PerchRadiusThreshold;
	
	// If we don't have cached floor information, we need to search for it again
	if (!SimBlackboard->TryGet(CommonBlackboard::LastFloorResult, CurrentFloor))
	{
		UFloorQueryUtils::FindFloor(MovingComponents, FloorCheckSettings, UpdatedComponent->GetComponentLocation(), CurrentFloor);
	}

	OutputSyncState.MoveDirectionIntent = (ProposedMove.bHasDirIntent ? ProposedMove.DirectionIntent : FVector::ZeroVector);

	const FRotator StartingOrient = StartingSyncState->GetOrientation_WorldSpace();
	const FRotator TargetOrient = UMovementUtils::ApplyAngularVelocityToRotator(StartingOrient, ProposedMove.AngularVelocityDegrees, DeltaSeconds);
	
	const bool bIsOrientationChanging = !StartingOrient.Equals(TargetOrient);

	FQuat TargetOrientQuat = TargetOrient.Quaternion();
	if (CommonLegacySettings->bShouldRemainVertical)
	{
		TargetOrientQuat = FRotationMatrix::MakeFromZX(UpDirection, TargetOrientQuat.GetForwardVector()).ToQuat();
	}

	FHitResult MoveHitResult(1.f);
	
	FVector CurMoveDelta = OrigMoveDelta;
	
	FOptionalFloorCheckResult StepUpFloorResult;	// passed to sub-operations, so we can use their final floor results if they did a test

	bool bDidAttemptMovement = false;

	float PercentTimeAppliedSoFar = MoveHitResult.Time;

	if (!CurMoveDelta.IsNearlyZero() || bIsOrientationChanging)
	{
		// Attempt to move the full amount first
		bDidAttemptMovement = true;
		bool bMoved = UMovementUtils::TrySafeMoveUpdatedComponent(Params.MovingComps, CurMoveDelta, TargetOrientQuat, true, MoveHitResult, ETeleportType::None, MoveRecord);
		float LastMoveSeconds = DeltaSeconds;

		if (MoveHitResult.bStartPenetrating)
		{
			// We started by being stuck in geometry and need to resolve it first
			// TODO: try to resolve starting stuck in geometry
		}
		else if (MoveHitResult.IsValidBlockingHit())
		{
			// We impacted something (possibly a ramp, possibly a barrier)
			PercentTimeAppliedSoFar = MoveHitResult.Time;		

			// Check if the blockage is a walkable ramp rising in front of us
			if ((MoveHitResult.Time > 0.f) && (MoveHitResult.Normal.Dot(UpDirection) > UE_KINDA_SMALL_NUMBER) && 
			    UFloorQueryUtils::IsHitSurfaceWalkable(MoveHitResult, UpDirection, CommonLegacySettings->MaxWalkSlopeCosine))
			{
				// It's a walkable ramp, so cut up the move and attempt to move the remainder of it along the ramp's surface, possibly generating another hit
				const float PercentTimeRemaining = 1.f - PercentTimeAppliedSoFar;
				CurMoveDelta = UGroundMovementUtils::ComputeDeflectedMoveOntoRamp(CurMoveDelta * PercentTimeRemaining, UpDirection, MoveHitResult, CommonLegacySettings->MaxWalkSlopeCosine, CurrentFloor.bLineTrace);
				UMovementUtils::TrySafeMoveUpdatedComponent(Params.MovingComps, CurMoveDelta, TargetOrientQuat, true, MoveHitResult, ETeleportType::None, MoveRecord);
				LastMoveSeconds = PercentTimeRemaining * LastMoveSeconds;

				const float SecondHitPercent = MoveHitResult.Time * PercentTimeRemaining;
				PercentTimeAppliedSoFar = FMath::Clamp(PercentTimeAppliedSoFar + SecondHitPercent, 0.f, 1.f);
			}

			if (MoveHitResult.IsValidBlockingHit())
			{
				// If still blocked, try to step up onto the blocking object OR slide along it

				// JAH TODO: Take movement bases into account
				if (UGroundMovementUtils::CanStepUpOnHitSurface(MoveHitResult)) // || (CharacterOwner->GetMovementBase() != nullptr && Hit.HitObjectHandle == CharacterOwner->GetMovementBase()->GetOwner()))
				{
					// hit a barrier or unwalkable surface, try to step up and onto it
					const FVector PreStepUpLocation = UpdatedComponent->GetComponentLocation();
					const FVector DownwardDir = -MoverComp->GetUpDirection();

					
					float StepUpMovePercentage = UGroundMovementUtils::TryMoveToStepUp(Params.MovingComps, DownwardDir, CommonLegacySettings->MaxStepHeight, FloorCheckSettings, OrigMoveDelta * (1.f - PercentTimeAppliedSoFar), MoveHitResult, CurrentFloor, false, &StepUpFloorResult, MoveRecord);
					PercentTimeAppliedSoFar = FMath::Clamp(PercentTimeAppliedSoFar + (StepUpMovePercentage * (1.f - PercentTimeAppliedSoFar)), 0.f, 1.f);
					if (StepUpMovePercentage <= 0)
					{
						FMoverOnImpactParams ImpactParams(DefaultModeNames::Walking, MoveHitResult, OrigMoveDelta);
						MoverComp->HandleImpact(ImpactParams);
						float PercentAvailableToSlide = 1.f - PercentTimeAppliedSoFar;
						float SlideAmount = UGroundMovementUtils::TryWalkToSlideAlongSurface(Params.MovingComps, OrigMoveDelta, PercentAvailableToSlide, TargetOrientQuat, MoveHitResult.Normal, MoveHitResult, true, MoveRecord, CommonLegacySettings->MaxWalkSlopeCosine, CommonLegacySettings->MaxStepHeight);
						PercentTimeAppliedSoFar += PercentAvailableToSlide * SlideAmount;
					}
				}
				else if (MoveHitResult.Component.IsValid() && !MoveHitResult.Component.Get()->CanCharacterStepUp(Cast<APawn>(MoveHitResult.GetActor())))
				{
					FMoverOnImpactParams ImpactParams(DefaultModeNames::Walking, MoveHitResult, OrigMoveDelta);
					MoverComp->HandleImpact(ImpactParams);
					float PercentAvailableToSlide = 1.f - PercentTimeAppliedSoFar;
					float SlideAmount = UGroundMovementUtils::TryWalkToSlideAlongSurface(Params.MovingComps, OrigMoveDelta, 1.f - PercentTimeAppliedSoFar, TargetOrientQuat, MoveHitResult.Normal, MoveHitResult, true, MoveRecord, CommonLegacySettings->MaxWalkSlopeCosine, CommonLegacySettings->MaxStepHeight);
					PercentTimeAppliedSoFar += PercentAvailableToSlide * SlideAmount;
				}
			}
		}

		// Search for the floor we've ended up on
		UFloorQueryUtils::FindFloor(MovingComponents, FloorCheckSettings, UpdatedComponent->GetComponentLocation(), CurrentFloor);

		if (CurrentFloor.IsWalkableFloor())
		{
			UGroundMovementUtils::TryMoveToAdjustHeightAboveFloor(MoverComp, CurrentFloor, CommonLegacySettings->MaxWalkSlopeCosine, MoveRecord);
		}
    
		if (!CurrentFloor.IsWalkableFloor() && !CurrentFloor.HitResult.bStartPenetrating)
		{
			// No floor or not walkable, so let's let the airborne movement mode deal with it
			OutputState.MovementEndState.NextModeName = CommonLegacySettings->AirMovementModeName;
			OutputState.MovementEndState.RemainingMs = Params.TimeStep.StepMs - (Params.TimeStep.StepMs * PercentTimeAppliedSoFar);
			MoveRecord.SetDeltaSeconds((Params.TimeStep.StepMs - OutputState.MovementEndState.RemainingMs) * 0.001f);
			CaptureFinalState(StartingSyncState, UpdatedComponent, bDidAttemptMovement, CurrentFloor, MoveRecord, Params.TimeStep, ProposedMove.AngularVelocityDegrees, OutputSyncState);

			return;
		}
	}
	else
	{
		// If the actor isn't moving we still may need to check if they have a valid floor, such as if they're on an elevator platform moving up/down
		if ((FloorCheckPolicy == EStaticFloorCheckPolicy::Always) || 
		    (FloorCheckPolicy == EStaticFloorCheckPolicy::OnDynamicBaseOnly && StartingSyncState->GetMovementBase()))
		{
			UFloorQueryUtils::FindFloor(MovingComponents, FloorCheckSettings, UpdatedComponent->GetComponentLocation(), CurrentFloor);
		
			FHitResult Hit(CurrentFloor.HitResult);
			if (Hit.bStartPenetrating)
			{
				// The floor check failed because it started in penetration
				// We do not want to try to move downward because the downward sweep failed, rather we'd like to try to pop out of the floor.
				Hit.TraceEnd = Hit.TraceStart + UpDirection * 2.4;
				FVector RequestedAdjustment = UMovementUtils::ComputePenetrationAdjustment(Hit);
			
				const EMoveComponentFlags IncludeBlockingOverlapsWithoutEvents = (MOVECOMP_NeverIgnoreBlockingOverlaps | MOVECOMP_DisableBlockingOverlapDispatch);
				EMoveComponentFlags MoveComponentFlags = MOVECOMP_NoFlags;
				MoveComponentFlags = (MoveComponentFlags | IncludeBlockingOverlapsWithoutEvents);
				UMovementUtils::TryMoveToResolvePenetration(Params.MovingComps, MoveComponentFlags, RequestedAdjustment, Hit, UpdatedComponent->GetComponentQuat(), MoveRecord);
			}
		
			if (!CurrentFloor.IsWalkableFloor() && !Hit.bStartPenetrating)
			{
				// No floor or not walkable, so let's let the airborne movement mode deal with it
				OutputState.MovementEndState.NextModeName = CommonLegacySettings->AirMovementModeName;
				OutputState.MovementEndState.RemainingMs = Params.TimeStep.StepMs;
				MoveRecord.SetDeltaSeconds((Params.TimeStep.StepMs - OutputState.MovementEndState.RemainingMs) * 0.001f);
				CaptureFinalState(StartingSyncState, UpdatedComponent, bDidAttemptMovement, CurrentFloor, MoveRecord, Params.TimeStep, ProposedMove.AngularVelocityDegrees, OutputSyncState);

				return;
			}
		}
	}

	CaptureFinalState(StartingSyncState, UpdatedComponent, bDidAttemptMovement, CurrentFloor, MoveRecord, Params.TimeStep, ProposedMove.AngularVelocityDegrees, OutputSyncState);

}

UObject* UWalkingMode::GetTurnGenerator()
{
	return TurnGenerator;
}

void UWalkingMode::SetTurnGeneratorClass(TSubclassOf<UObject> TurnGeneratorClass)
{
	if (TurnGeneratorClass)
	{
		TurnGenerator = NewObject<UObject>(this, TurnGeneratorClass);
	}
	else
	{
		TurnGenerator = nullptr; // Clearing the turn generator is valid - will go back to the default turn generation
	}
}


void UWalkingMode::OnRegistered(const FName ModeName, const FMoverSimContext& SimContext)
{
	Super::OnRegistered(ModeName, SimContext);

	CommonLegacySettings = GetMoverComponent()->FindSharedSettings<UCommonLegacyMovementSettings>();
	ensureMsgf(CommonLegacySettings, TEXT("Failed to find instance of CommonLegacyMovementSettings on %s. Movement may not function properly."), *GetPathNameSafe(this));
}

void UWalkingMode::OnUnregistered(const FMoverSimContext& SimContext)
{
	CommonLegacySettings = nullptr;

	Super::OnUnregistered(SimContext);
}

void UWalkingMode::CaptureFinalState(const FMoverDefaultSyncState* StartingState, USceneComponent* UpdatedComponent, bool bDidAttemptMovement, const FFloorCheckResult& FloorResult, const FMovementRecord& Record, const FMoverTimeStep& TimeStep, const FVector& AngularVelocityDegrees, FMoverDefaultSyncState& OutputSyncState) const
{
	const UMoverComponent* MoverComp = GetMoverComponent();
	UMoverBlackboard* SimBlackboard = MoverComp->GetSimBlackboard_Mutable();

	FRelativeBaseInfo PriorBaseInfo;
	const bool bHasPriorBaseInfo = SimBlackboard->TryGet(CommonBlackboard::LastFoundDynamicMovementBase, PriorBaseInfo);

	FRelativeBaseInfo CurrentBaseInfo = UpdateFloorAndBaseInfo(FloorResult, UpdatedComponent->GetComponentLocation());

	// If we're on a dynamic base and we're not trying to move, keep using the same relative actor location. This prevents slow relative 
	//  drifting that can occur from repeated floor sampling as the base moves through the world.
	if (!bDidAttemptMovement && CurrentBaseInfo.HasRelativeInfo()
		&& bHasPriorBaseInfo && PriorBaseInfo.UsesSameBase(CurrentBaseInfo))
	{
 		CurrentBaseInfo.ContactLocalPosition           = PriorBaseInfo.ContactLocalPosition;
 		CurrentBaseInfo.WorldspaceOffsetFromContactPos = PriorBaseInfo.WorldspaceOffsetFromContactPos;
	}

	// TODO: Update Main/large movement record with substeps from our local record
	
	if (CurrentBaseInfo.HasRelativeInfo())
	{
		SimBlackboard->Set(CommonBlackboard::LastFoundDynamicMovementBase, CurrentBaseInfo);

		OutputSyncState.SetTransforms_WorldSpace( UpdatedComponent->GetComponentLocation(),
												  UpdatedComponent->GetComponentRotation(),
												  Record.GetRelevantVelocity(),
												  AngularVelocityDegrees,
												  
												  CurrentBaseInfo.MovementBase.Get(), CurrentBaseInfo.BoneName);
	}
	else
	{
		SimBlackboard->Invalidate(CommonBlackboard::LastFoundDynamicMovementBase);

		FVector NewWorldVelocity = Record.GetRelevantVelocity();

		// If we're walking off a movement base, keep the base's velocity so we carry momentum
		if (const UPrimitiveComponent* BaseComp = StartingState->GetMovementBase())
		{
			FVector DepartingBaseVelocity;
			if (UBasedMovementUtils::GetMovementBaseVelocityAtPoint(BaseComp, UpdatedComponent->GetComponentLocation(), OUT DepartingBaseVelocity))
			{
				NewWorldVelocity += DepartingBaseVelocity;
			}
		}

		OutputSyncState.SetTransforms_WorldSpace( UpdatedComponent->GetComponentLocation(),
												  UpdatedComponent->GetComponentRotation(),
												  NewWorldVelocity,
												  AngularVelocityDegrees,
												  nullptr);	// no movement base
	}

	UpdatedComponent->ComponentVelocity = OutputSyncState.GetVelocity_WorldSpace();
}


FRelativeBaseInfo UWalkingMode::UpdateFloorAndBaseInfo(const FFloorCheckResult& FloorResult, FVector CurrentLocation) const
{
	FRelativeBaseInfo ReturnBaseInfo;

	const UMoverComponent* MoverComp = GetMoverComponent();
	UMoverBlackboard* SimBlackboard = MoverComp->GetSimBlackboard_Mutable();

	SimBlackboard->Set(CommonBlackboard::LastFloorResult, FloorResult);

	if (FloorResult.IsWalkableFloor() && UBasedMovementUtils::IsADynamicBase(FloorResult.HitResult.GetComponent()))
	{
		ReturnBaseInfo.SetFromFloorResult(FloorResult, CurrentLocation, MoverComp->GetUpDirection());
	}

	return ReturnBaseInfo;
}
