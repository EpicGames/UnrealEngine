// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultMovementSet/MovementModifiers/StanceModifier.h"

#include "MoverComponent.h"
#include "Components/CapsuleComponent.h"
#include "DefaultMovementSet/CharacterMoverComponent.h"
#include "DefaultMovementSet/Settings/CommonLegacyMovementSettings.h"
#include "DefaultMovementSet/InstantMovementEffects/BasicInstantMovementEffects.h"
#include "DefaultMovementSet/Settings/StanceSettings.h"
#include "Engine/Blueprint.h"
#include "GameFramework/Pawn.h"
#include "MoveLibrary/MovementUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StanceModifier)

FStanceModifier::FStanceModifier()
	: ActiveStance(EStanceMode::Crouch)
{
	DurationMs = -1.0f;
}

bool FStanceModifier::HasGameplayTag(FGameplayTag TagToFind, bool bExactMatch) const
{
	// TODO: Eventually check for other stance tags here like prone
	if (bExactMatch)
	{
		return TagToFind.MatchesTagExact(Mover_IsCrouching);
	}

	return TagToFind.MatchesTag(Mover_IsCrouching);
}

void FStanceModifier::GetGameplayTags(FGameplayTagContainer& InOutTags) const
{
	// TODO: Eventually add other stance tags here like prone
	InOutTags.AddTag(Mover_IsCrouching);
}

void FStanceModifier::OnStart(UMoverComponent* MoverComp, const FMoverTimeStep& TimeStep, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState)
{
	if (UStanceSettings* StanceSettings = MoverComp->FindSharedSettings_Mutable<UStanceSettings>())
	{
		if (const UCapsuleComponent* CapsuleComponent = Cast<UCapsuleComponent>(MoverComp->GetUpdatedComponent()))
		{
			float OldHalfHeight = CapsuleComponent->GetScaledCapsuleHalfHeight();
			float NewHalfHeight = 0;
			float NewEyeHeight = 0;

			switch (ActiveStance)
			{
			default:
			case EStanceMode::Crouch:
				NewHalfHeight = StanceSettings->CrouchHalfHeight;
				NewEyeHeight = StanceSettings->CrouchedEyeHeight;
				break;

			// Prone isn't currently implemented
			case EStanceMode::Prone:
				UE_LOGF(LogMover, Warning, "Stance got into prone stance - That stance is not currently implemented.");
				// TODO: returning here so we don't apply any bad state to actor in case prone was set. Eventually, the return should be removed once prone is implemented properly
				DurationMs = 0;
				return;
			}
			
			AdjustCapsule(MoverComp, OldHalfHeight, NewHalfHeight, NewEyeHeight);
			ApplyMovementSettings(MoverComp);
		}
	}
}

void FStanceModifier::OnEnd(UMoverComponent* MoverComp, const FMoverTimeStep& TimeStep, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState)
{
	const AActor* OwnerCDO = Cast<AActor>(MoverComp->GetOwner()->GetClass()->GetDefaultObject());

	if (UCapsuleComponent* CapsuleComponent = Cast<UCapsuleComponent>(MoverComp->GetUpdatedComponent()))
	{
		if (const UCapsuleComponent* OriginalCapsule = UMovementUtils::GetOriginalComponentType<UCapsuleComponent>(MoverComp->GetOwner()))
		{
			if (const APawn* OwnerCDOAsPawn = Cast<APawn>(OwnerCDO))
			{
				AdjustCapsule(MoverComp, CapsuleComponent->GetScaledCapsuleHalfHeight(), OriginalCapsule->GetScaledCapsuleHalfHeight(), OwnerCDOAsPawn->BaseEyeHeight);
				RevertMovementSettings(MoverComp);
			}
		}
	}
}

void FStanceModifier::OnPreMovement(UMoverComponent* MoverComp, const FMoverTimeStep& TimeStep)
{
	// TODO: Check for different inputs/state here and manage swapping between stances - use AdjustCapsule and Apply/Revert movement settings.

	// TODO: Prone isn't currently implemented - so we're just going to cancel the modifier if we got into that state
	if (ActiveStance == EStanceMode::Prone)
	{
		UE_LOGF(LogMover, Warning, "Stance got into prone stance - That stance is not currently implemented.");
		DurationMs = 0;
	}
}

void FStanceModifier::OnPostMovement(UMoverComponent* MoverComp, const FMoverTimeStep& TimeStep, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState)
{
	FMovementModifierBase::OnPostMovement(MoverComp, TimeStep, SyncState, AuxState);
}

FMovementModifierBase* FStanceModifier::Clone() const
{
	FStanceModifier* CopyPtr = new FStanceModifier(*this);
	return CopyPtr;
}

void FStanceModifier::NetSerialize(FArchive& Ar)
{
	Super::NetSerialize(Ar);
}

UScriptStruct* FStanceModifier::GetScriptStruct() const
{
	return FStanceModifier::StaticStruct();
}

FString FStanceModifier::ToSimpleString() const
{
	return FString::Printf(TEXT("Stance Modifier"));
}

void FStanceModifier::AddReferencedObjects(FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);
}

bool FStanceModifier::CanExpand(const UCharacterMoverComponent* MoverComp) const
{
	float StandingHalfHeight = 90;
	float CurrentHalfHeight = 55;

	USceneComponent* UpdatedComponent = MoverComp->GetUpdatedComponent();
	UPrimitiveComponent* UpdatedCompAsPrimitive = Cast<UPrimitiveComponent>(UpdatedComponent);
	
	if (const UCapsuleComponent* OriginalCapsule = UMovementUtils::GetOriginalComponentType<UCapsuleComponent>(MoverComp->GetOwner()))
	{
		StandingHalfHeight = OriginalCapsule->GetScaledCapsuleHalfHeight();
	}

	if (UCapsuleComponent* CapsuleComponent = Cast<UCapsuleComponent>(UpdatedComponent))
	{
		CurrentHalfHeight = CapsuleComponent->GetScaledCapsuleHalfHeight();
	}

	float HalfHeightDifference = StandingHalfHeight - CurrentHalfHeight;
	
	// TODO: pluggable shapes
	FCollisionQueryParams CapsuleParams(SCENE_QUERY_STAT(CrouchTrace), false, MoverComp->GetOwner());
	FCollisionResponseParams ResponseParam;
	UMovementUtils::InitCollisionParams(UpdatedCompAsPrimitive, CapsuleParams, ResponseParam);

	const FMoverDefaultSyncState* SyncState = MoverComp->GetSyncState().SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	
	FVector PawnLocation = SyncState->GetLocation_WorldSpace();
	FQuat PawnRot = SyncState->GetOrientation_WorldSpace().Quaternion();
	float PawnRadius = 0.0f;
	float PawnHalfHeight = 0.0f;
	UpdatedCompAsPrimitive->CalcBoundingCylinder(PawnRadius, PawnHalfHeight);
	
	// TODO: Compensate for the difference between current capsule size and standing size
	FCollisionShape StandingCapsuleShape = FCollisionShape::MakeCapsule(PawnRadius, StandingHalfHeight);
	const ECollisionChannel CollisionChannel = UpdatedCompAsPrimitive->GetCollisionObjectType();
	bool bEncroached = true;

	// TODO: We may need to expand this check to look at more than just the initial overlap - see CMC Uncrouch for details
	if (!ShouldExpandingMaintainBase(MoverComp))
	{
		// Expand in place
		bEncroached = UMovementUtils::OverlapTest(UpdatedComponent, UpdatedCompAsPrimitive, PawnLocation, PawnRot, CollisionChannel, StandingCapsuleShape, MoverComp->GetOwner());
	}
	else
	{
		// Expand while keeping base location the same.
		FVector StandingLocation = PawnLocation + (HalfHeightDifference + .01f) * MoverComp->GetUpDirection();
		bEncroached = UMovementUtils::OverlapTest(UpdatedComponent, UpdatedCompAsPrimitive, StandingLocation, PawnRot, CollisionChannel, StandingCapsuleShape, MoverComp->GetOwner());
	}

	return !bEncroached;
}

bool FStanceModifier::ShouldExpandingMaintainBase(const UCharacterMoverComponent* MoverComp) const
{
	if (MoverComp->IsOnGround())
	{
		return true;
	}

	return false;
}

void FStanceModifier::AdjustCapsule(UMoverComponent* MoverComp, float OldHalfHeight, float NewHalfHeight, float NewEyeHeight)
{
	const float HalfHeightDifference = FMath::Abs(NewHalfHeight - OldHalfHeight);
	const bool bExpanding = OldHalfHeight < NewHalfHeight;
	
	// Set capsule size to crouching size
	if (UCapsuleComponent* CapsuleComponent = Cast<UCapsuleComponent>(MoverComp->GetOwner()->FindComponentByClass(UCapsuleComponent::StaticClass())))
	{
		if (CapsuleComponent->GetUnscaledCapsuleHalfHeight() == NewHalfHeight)
		{
			return;
		}
		
		CapsuleComponent->SetCapsuleSize(CapsuleComponent->GetUnscaledCapsuleRadius(), NewHalfHeight, false);
	}

	// update eye height on pawn
	if (APawn* MoverCompOwnerAsPawn = Cast<APawn>(MoverComp->GetOwner()))
	{
		MoverCompOwnerAsPawn->BaseEyeHeight = NewEyeHeight;
	}

	const FVector CapsuleOffset = MoverComp->GetUpDirection() * (bExpanding ? HalfHeightDifference : -HalfHeightDifference);
	// This is only getting used to add relative offset - so assuming z is up is fine here
	const FVector VisualOffset = FVector::UpVector * (bExpanding ? -HalfHeightDifference : HalfHeightDifference);
	
	// Adjust location of capsule as setting it's size left it floating
	TSharedPtr<FTeleportEffect> TeleportEffect = MakeShared<FTeleportEffect>();
	TeleportEffect->TargetLocation = MoverComp->GetUpdatedComponentTransform().GetLocation() + (CapsuleOffset);
	MoverComp->QueueInstantMovementEffect(TeleportEffect);
	
	// Add offset to visual component as the base location has changed
	FTransform MoverVisualComponentOffset = MoverComp->GetBaseVisualComponentTransform();
	MoverVisualComponentOffset.SetLocation(MoverVisualComponentOffset.GetLocation() + VisualOffset);
	MoverComp->SetBaseVisualComponentTransform(MoverVisualComponentOffset);
	
	MoverComp->GetPrimaryVisualComponent()->SetRelativeLocation(MoverVisualComponentOffset.GetLocation(), false, nullptr, ETeleportType::TeleportPhysics);
}

void FStanceModifier::ApplyMovementSettings(UMoverComponent* MoverComp)
{
	switch (ActiveStance)
	{
	default:
	case EStanceMode::Crouch:
		if (UStanceSettings* StanceSettings = MoverComp->FindSharedSettings_Mutable<UStanceSettings>())
		{
			// Update relevant movement settings
			if (UCommonLegacyMovementSettings* MovementSettings = MoverComp->FindSharedSettings_Mutable<UCommonLegacyMovementSettings>())
			{
				MovementSettings->Acceleration = StanceSettings->CrouchingMaxAcceleration;
				MovementSettings->MaxSpeed = StanceSettings->CrouchingMaxSpeed;
			}
		}
		
		break;

	// Prone isn't currently implemented properly so we're just doing nothing for now
	case EStanceMode::Prone:
		UE_LOGF(LogMover, Warning, "Stance got into prone stance - That mode is not currently implemented fully.");
		break;
	}
}

void FStanceModifier::RevertMovementSettings(UMoverComponent* MoverComp)
{
	if (const UMoverComponent* CDOMoverComp = UMovementUtils::GetOriginalComponentType<UMoverComponent>(MoverComp->GetOwner()))
	{
		const UCommonLegacyMovementSettings* OriginalMovementSettings = CDOMoverComp->FindSharedSettings<UCommonLegacyMovementSettings>();
		UCommonLegacyMovementSettings* MovementSettings = MoverComp->FindSharedSettings_Mutable<UCommonLegacyMovementSettings>();
		
		// Revert movement settings back to original settings
		if (MovementSettings && OriginalMovementSettings)
		{
			MovementSettings->Acceleration = OriginalMovementSettings->Acceleration;
			MovementSettings->MaxSpeed = OriginalMovementSettings->MaxSpeed;
		}
	}
}
