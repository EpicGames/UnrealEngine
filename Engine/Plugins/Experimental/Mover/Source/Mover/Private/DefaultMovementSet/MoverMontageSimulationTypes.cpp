// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultMovementSet/MoverMontageSimulationTypes.h"
#include "Animation/AnimMontage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoverMontageSimulationTypes)

void FMoverSimDrivenMontageData::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct)
{
	const FMoverSimDrivenMontageData& FromData = static_cast<const FMoverSimDrivenMontageData&>(From);
	const FMoverSimDrivenMontageData& ToData   = static_cast<const FMoverSimDrivenMontageData&>(To);

	// Start from To (authoritative for Montage ptr, PlayRate, timing thresholds, etc.)
	MontageStates = ToData.MontageStates;

	// Lerp CurrentPosition and compute precise lifecycle flag timing for each matching entry
	for (FMoverSimDrivenMontageEntry& Entry : MontageStates)
	{
		const FMoverSimDrivenMontageEntry* FromEntry = FromData.MontageStates.FindByPredicate(
			[&](const FMoverSimDrivenMontageEntry& E) { return E.Montage == Entry.Montage; });

		if (FromEntry)
		{
			Entry.CurrentPosition = FMath::Lerp(FromEntry->CurrentPosition, Entry.CurrentPosition, Pct);

			// Reconstruct the interpolated simulation time so lifecycle events fire at the exact moment the
			// display time crosses the threshold, rather than up to one substep early (the To-frame bias).
			const float InterpolatedSimTimeMs = FMath::Lerp(FromEntry->FrameSimTimeMs, Entry.FrameSimTimeMs, Pct);
			Entry.bShouldBlendOut = Entry.BlendOutSimTimeMs >= 0.f && InterpolatedSimTimeMs >= Entry.BlendOutSimTimeMs;
			Entry.bIsFinished     = Entry.FinishSimTimeMs  >= 0.f && InterpolatedSimTimeMs >= Entry.FinishSimTimeMs;
		}
	}

	// Pass through any From-only entries that carried a lifecycle signal.
	// A montage can be present in From but absent in To when it finished during the [From, To]
	// interval: GenerateMove_Async sets bIsFinished (or bShouldBlendOut) on substep N,
	// then FlushMoveArrays removes the move before substep N+1, so To has no entry for it.
	// Without this, FlushMontageStates would find the montage missing from SeenMontages, see
	// bShouldBlendOut=false in DrivenMontageStateMap (never updated), and hard-stop the montage
	// via StopDrivenMontage(0) -- discarding any blend-out.
	// Abrupt cancellations (both flags false) are intentionally excluded: that case is handled
	// by the StopDrivenMontage path in FlushMontageStates.
	for (const FMoverSimDrivenMontageEntry& FromEntry : FromData.MontageStates)
	{
		const bool bPresentInTo = MontageStates.ContainsByPredicate(
			[&](const FMoverSimDrivenMontageEntry& E) { return E.Montage == FromEntry.Montage; });

		if (!bPresentInTo && (FromEntry.bIsFinished || FromEntry.bShouldBlendOut))
		{
			MontageStates.Add(FromEntry);
		}
	}
}
