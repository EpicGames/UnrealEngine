// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultMovementSet/NavMoverComponent.h"

#include "AI/NavigationSystemBase.h"
#include "AI/Navigation/PathFollowingAgentInterface.h"
#include "Components/CapsuleComponent.h"
#include "DefaultMovementSet/InstantMovementEffects/BasicInstantMovementEffects.h"
#include "DefaultMovementSet/Settings/CommonLegacyMovementSettings.h"
#include "MoveLibrary/MovementUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavMoverComponent)

UNavMoverComponent::UNavMoverComponent()
	: AvoidanceUID(0)
	, AvoidanceWeight(0.5)
	, AvoidanceConsiderationRadius(500)
{
	bWantsInitializeComponent = true;
	bAutoActivate = true;
	
	AvoidanceGroup.bGroup0 = true;
	GroupsToAvoid.Packed = 0xFFFFFFFF;
	GroupsToIgnore.Packed = 0;
}

void UNavMoverComponent::SetAvoidanceGroupMask(const FNavAvoidanceMask& GroupMask)
{
	SetAvoidanceGroupMask(GroupMask.Packed);
}

void UNavMoverComponent::SetGroupsToAvoidMask(const FNavAvoidanceMask& GroupMask)
{
	SetGroupsToAvoidMask(GroupMask.Packed);
}

void UNavMoverComponent::SetGroupsToIgnoreMask(const FNavAvoidanceMask& GroupMask)
{
	SetGroupsToIgnoreMask(GroupMask.Packed);
}

void UNavMoverComponent::InitializeComponent()
{
	Super::InitializeComponent();

	if (const AActor* MovementCompOwner = GetOwner())
	{
		MoverComponent = MovementCompOwner->FindComponentByClass<UMoverComponent>();
	}
	
	if (!MoverComponent.IsValid())
	{
		UE_LOGF(LogMover, Warning, "NavMoverComponent on %ls could not find a valid MoverComponent and will not function properly.", *GetNameSafe(GetOwner()));
	}
}

void UNavMoverComponent::BeginPlay()
{
	Super::BeginPlay();

	if (MoverComponent.IsValid() && MoverComponent->GetUpdatedComponent())
	{
		UpdateNavAgent(*MoverComponent->GetUpdatedComponent());
	}
	else
	{
		UpdateNavAgent(*GetOwner());
	}
}

float UNavMoverComponent::GetMaxSpeedForNavMovement() const
{
	float MaxSpeed = 0.0f;
	
	if (MoverComponent.IsValid())
	{
		if (const UCommonLegacyMovementSettings* MovementSettings = MoverComponent.Get()->FindSharedSettings_Mutable<UCommonLegacyMovementSettings>())
		{
			MaxSpeed = MovementSettings->MaxSpeed;
		}
	}

	return MaxSpeed;
}

void UNavMoverComponent::StopMovementImmediately()
{
	if (MoverComponent.IsValid())
	{
		TSharedPtr<FApplyVelocityEffect> VelocityEffect = MakeShared<FApplyVelocityEffect>();
		MoverComponent->QueueInstantMovementEffect(VelocityEffect);
	}
	
	CachedNavMoveInputIntent = FVector::ZeroVector;
	CachedNavMoveInputVelocity = FVector::ZeroVector;
}

bool UNavMoverComponent::ConsumeNavMovementData(FVector& OutMoveInputIntent, FVector& OutMoveInputVelocity)
{
	const bool bHasFrameAdvanced = GFrameCounter > GameFrameNavMovementConsumed;
	const bool bNoNewRequests = GameFrameNavMovementConsumed > GameFrameNavMovementRequested;
	bool bHasNavMovement = false;
	
	if (bHasFrameAdvanced && bNoNewRequests)
	{
		CachedNavMoveInputIntent = FVector::ZeroVector;
		CachedNavMoveInputVelocity = FVector::ZeroVector;
	}
	else
	{
		OutMoveInputIntent = CachedNavMoveInputIntent;
		OutMoveInputVelocity = CachedNavMoveInputVelocity;
		bHasNavMovement = true;
		
		UE_LOGF(LogMover, VeryVerbose, "Applying %ls as NavMoveInputIntent.", *CachedNavMoveInputIntent.ToString());
		UE_LOGF(LogMover, VeryVerbose, "Applying %ls as NavMoveInputVelocity.", *CachedNavMoveInputVelocity.ToString());
	}

	GameFrameNavMovementConsumed = GFrameCounter;

	return bHasNavMovement;
}

FVector UNavMoverComponent::GetLocation() const
{
	if (MoverComponent.IsValid())
	{
		if (const USceneComponent* UpdatedComponent = MoverComponent->GetUpdatedComponent())
		{
			return UpdatedComponent->GetComponentLocation();
		}
	}
	
	return FVector(FLT_MAX);
}

FVector UNavMoverComponent::GetFeetLocation() const
{
	if (MoverComponent.IsValid())
	{
		if (const USceneComponent* UpdatedComponent = MoverComponent->GetUpdatedComponent())
		{
			return UpdatedComponent->GetComponentLocation() - FVector(0,0,UpdatedComponent->Bounds.BoxExtent.Z);
		}
	}

	return FNavigationSystem::InvalidLocation;
}

FVector UNavMoverComponent::GetFeetLocationAt(FVector ComponentLocation) const
{
	if (MoverComponent.IsValid())
	{
		if (const USceneComponent* UpdatedComponent = MoverComponent->GetUpdatedComponent())
		{
			return ComponentLocation - FVector(0, 0, UpdatedComponent->Bounds.BoxExtent.Z);
		}
	}
	
	return FNavigationSystem::InvalidLocation;
}

FBasedPosition UNavMoverComponent::GetFeetLocationBased() const
{
	FBasedPosition BasedPosition;
	if (MoverComponent.IsValid())
	{
		bool bSetFromMoverBlackboard = false;
		if (const UMoverBlackboard* Blackboard = MoverComponent->GetSimBlackboard())
		{
			FRelativeBaseInfo MovementBaseInfo;
			if (Blackboard->TryGet(CommonBlackboard::LastFoundDynamicMovementBase, MovementBaseInfo)) 
			{
				if (MovementBaseInfo.HasRelativeInfo())
				{
					bSetFromMoverBlackboard = true;
					BasedPosition.Base = MovementBaseInfo.MovementBase->GetOwner();
					BasedPosition.Position = MovementBaseInfo.ContactLocalPosition;
					BasedPosition.CachedBaseLocation = MovementBaseInfo.Location;
					BasedPosition.CachedBaseRotation = MovementBaseInfo.Rotation.Rotator();
					BasedPosition.CachedTransPosition = GetFeetLocation();
				}
			}
		}
		
		if (!bSetFromMoverBlackboard)
		{
			BasedPosition.Set(nullptr, GetFeetLocation());
		}
	}

	return BasedPosition;
}

void UNavMoverComponent::UpdateNavAgent(const UObject& ObjectToUpdateFrom)
{
	if (!NavMovementProperties.bUpdateNavAgentWithOwnersCollision)
	{
		return;
	}
	
	if (const UCapsuleComponent* CapsuleComponent = Cast<UCapsuleComponent>(&ObjectToUpdateFrom))
	{
		NavAgentProps.AgentRadius = CapsuleComponent->GetScaledCapsuleRadius();
		NavAgentProps.AgentHeight = CapsuleComponent->GetScaledCapsuleHalfHeight() * 2.f;;
	}
	else if (const AActor* ObjectAsActor = Cast<AActor>(&ObjectToUpdateFrom))
	{
		ensureMsgf(&ObjectToUpdateFrom == GetOwner(), TEXT("Object passed to UpdateNavAgent should be the owner actor of the Nav Movement Component"));
		// Can't call GetSimpleCollisionCylinder(), because no components will be registered.
		float BoundRadius, BoundHalfHeight;	
		ObjectAsActor->GetSimpleCollisionCylinder(BoundRadius, BoundHalfHeight);
		NavAgentProps.AgentRadius = BoundRadius;
		NavAgentProps.AgentHeight = BoundHalfHeight * 2.f;
	}
}

void UNavMoverComponent::RequestDirectMove(const FVector& MoveVelocity, bool bForceMaxSpeed)
{
	if (MoveVelocity.SizeSquared() < UE_KINDA_SMALL_NUMBER)
	{
		return;
	}

	GameFrameNavMovementRequested = GFrameCounter;
	
	if (IsFalling())
	{
		const FVector FallVelocity = MoveVelocity.GetClampedToMaxSize(GetMaxSpeedForNavMovement());
		// TODO: NS - we may eventually need something to help with air control and pathfinding
		//PerformAirControlForPathFollowing(FallVelocity, FallVelocity.Z);
		CachedNavMoveInputVelocity = FallVelocity;
		return;
	}

	CachedNavMoveInputVelocity = MoveVelocity;
	
	if (IsMovingOnGround())
	{
		const FPlane MovementPlane(FVector::ZeroVector, FVector::UpVector);
		CachedNavMoveInputVelocity = UMovementUtils::ConstrainToPlane(CachedNavMoveInputVelocity, MovementPlane, true);
	}
}

void UNavMoverComponent::RequestPathMove(const FVector& MoveInput)
{
	FVector AdjustedMoveInput(MoveInput);

	// preserve magnitude when moving on ground/falling and requested input has Z component
	// see ConstrainInputAcceleration for details
	if (MoveInput.Z != 0.f && (IsMovingOnGround() || IsFalling()))
	{
		const float Mag = MoveInput.Size();
		AdjustedMoveInput = MoveInput.GetSafeNormal2D() * Mag;
	}

	GameFrameNavMovementRequested = GFrameCounter;
	CachedNavMoveInputIntent = AdjustedMoveInput.GetSafeNormal();
}

bool UNavMoverComponent::CanStopPathFollowing() const
{
	return true;
}

void UNavMoverComponent::SetPathFollowingAgent(IPathFollowingAgentInterface* InPathFollowingAgent)
{
	PathFollowingComp = InPathFollowingAgent;
}

IPathFollowingAgentInterface* UNavMoverComponent::GetPathFollowingAgent()
{
	return PathFollowingComp.Get();
}

const IPathFollowingAgentInterface* UNavMoverComponent::GetPathFollowingAgent() const
{
	return PathFollowingComp.Get();
}

const FNavAgentProperties& UNavMoverComponent::GetNavAgentPropertiesRef() const
{
	return NavAgentProps;
}

FNavAgentProperties& UNavMoverComponent::GetNavAgentPropertiesRef()
{
	return NavAgentProps;
}

void UNavMoverComponent::ResetMoveState()
{
	MovementState = NavAgentProps;
}

bool UNavMoverComponent::CanStartPathFollowing() const
{
	return true;
}

bool UNavMoverComponent::IsCrouching() const
{
	return MoverComponent.IsValid() ? MoverComponent->HasGameplayTag(Mover_IsCrouching, true) : false;
}

bool UNavMoverComponent::IsFalling() const
{
	return MoverComponent.IsValid() ? MoverComponent->HasGameplayTag(Mover_IsFalling, true) : false;
}

bool UNavMoverComponent::IsMovingOnGround() const
{
	return MoverComponent.IsValid() ? MoverComponent->HasGameplayTag(Mover_IsOnGround, true) : false;
}

bool UNavMoverComponent::IsSwimming() const
{
	return MoverComponent.IsValid() ? MoverComponent->HasGameplayTag(Mover_IsSwimming, true) : false;
}

bool UNavMoverComponent::IsFlying() const
{
	return MoverComponent.IsValid() ? MoverComponent->HasGameplayTag(Mover_IsFlying, true) : false;
}

void UNavMoverComponent::GetSimpleCollisionCylinder(float& CollisionRadius, float& CollisionHalfHeight) const
{
	GetOwner()->GetSimpleCollisionCylinder(CollisionRadius, CollisionHalfHeight);
}

FVector UNavMoverComponent::GetSimpleCollisionCylinderExtent() const
{
	return GetOwner()->GetSimpleCollisionCylinderExtent();
}

FVector UNavMoverComponent::GetForwardVector() const
{
	return GetOwner()->GetActorForwardVector();
}

FVector UNavMoverComponent::GetVelocityForNavMovement() const
{
	return MoverComponent.IsValid() ? MoverComponent->GetVelocity() : FVector::ZeroVector;
}

void UNavMoverComponent::SetRVOAvoidanceUID(int32 UID)
{
	AvoidanceUID = UID;
}

int32 UNavMoverComponent::GetRVOAvoidanceUID()
{
	return AvoidanceUID;
}

void UNavMoverComponent::SetRVOAvoidanceWeight(float Weight)
{
	AvoidanceWeight = Weight;
}

float UNavMoverComponent::GetRVOAvoidanceWeight()
{
	return AvoidanceWeight;
}

FVector UNavMoverComponent::GetRVOAvoidanceOrigin()
{
	return GetFeetLocation();
}

float UNavMoverComponent::GetRVOAvoidanceRadius()
{
	const UCapsuleComponent* CapsuleComp = Cast<UCapsuleComponent>(GetUpdatedObject());
	return CapsuleComp ? CapsuleComp->GetScaledCapsuleRadius() : 0.0f;
}

float UNavMoverComponent::GetRVOAvoidanceConsiderationRadius()
{
	return AvoidanceConsiderationRadius;
}

float UNavMoverComponent::GetRVOAvoidanceHeight()
{
	const UCapsuleComponent* CapsuleComp = Cast<UCapsuleComponent>(GetUpdatedObject());
	return CapsuleComp ? CapsuleComp->GetScaledCapsuleHalfHeight() : 0.0f;
}

FVector UNavMoverComponent::GetVelocityForRVOConsideration()
{
	return GetVelocityForNavMovement();
}

void UNavMoverComponent::SetAvoidanceGroupMask(int32 GroupFlags)
{
	AvoidanceGroup.SetFlagsDirectly(GroupFlags);
}

int32 UNavMoverComponent::GetAvoidanceGroupMask()
{
	return AvoidanceGroup.Packed;
}

void UNavMoverComponent::SetGroupsToAvoidMask(int32 GroupFlags)
{
	GroupsToAvoid.SetFlagsDirectly(GroupFlags);
}

int32 UNavMoverComponent::GetGroupsToAvoidMask()
{
	return GroupsToAvoid.Packed;
}

void UNavMoverComponent::SetGroupsToIgnoreMask(int32 GroupFlags)
{
	GroupsToIgnore.SetFlagsDirectly(GroupFlags);
}

int32 UNavMoverComponent::GetGroupsToIgnoreMask()
{
	return GroupsToIgnore.Packed;
}
