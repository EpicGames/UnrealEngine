// Copyright Epic Games, Inc. All Rights Reserved.


#include "InputContainerStruct.h"
#include "MoverLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputContainerStruct)

#define LOCTEXT_NAMESPACE "MoverInputContainerStruct"



void FMoverInputContainerDataStruct::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float LerpFactor)
{
	const FMoverInputContainerDataStruct* FromContainer = static_cast<const FMoverInputContainerDataStruct*>(&From);
	const FMoverInputContainerDataStruct* ToContainer = static_cast<const FMoverInputContainerDataStruct*>(&To);

	InputCollection.Interpolate(FromContainer->InputCollection, ToContainer->InputCollection, LerpFactor);
}


FMoverDataStructBase* FMoverInputContainerDataStruct::Clone() const
{
	FMoverInputContainerDataStruct* CopyPtr = new FMoverInputContainerDataStruct(*this);
	return CopyPtr;
}

bool FMoverInputContainerDataStruct::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	if (!Super::NetSerialize(Ar, Map, bOutSuccess))
	{
		bOutSuccess = false;
		return false;
	}

	if (!InputCollection.NetSerialize(Ar, Map, bOutSuccess))
	{
		bOutSuccess = false;
		return false;
	}

	bOutSuccess = true;
	return true;
}


#undef LOCTEXT_NAMESPACE
