// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstantMovementEffect.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstantMovementEffect)

FInstantMovementEffect* FInstantMovementEffect::Clone() const
{
	// If child classes don't override this, saved moves will not work
	checkf(false, TEXT("FInstantMovementEffect::Clone() being called erroneously from %s. A FInstantMovementEffect should never be queued directly and Clone should always be overridden in child structs!"), *GetNameSafe(GetScriptStruct()));
	return nullptr;
}

void FInstantMovementEffect::NetSerialize(FArchive& Ar)
{
	
}

UScriptStruct* FInstantMovementEffect::GetScriptStruct() const
{
	return FInstantMovementEffect::StaticStruct();
}

FString FInstantMovementEffect::ToSimpleString() const
{
	return GetScriptStruct()->GetName();
}
