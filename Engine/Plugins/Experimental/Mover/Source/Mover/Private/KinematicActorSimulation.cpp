// Copyright Epic Games, Inc. All Rights Reserved.

#include "KinematicActorSimulation.h"

#include "Components/PrimitiveComponent.h"
#include "MovementModeStateMachine.h"
#include "MoverComponent.h"


UKinematicActorSimulation::UKinematicActorSimulation()
	: UMoverSimulation()
{
}

void UKinematicActorSimulation::BeginDestroy()
{
	Uninit();
	Super::BeginDestroy();
}


void UKinematicActorSimulation::Init(const FInitParams& InInitParams)
{
	InitParams = InInitParams;
	MoverComp = InitParams.MoverComp.Get();

	ModeFSM = NewObject<UMovementModeStateMachine>(this, TEXT("KinematicActorStateMachine"), RF_Transient);
	ModeFSM->SetSimulation(*this);

	RegisterMovementModes(InitParams.ModesToRegister, InitParams.StartingMovementMode);
	RegisterGlobalTransitions(InitParams.TransitionsToRegister);


	if (InitParams.StartingMovementMode != NAME_None)
	{
		if (ModeFSM->FindMovementMode(InitParams.StartingMovementMode))
		{
			ModeFSM->SetDefaultMode(InitParams.StartingMovementMode);
			ModeFSM->QueueNextMode(InitParams.StartingMovementMode);
		}
		else
		{
			UE_LOGF(LogMover, Warning, "Invalid StartingMovementMode '%ls' specified on %ls. Mover actor will not function until something queues a valid mode.",
				*InitParams.StartingMovementMode.ToString(), *GetNameSafe(GetTypedOuter<AActor>()));
		}
	}
}


void UKinematicActorSimulation::Uninit()
{
	if (ModeFSM)
	{
		ModeFSM->ClearAllMovementModes();
		ModeFSM->ClearAllGlobalTransitions();
	}

	MoverComp = nullptr;
	ModeFSM = nullptr;
}


void UKinematicActorSimulation::InitializeSimulationState(FMoverSyncState& OutSync, FMoverAuxStateContext& OutAux)
{
	ensureMsgf(IsInGameThread(), TEXT("%hs must only be called on the game thread"), __FUNCTION__);
	check(MoverComp);


	OutSync = FMoverSyncState();	// start fresh
	OutAux = FMoverAuxStateContext();

	// Add all initial persistent sync state types
	for (const FMoverDataPersistence& PersistentSyncEntry : MoverComp->PersistentSyncStateDataTypes)
	{
		if (PersistentSyncEntry.RequiredType.Get()) // This can happen if a previously existing required type was removed, causing a crash
		{
			OutSync.SyncStateCollection.FindOrAddDataByType(PersistentSyncEntry.RequiredType);
		}
	}

	// Mirror the scene component transform if we have one, otherwise it will be left at origin
	if (const USceneComponent* UpdatedComponent = MoverComp->GetUpdatedComponent())
	{
		if (FMoverDefaultSyncState* MoverState = OutSync.SyncStateCollection.FindMutableDataByType<FMoverDefaultSyncState>())
		{
			MoverState->SetTransforms_WorldSpace(
				UpdatedComponent->GetComponentLocation(),
				UpdatedComponent->GetComponentRotation(),
				UpdatedComponent->GetComponentVelocity(),
				FVector::ZeroVector);	// no initial angular velocity. May be able to get this if it was a PrimitiveComponent or we had access to the associated physics body
		}
	}

	OutSync.MovementMode = InitParams.StartingMovementMode;

	LastSyncState = OutSync;
	LastAuxState = OutAux;
}

void UKinematicActorSimulation::DoSimulationTick(const FMoverTimeStep& InTimeStep, const FMoverTickStartData& StartData, OUT FMoverTickEndData& EndData)
{
	// Pre-simulate
	
	// TODO: Determination of bIsResimulating and bIsFirstResimFrame should be the responsibility of the liaison / backend
	const bool bIsResimulating = InTimeStep.BaseSimTimeMs <= NewestSimTickTimeStep.BaseSimTimeMs;

	FMoverTimeStep MoverTimeStep(InTimeStep);
	MoverTimeStep.bIsResimulating = bIsResimulating;
	MoverTimeStep.bIsFirstResimFrame = bJustStartedResim;
	bJustStartedResim = false;

	PrepareStateDataForNewFrame(MoverTimeStep, StartData, OUT EndData);

	CheckForExternalChanges(StartData);

	OnPreSimulationAdvance(MoverTimeStep, StartData, OUT EndData);

	
	// Perform simulation step
	AdvanceSimulation(MoverTimeStep, StartData, OUT EndData);


	// Post-simulate
	OnPostSimulationAdvance(MoverTimeStep, StartData, OUT EndData);


	if (MoverTimeStep.ServerFrame > NewestSimTickTimeStep.ServerFrame || MoverTimeStep.BaseSimTimeMs > NewestSimTickTimeStep.BaseSimTimeMs)
	{
		NewestSimTickTimeStep = MoverTimeStep;
	}

	LastSyncState = EndData.SyncState;
	LastAuxState = EndData.AuxState;

}

void UKinematicActorSimulation::RollbackToPriorState(const FMoverTimeStep& NewBaseTimeStep, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState)
{
	const FMoverSyncState& InvalidSyncState = LastSyncState;
	const FMoverAuxStateContext& InvalidAuxState = LastAuxState;

	ModeFSM->OnSimulationPreRollback(&InvalidSyncState, &SyncState, &InvalidAuxState, &AuxState, NewBaseTimeStep);

	Blackboard->Invalidate(EInvalidationReason::Rollback);
	RollbackBlackboard->BeginRollback(NewBaseTimeStep);

	ModeFSM->OnSimulationRollback(&SyncState, &AuxState, NewBaseTimeStep);

	RollbackBlackboard->EndRollback();

	LastSimTickTimeStep = NewBaseTimeStep;
	LastSyncState = SyncState;
	LastAuxState = AuxState;

	bJustStartedResim = true;

	OnSimulationRolledBack(NewBaseTimeStep, SyncState, AuxState);
}


void UKinematicActorSimulation::QueueNextMode(FModeChangeParams ModeChangeParams)
{
	ModeFSM->QueueNextMode(ModeChangeParams.DesiredModeName, ModeChangeParams.bShouldReenter);
}

FName UKinematicActorSimulation::GetNextModeName() const
{
	const FName QueuedModeName = ModeFSM->GetQueuedModeName();

	if (QueuedModeName != NAME_None)
	{
		return QueuedModeName;
	}

	return ModeFSM->GetCurrentModeName();
}

bool UKinematicActorSimulation::HasFeaturesWithTag(FGameplayTag TagToMatch, bool bRequireExactMatch) const
{
	return ModeFSM->HasFeaturesWithTag(TagToMatch, bRequireExactMatch);
}

void UKinematicActorSimulation::CancelFeaturesWithTag(FGameplayTag TagToCancel, bool bRequireExactMatch)
{
	ModeFSM->CancelFeaturesWithTag(TagToCancel, bRequireExactMatch);
}

void UKinematicActorSimulation::RegisterMovementMode(FModeRegistrationParams ModeRegisterParams)
{
	ModeFSM->RegisterMovementMode(ModeRegisterParams.ModeName, ModeRegisterParams.ModeObject, ModeRegisterParams.bIsTheDefaultMode);
}

void UKinematicActorSimulation::UnregisterMovementMode(FModeRegistrationParams ModeRegisterParams)
{
	FName ModeNameToRemove = ModeRegisterParams.ModeName;

	if (ModeFSM->GetCurrentModeName() == ModeNameToRemove)
	{
		UE_LOGF(LogMover, Warning, "The mode being removed (%ls Movement Mode) is the mode this actor (%ls) is currently in. It was removed but may cause issues. Consider waiting to remove the mode or queueing a different valid mode to avoid issues.", *ModeNameToRemove.ToString(), *GetContextNameSafe());
	}

	ModeFSM->UnregisterMovementMode(ModeNameToRemove);
}

const FMovementModifierBase* UKinematicActorSimulation::FindQueuedModifier(FMovementModifierHandle ModifierHandle) const
{
	return ModeFSM->FindQueuedModifier(ModifierHandle);
}

const FMovementModifierBase* UKinematicActorSimulation::FindQueuedModifierByType(const UScriptStruct* ModifierType) const
{
	return ModeFSM->FindQueuedModifierByType(ModifierType);
}

FMovementModifierHandle UKinematicActorSimulation::QueueMovementModifier(TSharedPtr<FMovementModifierBase> Modifier)
{
	return ModeFSM->QueueMovementModifier(Modifier);
}

void UKinematicActorSimulation::CancelModifierFromHandle(FMovementModifierHandle ModifierHandle)
{
	return ModeFSM->CancelModifierFromHandle(ModifierHandle);
}

void UKinematicActorSimulation::RegisterMovementModes(const TMap<FName, TWeakObjectPtr<UBaseMovementMode>>& ModeMap, FName StartingModeName)
{

	ModeFSM->ClearAllMovementModes();

	FModeRegistrationParams Params;

	for (const TPair<FName, TWeakObjectPtr<UBaseMovementMode>>& Element : ModeMap)
	{
		if (Element.Value.Get() == nullptr)
		{
			UE_LOGF(LogMover, Warning, "Invalid Movement Mode type '%ls' detected on %ls. Mover actor will not function correctly.",
				*Element.Key.ToString(), *GetContextNameSafe());
			continue;
		}

		Params.ModeName = Element.Key;
		Params.ModeObject = Element.Value.Get();
		Params.bIsTheDefaultMode = (StartingModeName == Element.Key);

		RegisterMovementMode(Params);
	}
}

void UKinematicActorSimulation::RegisterGlobalTransitions(const TArray<TWeakObjectPtr<UBaseMovementModeTransition>>& Transitions)
{
	ModeFSM->ClearAllGlobalTransitions();

	for (const TWeakObjectPtr<UBaseMovementModeTransition>& Transition : Transitions)
	{
		ModeFSM->RegisterGlobalTransition(Transition.Get());
	}
}

void UKinematicActorSimulation::CheckForExternalChanges(const FMoverTickStartData& SimStartingData)
{
	if (!MoverComp || (!MoverComp->bWarnOnExternalMovement && !MoverComp->bAcceptExternalMovement))
	{
		return;
	}

	const USceneComponent* UpdatedComponent = MoverComp->GetUpdatedComponent();
	if (!UpdatedComponent)
	{
		return;
	}

	if (const FMoverDefaultSyncState* StartingSyncState = SimStartingData.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>())
	{
		if (StartingSyncState->GetMovementBase())
		{
			return;	// TODO: need alternative handling of movement checks when based on another object
		}

		const FTransform& ComponentTransform = UpdatedComponent->GetComponentTransform();

		if (!ComponentTransform.GetLocation().Equals(StartingSyncState->GetLocation_WorldSpace()))
		{
			if (MoverComp->bWarnOnExternalMovement)
			{
				UE_LOGF(LogMover, Warning, "%ls: Simulation start location (%ls) disagrees with actual component location (%ls). This indicates movement of the component out-of-band with the simulation, and may cause poor quality motion.",
					*GetContextNameSafe(),
					*StartingSyncState->GetLocation_WorldSpace().ToCompactString(),
					*UpdatedComponent->GetComponentLocation().ToCompactString());
			}

			if (MoverComp->bAcceptExternalMovement)
			{
				FMoverDefaultSyncState* MutableSyncState = const_cast<FMoverDefaultSyncState*>(StartingSyncState);

				MutableSyncState->SetTransforms_WorldSpace(ComponentTransform.GetLocation(),
					ComponentTransform.GetRotation().Rotator(),
					MutableSyncState->GetVelocity_WorldSpace(),
					MutableSyncState->GetAngularVelocityDegrees_WorldSpace());
			}
		}
	}
}

void UKinematicActorSimulation::PrepareStateDataForNewFrame(const FMoverTimeStep& TimeStep, const FMoverTickStartData& StartData, FMoverTickEndData& EndData)
{
	EndData.AuxState = StartData.AuxState;

	if (MoverComp)
	{
		// Some sync state data should carry over between frames
		for (const FMoverDataPersistence& PersistentSyncEntry : MoverComp->PersistentSyncStateDataTypes)
		{
			bool bShouldAddDefaultData = true;

			if (PersistentSyncEntry.bCopyFromPriorFrame)
			{
				if (const FMoverDataStructBase* PriorFrameData = StartData.SyncState.SyncStateCollection.FindDataByType(PersistentSyncEntry.RequiredType))
				{
					EndData.SyncState.SyncStateCollection.AddDataByCopy(PriorFrameData);
					bShouldAddDefaultData = false;
				}
			}

			if (bShouldAddDefaultData)
			{
				EndData.SyncState.SyncStateCollection.FindOrAddDataByType(PersistentSyncEntry.RequiredType);
			}
		}

		// Make sure any other sync state structs that aren't supposed to be persistent are removed
		const TArray<TSharedPtr<FMoverDataStructBase>>& AllSyncStructs = EndData.SyncState.SyncStateCollection.GetDataArray();
		for (int32 i = AllSyncStructs.Num() - 1; i >= 0; --i)
		{
			bool bShouldRemoveStructType = true;

			const UScriptStruct* ScriptStruct = AllSyncStructs[i]->GetScriptStruct();

			for (const FMoverDataPersistence& PersistentSyncEntry : MoverComp->PersistentSyncStateDataTypes)
			{
				if (PersistentSyncEntry.RequiredType == ScriptStruct)
				{
					bShouldRemoveStructType = false;
					break;
				}
			}

			if (bShouldRemoveStructType)
			{
				EndData.SyncState.SyncStateCollection.RemoveDataByType(ScriptStruct);
			}
		}
	}

}



//
//void UKinematicActorSimulation::OnPreSimulationTick(const FMoverTimeStep& TimeStep, const UE::ChaosMover::FSimulationInputData& InputData)
//{
//	// TODO:  Implement me. What parameters should we have?
//}
//
//void UKinematicActorSimulation::OnPostSimulationTick(const FMoverTimeStep& TimeStep, UE::ChaosMover::FSimulationOutputData& OutputData)
//{
//	// TODO:  Implement me. What parameters should we have?
//}


void UKinematicActorSimulation::OnPreSimulationAdvance(const FMoverTimeStep& TimeStep, const FMoverTickStartData& StartData, FMoverTickEndData& EndData)
{
	if (FCharacterDefaultInputs* Input = StartData.InputCmd.InputCollection.FindMutableDataByType<FCharacterDefaultInputs>())
	{
		if (Input && !Input->SuggestedMovementMode.IsNone())
		{
			ModeFSM->QueueNextMode(Input->SuggestedMovementMode);
		}
	}
}

void UKinematicActorSimulation::AdvanceSimulation(const FMoverTimeStep& TimeStep, const FMoverTickStartData& StartData, FMoverTickEndData& EndData)
{
	USceneComponent* UpdatedComp = MoverComp ? MoverComp->GetUpdatedComponent() : nullptr;
	UPrimitiveComponent* UpdatedCompAsPrimitive = Cast<UPrimitiveComponent>(UpdatedComp);

	FMoverSimContext SimContext;
	SimContext.Simulation = this;
	SimContext.Blackboard = GetRollbackBlackboardSimWrapper();
	SimContext.SimulationOwner = MoverComp;

	RollbackBlackboard->BeginSimulationFrame(TimeStep);

	// Tick the actual simulation. This is where the proposed moves are queried and executed, affecting change to the moving actor's gameplay state and captured in the output sim state
	if (IsInGameThread())
	{
		// If we're on the game thread, we can make use of a scoped movement update for better perf of multi-step movements.  If not, then we're definitely not moving the component in immediate mode so the scope would have no effect.
		FScopedMovementUpdate ScopedMovementUpdate(UpdatedComp, EScopedUpdate::DeferredUpdates);
		ModeFSM->OnSimulationTick(UpdatedComp, UpdatedCompAsPrimitive, Blackboard, SimContext, StartData, TimeStep, OUT EndData);
	}
	else
	{
		ModeFSM->OnSimulationTick(UpdatedComp, UpdatedCompAsPrimitive, Blackboard, SimContext, StartData, TimeStep, OUT EndData);
	}

	const FName MovementModeAfterTick = ModeFSM->GetCurrentModeName();
	EndData.SyncState.MovementMode = MovementModeAfterTick;


	RollbackBlackboard->EndSimulationFrame();
}

void UKinematicActorSimulation::OnPostSimulationAdvance(const FMoverTimeStep& TimeStep, const FMoverTickStartData& StartData, FMoverTickEndData& EndData)
{


}


void UKinematicActorSimulation::OnSimulationRolledBack(const FMoverTimeStep& NewBaseTimeStep, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState)
{
}



FString UKinematicActorSimulation::GetContextNameSafe() const
{
	if (MoverComp)
	{
		return FString::Printf(TEXT("%s (%s)"), *GetNameSafe(MoverComp->GetOwner()), *StaticEnum<ENetRole>()->GetValueAsString(MoverComp->GetOwnerRole()));
	}

	return TEXT("None");
}