// Copyright Epic Games, Inc. All Rights Reserved.

#include "LayeredMove.h"
#include "MoverLog.h"
#include "MoverComponent.h"
#include "MoverSimulationTypes.h"
#include "MoverModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LayeredMove)

const double LayeredMove_InvalidTime = -UE_BIG_NUMBER;

void FLayeredMoveFinishVelocitySettings::NetSerialize(FArchive& Ar)
{
	uint8 bHasFinishVelocitySettings = Ar.IsSaving() ? 0 : (FinishVelocityMode == ELayeredMoveFinishVelocityMode::MaintainLastRootMotionVelocity);
	Ar.SerializeBits(&bHasFinishVelocitySettings, 1);

	if (bHasFinishVelocitySettings)
	{
		uint8 FinishVelocityModeAsU8 = (uint8)(FinishVelocityMode);
		Ar << FinishVelocityModeAsU8;
		FinishVelocityMode = (ELayeredMoveFinishVelocityMode)FinishVelocityModeAsU8;

		if (FinishVelocityMode == ELayeredMoveFinishVelocityMode::SetVelocity)
		{
			Ar << SetVelocity;
		}
		else if (FinishVelocityMode == ELayeredMoveFinishVelocityMode::ClampVelocity)
		{
			Ar << ClampVelocity;
		}
	}
}

FLayeredMoveBase::FLayeredMoveBase() :
	MixMode(EMoveMixMode::AdditiveVelocity),
	Priority(0),
	DurationMs(-1.f),
	StartSimTimeMs(LayeredMove_InvalidTime)
{
}


void FLayeredMoveBase::StartMove(const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, double CurrentSimTimeMs)
{
	StartSimTimeMs = CurrentSimTimeMs;
	OnStart(MoverComp, SimBlackboard);
}

void FLayeredMoveBase::StartMove_Async(UMoverBlackboard* SimBlackboard, const FMoverTime& SimTime)
{
	StartSimTimeMs = SimTime.TimeMs;
	OnStart_Async(SimBlackboard, SimTime);
}

bool FLayeredMoveBase::GenerateMove_Async(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove)
{
	ensureMsgf(false, TEXT("GenerateMove_Async is not implemented"));
	return false;
}

bool FLayeredMoveBase::IsFinished(double CurrentSimTimeMs) const
{
	const bool bHasStarted = (StartSimTimeMs >= 0.0);
	const bool bTimeExpired = bHasStarted && (DurationMs > 0.f) && (StartSimTimeMs + DurationMs <= CurrentSimTimeMs);
	const bool bDidTickOnceAndExpire = bHasStarted && (DurationMs == 0.f);

	return bTimeExpired || bDidTickOnceAndExpire;
}

void FLayeredMoveBase::EndMove(const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, double CurrentSimTimeMs)
{
	OnEnd(MoverComp, SimBlackboard, CurrentSimTimeMs);
}

void FLayeredMoveBase::EndMove_Async(UMoverBlackboard* SimBlackboard, double CurrentSimTimeMs)
{
	OnEnd_Async(SimBlackboard, CurrentSimTimeMs);
}

FLayeredMoveBase* FLayeredMoveBase::Clone() const
{
	// If child classes don't override this, saved moves will not work
	checkf(false, TEXT("FLayeredMoveBase::Clone() being called erroneously from %s. A FLayeredMoveBase should never be queued directly and Clone should always be overridden in child structs!"), *GetNameSafe(GetScriptStruct()));
	return nullptr;
}


void FLayeredMoveBase::NetSerialize(FArchive& Ar)
{
	uint8 MixModeAsU8 = (uint8)MixMode;
	Ar << MixModeAsU8;
	MixMode = (EMoveMixMode)MixModeAsU8;

	uint8 bHasDefaultPriority = Priority == 0;
	Ar.SerializeBits(&bHasDefaultPriority, 1);
	if (!bHasDefaultPriority)
	{
		Ar << Priority;
	}
	
	Ar << DurationMs;
	Ar << StartSimTimeMs;

	FinishVelocitySettings.NetSerialize(Ar);
}


UScriptStruct* FLayeredMoveBase::GetScriptStruct() const
{
	return FLayeredMoveBase::StaticStruct();
}


FString FLayeredMoveBase::ToSimpleString() const
{
	return GetScriptStruct()->GetName();
}


FLayeredMoveGroup::FLayeredMoveGroup()
	: ResidualVelocity(FVector::Zero())
	, ResidualClamping(-1.f)
	, bApplyResidualVelocity(false)
{
}


void FLayeredMoveGroup::QueueLayeredMove(TSharedPtr<FLayeredMoveBase> Move)
{
	if (ensure(Move.IsValid()))
	{
		QueuedLayeredMoves.Add(Move);
		UE_LOGF(LogMover, VeryVerbose, "LayeredMove queued move (%ls)", *Move->ToSimpleString());
	}
}

void FLayeredMoveGroup::CancelMovesByTag(FGameplayTag Tag, bool bRequireExactMatch)
{
	// Queued moves can be removed now before they become active
	QueuedLayeredMoves.RemoveAll([Tag, bRequireExactMatch](const TSharedPtr<FLayeredMoveBase>& Move)
		{
			return (Move.IsValid() && Move->HasGameplayTag(Tag, bRequireExactMatch));
		});

	// Schedule a tag cancellation request, to catch any active moves during simulation
	TagCancellationRequests.Add(TPair<FGameplayTag, bool>(Tag, bRequireExactMatch));
}

TArray<TSharedPtr<FLayeredMoveBase>> FLayeredMoveGroup::GenerateActiveMoves(const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard)
{
	FlushMoveArrays(MoverComp, SimBlackboard, TimeStep.ToStartTime(), /* bIsAsync = */ false);
	return ActiveLayeredMoves;
}

TArray<TSharedPtr<FLayeredMoveBase>> FLayeredMoveGroup::GenerateActiveMoves_Async(const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard)
{
	FlushMoveArrays(nullptr, SimBlackboard, TimeStep.ToStartTime(), /* bIsAsync = */ true);
	return ActiveLayeredMoves;
}

void FLayeredMoveGroup::NetSerialize(FArchive& Ar, uint8 MaxNumMovesToSerialize/* = MAX_uint8*/)
{
	// TODO: Warn if some sources will be dropped
	const uint8 NumActiveMovesToSerialize = FMath::Min<int32>(ActiveLayeredMoves.Num(), MaxNumMovesToSerialize);
	const uint8 NumQueuedMovesToSerialize = NumActiveMovesToSerialize < MaxNumMovesToSerialize ? MaxNumMovesToSerialize - NumActiveMovesToSerialize : 0;
	NetSerializeLayeredMovesArray(Ar, ActiveLayeredMoves, NumActiveMovesToSerialize);
	NetSerializeLayeredMovesArray(Ar, QueuedLayeredMoves, NumQueuedMovesToSerialize);
}


static void CopyLayeredMoveArray(TArray<TSharedPtr<FLayeredMoveBase>>& Dest, const TArray<TSharedPtr<FLayeredMoveBase>>& Src)
{
	bool bCanCopyInPlace = (UE::Mover::DisableDataCopyInPlace == 0 && Dest.Num() == Src.Num());
	if (bCanCopyInPlace)
	{
		// If copy in place is enabled and the arrays are the same size, copy by index
		for (int32 i = 0; i < Dest.Num(); ++i)
		{
			if (FLayeredMoveBase* SrcData = Src[i].Get())
			{
				FLayeredMoveBase* DestData = Dest[i].Get();
				UScriptStruct* SourceStruct = SrcData->GetScriptStruct();

				if (DestData && SourceStruct == DestData->GetScriptStruct())
				{
					// Same type so copy in place
					SourceStruct->CopyScriptStruct(DestData, SrcData, 1);
				}
				else
				{
					// Different type so replace the shared ptr with a clone
					Dest[i] = TSharedPtr<FLayeredMoveBase>(SrcData->Clone());
				}
			}
			else
			{
				// Found invalid source, fall back to full copy
				bCanCopyInPlace = false;
				break;
			}
		}
	}
	
	if (!bCanCopyInPlace)
	{
		// Deep copy active moves
		Dest.Empty(Src.Num());
		for (int i = 0; i < Src.Num(); ++i)
		{
			if (Src[i].IsValid())
			{
				FLayeredMoveBase* CopyOfSourcePtr = Src[i]->Clone();
				Dest.Add(TSharedPtr<FLayeredMoveBase>(CopyOfSourcePtr));
			}
			else
			{
				UE_LOGF(LogMover, Warning, "CopyLayeredMoveArray trying to copy invalid Other Layered Move");
			}
		}
	}
}


FLayeredMoveGroup& FLayeredMoveGroup::operator=(const FLayeredMoveGroup& Other)
{
	// Perform deep copy of this Group
	if (this != &Other)
	{
		CopyLayeredMoveArray(ActiveLayeredMoves, Other.ActiveLayeredMoves);
		CopyLayeredMoveArray(QueuedLayeredMoves, Other.QueuedLayeredMoves);

		TagCancellationRequests = Other.TagCancellationRequests;
	}

	return *this;
}

bool FLayeredMoveGroup::operator==(const FLayeredMoveGroup& Other) const
{
	// Deep move-by-move comparison
	if (ActiveLayeredMoves.Num() != Other.ActiveLayeredMoves.Num())
	{
		return false;
	}
	if (QueuedLayeredMoves.Num() != Other.QueuedLayeredMoves.Num())
	{
		return false;
	}


	for (int32 i = 0; i < ActiveLayeredMoves.Num(); ++i)
	{
		if (ActiveLayeredMoves[i].IsValid() == Other.ActiveLayeredMoves[i].IsValid())
		{
			if (ActiveLayeredMoves[i].IsValid())
			{
				// TODO: Implement deep equality checks
				// 				if (!ActiveLayeredMoves[i]->MatchesAndHasSameState(Other.ActiveLayeredMoves[i].Get()))
				// 				{
				// 					return false; // They're valid and don't match/have same state
				// 				}
			}
		}
		else
		{
			return false; // Mismatch in validity
		}
	}
	for (int32 i = 0; i < QueuedLayeredMoves.Num(); ++i)
	{
		if (QueuedLayeredMoves[i].IsValid() == Other.QueuedLayeredMoves[i].IsValid())
		{
			if (QueuedLayeredMoves[i].IsValid())
			{
				// TODO: Implement deep equality checks
				// 				if (!QueuedLayeredMoves[i]->MatchesAndHasSameState(Other.QueuedLayeredMoves[i].Get()))
				// 				{
				// 					return false; // They're valid and don't match/have same state
				// 				}
			}
		}
		else
		{
			return false; // Mismatch in validity
		}
	}
	return true;
}

bool FLayeredMoveGroup::operator!=(const FLayeredMoveGroup& Other) const
{
	return !(FLayeredMoveGroup::operator==(Other));
}

bool FLayeredMoveGroup::HasSameContents(const FLayeredMoveGroup& Other) const
{
	// Only compare the types of moves contained, not the state
	if (ActiveLayeredMoves.Num() != Other.ActiveLayeredMoves.Num() ||
		QueuedLayeredMoves.Num() != Other.QueuedLayeredMoves.Num())
	{
		return false;
	}

	for (int32 i = 0; i < ActiveLayeredMoves.Num(); ++i)
	{
		if (ActiveLayeredMoves[i]->GetScriptStruct() != Other.ActiveLayeredMoves[i]->GetScriptStruct())
		{
			return false;
		}
	}

	for (int32 i = 0; i < QueuedLayeredMoves.Num(); ++i)
	{
		if (QueuedLayeredMoves[i]->GetScriptStruct() != Other.QueuedLayeredMoves[i]->GetScriptStruct())
		{
			return false;
		}
	}

	return true;
}


void FLayeredMoveGroup::AddStructReferencedObjects(FReferenceCollector& Collector) const
{
	for (const TSharedPtr<FLayeredMoveBase>& LayeredMove : ActiveLayeredMoves)
	{
		if (LayeredMove.IsValid())
		{
			LayeredMove->AddReferencedObjects(Collector);
		}
	}

	for (const TSharedPtr<FLayeredMoveBase>& LayeredMove : QueuedLayeredMoves)
	{
		if (LayeredMove.IsValid())
		{
			LayeredMove->AddReferencedObjects(Collector);
		}
	}
}

FString FLayeredMoveGroup::ToSimpleString() const
{
	return FString::Printf(TEXT("FLayeredMoveGroup. Active: %i Queued: %i"), ActiveLayeredMoves.Num(), QueuedLayeredMoves.Num());
}

const FLayeredMoveBase* FLayeredMoveGroup::FindActiveMove(const UScriptStruct* LayeredMoveStructType) const
{
	for (const TSharedPtr<FLayeredMoveBase>& ActiveMove : ActiveLayeredMoves)
	{
		if (ActiveMove && ActiveMove->GetScriptStruct()->IsChildOf(LayeredMoveStructType))
		{
			return ActiveMove.Get();
		}
	}
	return nullptr;
}

const FLayeredMoveBase* FLayeredMoveGroup::FindQueuedMove(const UScriptStruct* LayeredMoveStructType) const
{
	for (const TSharedPtr<FLayeredMoveBase>& QueuedMove : QueuedLayeredMoves)
	{
		if (QueuedMove && QueuedMove->GetScriptStruct()->IsChildOf(LayeredMoveStructType))
		{
			return QueuedMove.Get();
		}
	}
	return nullptr;
}

void FLayeredMoveGroup::FlushMoveArrays(const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, const FMoverTime& SimTime, bool bIsAsync)
{
	if (bIsAsync)
	{
		ensureMsgf(MoverComp == nullptr, TEXT("In async mode, no mover Component should be passed in as argument in FLayeredMoveGroup::FlushMoveArrays"));
		MoverComp = nullptr;
	}

	const double CurrentSimTimeMs = SimTime.TimeMs;

	bool bResidualVelocityOverridden = false;
	bool bClampVelocityOverridden = false;
	
	// Process any cancellations
	{
		for (TPair<FGameplayTag, bool> CancelRequest : TagCancellationRequests)
		{
			const FGameplayTag TagToMatch = CancelRequest.Key;
			const bool bRequireExactMatch = CancelRequest.Value;

			// Only removing active ones because queued ones were cancelled at the time the request was added
			ActiveLayeredMoves.RemoveAll([MoverComp, SimBlackboard, CurrentSimTimeMs, &bResidualVelocityOverridden, &bClampVelocityOverridden, bIsAsync, TagToMatch, bRequireExactMatch, this] (const TSharedPtr<FLayeredMoveBase>& Move)
				{
					if (Move.IsValid() && Move->HasGameplayTag(TagToMatch, bRequireExactMatch))
					{
						GatherResidualVelocitySettings(Move, bResidualVelocityOverridden, bClampVelocityOverridden);
						if (bIsAsync)
						{
							Move->EndMove_Async(SimBlackboard, CurrentSimTimeMs);
						}
						else
						{
							Move->EndMove(MoverComp, SimBlackboard, CurrentSimTimeMs);
						}
						return true;
					}

					return false;
				});
		}

		TagCancellationRequests.Empty();
	}

	
	// Remove any finished moves
	ActiveLayeredMoves.RemoveAll([MoverComp, SimBlackboard, CurrentSimTimeMs, &bResidualVelocityOverridden, &bClampVelocityOverridden, bIsAsync, this]
		(const TSharedPtr<FLayeredMoveBase>& Move)
		{
			if (Move.IsValid())
			{
				if (Move->IsFinished(CurrentSimTimeMs))
				{
					GatherResidualVelocitySettings(Move, bResidualVelocityOverridden, bClampVelocityOverridden);
					if (bIsAsync)
					{
						Move->EndMove_Async(SimBlackboard, CurrentSimTimeMs);
					}
					else
					{
						Move->EndMove(MoverComp, SimBlackboard, CurrentSimTimeMs);
					}
					return true;
				}
			}
			else
			{
				return true;	
			}

			return false;
		});

	// Make any queued moves active
	for (TSharedPtr<FLayeredMoveBase>& QueuedMove : QueuedLayeredMoves)
	{
		ActiveLayeredMoves.Add(QueuedMove);
		if (bIsAsync)
		{
			QueuedMove->StartMove_Async(SimBlackboard, SimTime);
		}
		else
		{
			QueuedMove->StartMove(MoverComp, SimBlackboard, CurrentSimTimeMs);
		}
	}

	QueuedLayeredMoves.Empty();
}

void FLayeredMoveGroup::GatherResidualVelocitySettings(const TSharedPtr<FLayeredMoveBase>& Move, bool& bResidualVelocityOverridden, bool& bClampVelocityOverridden)
{
	if (Move->FinishVelocitySettings.FinishVelocityMode == ELayeredMoveFinishVelocityMode::SetVelocity)
	{
		if (Move->MixMode == EMoveMixMode::OverrideVelocity)
		{
			if (bResidualVelocityOverridden)
			{
				UE_LOGF(LogMover, Log, "Multiple LayeredMove residual settings have a MixMode that overrides. Only one will take effect.");
			}

			bResidualVelocityOverridden = true;
			ResidualVelocity = Move->FinishVelocitySettings.SetVelocity;
		}
		else if (Move->MixMode == EMoveMixMode::AdditiveVelocity && !bResidualVelocityOverridden)
		{
			ResidualVelocity += Move->FinishVelocitySettings.SetVelocity;
		}
		else if (Move->MixMode == EMoveMixMode::OverrideAll)
		{
			if (bResidualVelocityOverridden)
			{
				UE_LOGF(LogMover, Log, "Multiple LayeredMove residual settings have a MixMode that overrides. Only one will take effect.");
			}

			bResidualVelocityOverridden = true;
			ResidualVelocity = Move->FinishVelocitySettings.SetVelocity;
		}
		else
		{
			check(0);	// unhandled case
		}
		bApplyResidualVelocity = true;
	}
	else if (Move->FinishVelocitySettings.FinishVelocityMode == ELayeredMoveFinishVelocityMode::ClampVelocity)
	{
		if (Move->MixMode == EMoveMixMode::OverrideVelocity)
		{
			if (bClampVelocityOverridden)
			{
				UE_LOGF(LogMover, Log, "Multiple LayeredMove residual settings have a MixMode that overrides. Only one will take effect.");
			}

			bClampVelocityOverridden = true;
			ResidualClamping = Move->FinishVelocitySettings.ClampVelocity;
		}
		else if (Move->MixMode == EMoveMixMode::AdditiveVelocity && !bClampVelocityOverridden)
		{
			if (ResidualClamping < 0)
			{
				ResidualClamping = Move->FinishVelocitySettings.ClampVelocity;
			}
			// No way to really add clamping so we instead apply it if it was smaller
			else if (ResidualClamping > Move->FinishVelocitySettings.ClampVelocity)
			{
				ResidualClamping = Move->FinishVelocitySettings.ClampVelocity;
			}
		}
		else if (Move->MixMode == EMoveMixMode::OverrideAll)
		{
			if (bClampVelocityOverridden)
			{
				UE_LOGF(LogMover, Log, "Multiple LayeredMove residual settings have a MixMode that overrides. Only one will take effect.");
			}

			bClampVelocityOverridden = true;
			ResidualClamping = Move->FinishVelocitySettings.ClampVelocity;
		}
		else
		{
			check(0);	// unhandled case
		}
	}
}

struct FLayeredMoveDeleter
{
	FORCEINLINE void operator()(FLayeredMoveBase* Object) const
	{
		check(Object);
		UScriptStruct* ScriptStruct = Object->GetScriptStruct();
		check(ScriptStruct);
		ScriptStruct->DestroyStruct(Object);
		FMemory::Free(Object);
	}
};


/* static */ void FLayeredMoveGroup::NetSerializeLayeredMovesArray(FArchive& Ar, TArray< TSharedPtr<FLayeredMoveBase> >& LayeredMovesArray, uint8 MaxNumLayeredMovesToSerialize /*=MAX_uint8*/)
{
	uint8 NumMovesToSerialize;
	if (Ar.IsSaving())
	{
		UE_CLOGF(LayeredMovesArray.Num() > MaxNumLayeredMovesToSerialize, LogMover, Warning, "Too many Layered Moves (%d!) to net serialize. Clamping to %d",
			LayeredMovesArray.Num(), MaxNumLayeredMovesToSerialize);

		NumMovesToSerialize = FMath::Min<int32>(LayeredMovesArray.Num(), MaxNumLayeredMovesToSerialize);
	}

	Ar << NumMovesToSerialize;

	if (Ar.IsLoading())
	{
		LayeredMovesArray.SetNumZeroed(NumMovesToSerialize);
	}

	for (int32 i = 0; i < NumMovesToSerialize && !Ar.IsError(); ++i)
	{
		TCheckedObjPtr<UScriptStruct> ScriptStruct = LayeredMovesArray[i].IsValid() ? LayeredMovesArray[i]->GetScriptStruct() : nullptr;
		UScriptStruct* ScriptStructLocal = ScriptStruct.Get();
		Ar << ScriptStruct;

		if (ScriptStruct.IsValid())
		{
			// Restrict replication to derived classes of FLayeredMoveBase for security reasons:
			// If FLayeredMoveGroup is replicated through a Server RPC, we need to prevent clients from sending us
			// arbitrary ScriptStructs due to the allocation/reliance on GetCppStructOps below which could trigger a server crash
			// for invalid structs. All provided sources are direct children of FLayeredMoveBase and we never expect to have deep hierarchies
			// so this should not be too costly
			bool bIsDerivedFromBase = false;
			UStruct* CurrentSuperStruct = ScriptStruct->GetSuperStruct();
			while (CurrentSuperStruct)
			{
				if (CurrentSuperStruct == FLayeredMoveBase::StaticStruct())
				{
					bIsDerivedFromBase = true;
					break;
				}
				CurrentSuperStruct = CurrentSuperStruct->GetSuperStruct();
			}

			if (bIsDerivedFromBase)
			{
				if (Ar.IsLoading())
				{
					if (LayeredMovesArray[i].IsValid() && ScriptStructLocal == ScriptStruct.Get())
					{
						// What we have locally is the same type as we're being serialized into, so we don't need to
						// reallocate - just use existing structure
					}
					else
					{
						// For now, just reset/reallocate the data when loading.
						// Longer term if we want to generalize this and use it for property replication, we should support
						// only reallocating when necessary
						FLayeredMoveBase* NewMove = (FLayeredMoveBase*)FMemory::Malloc(ScriptStruct->GetCppStructOps()->GetSize());
						ScriptStruct->InitializeStruct(NewMove);

						LayeredMovesArray[i] = TSharedPtr<FLayeredMoveBase>(NewMove, FLayeredMoveDeleter());
					}
				}

				LayeredMovesArray[i]->NetSerialize(Ar);
			}
			else
			{
				UE_LOGF(LogMover, Error, "FLayeredMoveGroup::NetSerialize: ScriptStruct not derived from FLayeredMoveBase attempted to serialize.");
				Ar.SetError();
				break;
			}
		}
		else if (ScriptStruct.IsError())
		{
			UE_LOGF(LogMover, Error, "FLayeredMoveGroup::NetSerialize: Invalid ScriptStruct serialized.");
			Ar.SetError();
			break;
		}
	}
}

void FLayeredMoveGroup::ResetResidualVelocity()
{
	bApplyResidualVelocity = false;
	ResidualVelocity = FVector::ZeroVector;
	ResidualClamping = -1.f;
}

void FLayeredMoveGroup::Reset()
{
	ResetResidualVelocity();
	QueuedLayeredMoves.Empty();
	ActiveLayeredMoves.Empty();
	TagCancellationRequests.Empty();
}

#if WITH_CHAOS_VISUAL_DEBUGGER
void FLayeredMoveGroup::PrepareCVDMirror() const
{
	Debug_ActiveMoves.Reset(ActiveLayeredMoves.Num());
	for (const TSharedPtr<FLayeredMoveBase>& Move : ActiveLayeredMoves)
	{
		if (Move.IsValid())
		{
			Debug_ActiveMoves.Emplace_GetRef().InitializeAs(Move->GetScriptStruct(), reinterpret_cast<const uint8*>(Move.Get()));
		}
	}
}
#endif

