// Copyright Epic Games, Inc. All Rights Reserved.

#include "LayeredMoveBase.h"
#include "MoverLog.h"
#include "MoverComponent.h"
#include "MoverSimulationTypes.h"
#include "Blueprint/BlueprintExceptionInfo.h"
#include "MoveLibrary/MovementMixer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LayeredMoveBase)

#define LOCTEXT_NAMESPACE "Mover"

//////////////////////////////////////////////////////////////////////////
// FLayeredMoveInstancedData

bool FLayeredMoveInstancedData::operator==(const FLayeredMoveInstancedData& Other) const
{
	//@todo DanH: Compare logic class compatibility?
	if (GetScriptStruct() == Other.GetScriptStruct())
	{
		return Equals(Other);
	}
	return false;
}

bool FLayeredMoveInstancedData::Equals(const FLayeredMoveInstancedData& OtherData) const
{
	return StartSimTimeMs == OtherData.StartSimTimeMs
		&& DurationMs == OtherData.DurationMs;
}

void FLayeredMoveInstancedData::ActivateFromContext(const FLayeredMoveActivationParams* ActivationParams)
{
	
}

void FLayeredMoveInstancedData::NetSerialize(FArchive& Ar)
{
	Ar << StartSimTimeMs;
	Ar << DurationMs;
}

//////////////////////////////////////////////////////////////////////////
// ULayeredMoveLogic

ULayeredMoveLogic::ULayeredMoveLogic()
	: InstancedDataStructType(FLayeredMoveInstancedData::StaticStruct())
{
}

bool ULayeredMoveLogic::IsFinished_Implementation(const FMoverTimeStep& TimeStep, const UMoverBlackboard* SimBlackboard)
{
	const FLayeredMoveInstancedData& ExecData = AccessExecutionMoveData();
	const double DurationMs = ExecData.DurationMs;
	if (DurationMs < 0.f)
	{
		return false;
	}
	
	const double StartSimTimeMs = ExecData.GetStartSimTimeMs();
	
	const bool bHasStarted = StartSimTimeMs >= 0.f;
	const bool bTimeExpired = StartSimTimeMs + DurationMs <= TimeStep.BaseSimTimeMs;
	return bHasStarted && bTimeExpired;	
}

bool ULayeredMoveLogic::K2_GetActiveMoveData(ULayeredMoveLogic* MoveLogic, FLayeredMoveInstancedData& OutMoveData)
{
	// This will never be called, the exec version below will be hit instead
	checkNoEntry();
	return false;
}


DEFINE_FUNCTION(ULayeredMoveLogic::execK2_GetActiveMoveData)
{
	P_GET_OBJECT(ULayeredMoveLogic, MoveLogic);
	
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(nullptr);

	const FStructProperty* MoveDataProperty = CastField<FStructProperty>(Stack.MostRecentProperty);
	uint8* OutMoveDataPtr = Stack.MostRecentPropertyAddress;
	
	P_FINISH

	const bool bHasValidMoveData = ValidateMoveDataGetSet(P_THIS, MoveLogic, MoveDataProperty, OutMoveDataPtr, Stack);
	if (bHasValidMoveData)
	{
		// Write the active move data to the function output
		P_NATIVE_BEGIN;
		MoveDataProperty->Struct->CopyScriptStruct(OutMoveDataPtr, MoveLogic->CurrentInstancedData.Get());
		P_NATIVE_END;
	}
	
	*(bool*)RESULT_PARAM = bHasValidMoveData;
}

void ULayeredMoveLogic::K2_SetActiveMoveData(ULayeredMoveLogic* MoveLogic, const FLayeredMoveInstancedData& OutMoveData)
{
	// This will never be called, the exec version below will be hit instead
	checkNoEntry();
}

DEFINE_FUNCTION(ULayeredMoveLogic::execK2_SetActiveMoveData)
{
	P_GET_OBJECT(ULayeredMoveLogic, MoveLogic);
	
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(nullptr);

	const FStructProperty* MoveDataProperty = CastField<FStructProperty>(Stack.MostRecentProperty);
	const uint8* MoveDataPtr = Stack.MostRecentPropertyAddress;

	P_FINISH

	if (ValidateMoveDataGetSet(P_THIS, MoveLogic, MoveDataProperty, MoveDataPtr, Stack))
	{
		// Overwrite the contents of the current move data with that provided
		P_NATIVE_BEGIN;
		MoveDataProperty->Struct->CopyScriptStruct(MoveLogic->CurrentInstancedData.Get(), MoveDataPtr);
		P_NATIVE_END;
	}
}

FLayeredMoveInstance::FLayeredMoveInstance()
	: InstanceMoveData(nullptr)
	, MoveLogic(nullptr)
{
}

FLayeredMoveInstance::FLayeredMoveInstance(const FLayeredMoveInstance& OtherActiveLayeredMove)
	: InstanceMoveData(OtherActiveLayeredMove.InstanceMoveData)
	, MoveLogic(OtherActiveLayeredMove.MoveLogic)
{
}

FLayeredMoveInstance::FLayeredMoveInstance(const TSharedRef<FLayeredMoveInstancedData>& InMoveData, ULayeredMoveLogic* InMoveLogic)
	: InstanceMoveData(InMoveData)
	, MoveLogic(InMoveLogic)
{
}

bool ULayeredMoveLogic::ValidateMoveDataGetSet(const UObject* ObjectValidatingData, const ULayeredMoveLogic* MoveLogic, const FStructProperty* MoveDataProperty, const uint8* MoveDataPtr, FFrame& StackFrame)
{
	FText BlueprintExceptionText;
	bool bIsValid = true;
	if (!MoveLogic)
	{
		BlueprintExceptionText = LOCTEXT("ValidateMoveDataNoMoveLogic", "No MoveLogic was present.");
		bIsValid = false;
	}
	else if (!MoveLogic->CurrentInstancedData)
	{
		BlueprintExceptionText = LOCTEXT("ValidateMoveDataNoCurrentActiveMoveData", "No CurrentActiveMoveData on the MoveLogic was present.");
		bIsValid = false;
	}
	else if (!MoveDataProperty)
	{
		BlueprintExceptionText = LOCTEXT("ValidateMoveDataNoMoveData", "No MoveData was present.");
		bIsValid = false;
	}
	else if (!MoveDataPtr)
	{
		BlueprintExceptionText = LOCTEXT("ValidateMoveDataNoMoveData", "No MoveData was present.");
		bIsValid = false;
	}
	else if (MoveDataProperty->Struct != MoveLogic->InstancedDataStructType.Get())
	{
		BlueprintExceptionText = FText::Format(LOCTEXT("ValidateMoveDataMisMatchData", "MoveData passed in did not match MoveLogic Active mode data. Expected: {0}. Found: {1}."),
			FText::FromString(MoveDataProperty->Struct.GetName()),
			FText::FromString(MoveLogic->InstancedDataStructType.GetName())
		);
		bIsValid = false;
	}

	if (!bIsValid)
	{
		const FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::NonFatalError,
			BlueprintExceptionText
		);
		
		FBlueprintCoreDelegates::ThrowScriptException(ObjectValidatingData, StackFrame, ExceptionInfo);
	}
	
	return bIsValid;
}

//////////////////////////////////////////////////////////////////////////
// FLayeredMoveInstance

/**
 * Scoped wrapper that is the only means of calling the virtual functions in ULayeredMoveLogic
 * that depend on/expect access to valid active move data.
 */
class FScopedMoveLogicExecContext
{
public:	
	FScopedMoveLogicExecContext(ULayeredMoveLogic& MoveLogic, const TSharedRef<FLayeredMoveInstancedData>& MoveData)
		: LogicObj(MoveLogic)
	{
		MoveLogic.CurrentInstancedData = MoveData.ToSharedPtr();
	}

	template <typename... ArgsT>
    void OnStart(ArgsT&&... Args)
    {
    	LogicObj.OnStart(Args...);
    }
	
	template <typename... ArgsT>
	void OnEnd(ArgsT&&... Args)
	{
		LogicObj.OnEnd(Args...);
	}
	
	template <typename... ArgsT>
	bool GenerateMove(ArgsT&&... Args)
	{
		return LogicObj.GenerateMove(Args...);
	}

	template <typename... ArgsT>
	bool IsFinished(ArgsT&&... Args)
	{
		return LogicObj.IsFinished(Args...);
	}

	template <typename... ArgsT>
	void OnStart_Async(ArgsT&&... Args) { LogicObj.OnStart_Async(Args...); }

	template <typename... ArgsT>
	bool GenerateMove_Async(ArgsT&&... Args) { return LogicObj.GenerateMove_Async(Args...); }

	template <typename... ArgsT>
	void OnEnd_Async(ArgsT&&... Args) { LogicObj.OnEnd_Async(Args...); }

	~FScopedMoveLogicExecContext()
	{
		LogicObj.CurrentInstancedData.Reset();
	}

private:
	ULayeredMoveLogic& LogicObj;
};

void FLayeredMoveInstance::StartMove(const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard) const
{
	if (ensure(MoveLogic))
	{
		FScopedMoveLogicExecContext LogicExecContext(*MoveLogic, InstanceMoveData.ToSharedRef());
		LogicExecContext.OnStart(TimeStep, SimBlackboard);
		
		InstanceMoveData->StartSimTimeMs = TimeStep.BaseSimTimeMs;
	}
}

bool FLayeredMoveInstance::GenerateMove(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove) const
{
	if (ensure(MoveLogic))
	{
		FScopedMoveLogicExecContext LogicExecContext(*MoveLogic, InstanceMoveData.ToSharedRef());
		if (LogicExecContext.GenerateMove(TimeStep, SimBlackboard, StartState, OutProposedMove))
		{
			//@todo DanH: Feels like the preferred mode from moves should come from OnStart if that's the only time we listen to it
			// Wipe the preferred mode if this wasn't the first tick of the mode
			if (InstanceMoveData->GetStartSimTimeMs() < TimeStep.BaseSimTimeMs)
			{
				OutProposedMove.PreferredMode = NAME_None;
			}
			return true;
		}
	}
	
	return false;
}

void FLayeredMoveInstance::EndMove(const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard) const
{
	if (ensure(MoveLogic))
	{
		FScopedMoveLogicExecContext LogicExecContext(*MoveLogic, InstanceMoveData.ToSharedRef());
		LogicExecContext.OnEnd(TimeStep, SimBlackboard);
	}
}

bool FLayeredMoveInstance::IsFinished(const FMoverTimeStep& TimeStep, const UMoverBlackboard* SimBlackboard) const
{
	if (ensure(MoveLogic))
	{
		FScopedMoveLogicExecContext LogicExecContext(*MoveLogic, InstanceMoveData.ToSharedRef());
		return LogicExecContext.IsFinished(TimeStep, SimBlackboard);
	}
	return true;
}

void FLayeredMoveInstance::StartMove_Async(UMoverBlackboard* SimBlackboard, const FMoverTime& SimTime) const
{
	if (ensure(MoveLogic))
	{
		FScopedMoveLogicExecContext LogicExecContext(*MoveLogic, InstanceMoveData.ToSharedRef());
		LogicExecContext.OnStart_Async(SimBlackboard, SimTime);
		InstanceMoveData->StartSimTimeMs = SimTime.TimeMs;
	}
}

bool FLayeredMoveInstance::GenerateMove_Async(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove) const
{
	if (ensure(MoveLogic))
	{
		FScopedMoveLogicExecContext LogicExecContext(*MoveLogic, InstanceMoveData.ToSharedRef());
		if (LogicExecContext.GenerateMove_Async(TimeStep, SimBlackboard, StartState, OutProposedMove))
		{
			if (InstanceMoveData->GetStartSimTimeMs() < TimeStep.BaseSimTimeMs)
			{
				OutProposedMove.PreferredMode = NAME_None;
			}
			return true;
		}
	}
	return false;
}

void FLayeredMoveInstance::EndMove_Async(UMoverBlackboard* SimBlackboard, const FMoverTime& SimTime) const
{
	if (ensure(MoveLogic))
	{
		FScopedMoveLogicExecContext LogicExecContext(*MoveLogic, InstanceMoveData.ToSharedRef());
		LogicExecContext.OnEnd_Async(SimBlackboard, SimTime);
	}
}

const FLayeredMoveFinishVelocitySettings& FLayeredMoveInstance::GetFinishVelocitySettings() const
{
	if (MoveLogic)
	{
		return MoveLogic->GetFinishVelocitySettings();
	}
	
	static const FLayeredMoveFinishVelocitySettings Defaults;
	return Defaults;
}

EMoveMixMode FLayeredMoveInstance::GetMixMode() const
{
	if (MoveLogic)
	{
		return MoveLogic->GetMixMode();
	}
	return EMoveMixMode::AdditiveVelocity;
}

FLayeredMoveInstance FLayeredMoveInstance::Clone() const
{
	return FLayeredMoveInstance(TSharedRef<FLayeredMoveInstancedData>(InstanceMoveData->Clone()), MoveLogic.Get());
}

void FLayeredMoveInstance::NetSerialize(FArchive& Ar)
{
	// Step carefully! When writing from an archive, this move may have been created via TArray::AddZeroed, which does not call the constructor
	// Therefore, this is the one spot where we are actually ok with MoveData being null, but that violates the rules of TSharedRef.
	// So the ref is converted first to a TSharedPtr so we can safely, albeit cheekily, check it for validity.
	TSharedPtr<FLayeredMoveInstancedData> PossiblyNullData = InstanceMoveData;
	TCheckedObjPtr<UScriptStruct> DataStructType = PossiblyNullData ? PossiblyNullData->GetScriptStruct() : nullptr;
	UScriptStruct* ExistingDataType = DataStructType.Get();
	Ar << DataStructType;

	UClass* CurrentMoveLogicClassType = MoveLogic ? MoveLogic->GetClass() : nullptr;
	
	Ar << CurrentMoveLogicClassType;
	if (!MoveLogic)
	{
		MoveLogicClassType = CurrentMoveLogicClassType;
	}
	
	if (DataStructType.IsValid())
	{
		//@todo DanH: Fwiw FLayeredMoveInstanceGroup is not actually ever expected to be sent in a server RPC right?
		// Restrict replication to derived classes of FLayeredMoveInstancedData for security reasons:
		// If FLayeredMoveInstanceGroup is replicated through a Server RPC, we need to prevent clients from sending us
		// arbitrary ScriptStructs due to the allocation/reliance on GetCppStructOps below which could trigger a server crash
		// for invalid structs.
		if (DataStructType->IsChildOf(FLayeredMoveInstancedData::StaticStruct()))
		{
			// If the struct type at this index should be different from the one that's already here, we need to change it
			if (DataStructType.Get() != ExistingDataType && ensure(Ar.IsLoading()))
			{
				//@todo DanH: There was a comment here about doing it this way "for now", but it was copy-pasted from FRootMotionSourceGroup::NetSerializeRMSArray. Is it valid?
				FLayeredMoveInstancedData* NewMove = (FLayeredMoveInstancedData*)FMemory::Malloc(DataStructType->GetCppStructOps()->GetSize());
				DataStructType->InitializeStruct(NewMove);

				struct FSerializationAllocatedLayeredMoveDataDeleter
				{
					FORCEINLINE void operator()(FLayeredMoveInstancedData* MoveData) const
					{
						check(MoveData);
						UScriptStruct* ScriptStruct = MoveData->GetScriptStruct();
						check(ScriptStruct);
						ScriptStruct->DestroyStruct(MoveData);
						FMemory::Free(MoveData);
					}
				};
				InstanceMoveData = TSharedRef<FLayeredMoveInstancedData>(NewMove, FSerializationAllocatedLayeredMoveDataDeleter());
			}

			InstanceMoveData->NetSerialize(Ar);
		}
		else
		{
			UE_LOGF(LogMover, Error, "FLayeredMoveInstanceGroup::NetSerialize: ScriptStruct [%ls] not derived from FLayeredMoveInstancedData attempted to serialize.",
				*DataStructType->GetName());
			Ar.SetError();
		}
	}
	else if (DataStructType.IsError())
	{
		UE_LOGF(LogMover, Error, "FLayeredMoveInstanceGroup::NetSerialize: Invalid ScriptStruct serialized.");
		Ar.SetError();
	}
}

const UClass* FLayeredMoveInstance::GetSerializedMoveLogicClass() const
{
	return MoveLogicClassType;
}

bool FLayeredMoveInstance::PopulateMissingActiveMoveLogic(const TArray<TObjectPtr<ULayeredMoveLogic>>& RegisteredMoves)
{
	if (!HasLogic())
	{
		bool bFoundMissingLogic = false;
		
		if (MoveLogicClassType)
		{
			for (const TObjectPtr<ULayeredMoveLogic>& RegisteredMoveLogic : RegisteredMoves)
			{
				if (RegisteredMoveLogic->GetClass() == MoveLogicClassType)
				{
					MoveLogic = RegisteredMoveLogic;
					
					bFoundMissingLogic = true;
					break;
				}
			}

			if (!bFoundMissingLogic)
			{
				UE_LOGF(LogMover, Warning, "Active Layered Move couldn't find it's serialized MoveLogicClass (%ls) among registered MoveLogic", *MoveLogicClassType->GetName());
			}
		}
		else
		{
			UE_LOGF(LogMover, Warning, "Active Layered Move didn't have a valid logic class or class type to search for");
		}

		return bFoundMissingLogic;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
