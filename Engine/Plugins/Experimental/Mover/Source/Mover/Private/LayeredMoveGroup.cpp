// Copyright Epic Games, Inc. All Rights Reserved.

#include "LayeredMoveGroup.h"
#include "LayeredMoveBase.h"
#include "MoverLog.h"
#include "MoverTypes.h"
#include "MoveLibrary/MovementMixer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LayeredMoveGroup)

FLayeredMoveInstanceGroup::FLayeredMoveInstanceGroup()
	: ResidualClamping(-1.f)
	, bApplyResidualVelocity(false)
	, ResidualVelocity(FVector::Zero())
{
}

FLayeredMoveInstanceGroup& FLayeredMoveInstanceGroup::operator=(const FLayeredMoveInstanceGroup& Other)
{
	if (this != &Other)
	{
		const auto DeepCopyMovesFunc = [](const TArray<TSharedPtr<FLayeredMoveInstance>>& From, TArray<TSharedPtr<FLayeredMoveInstance>>& To)
		{
			To.Empty(From.Num());
			for (const TSharedPtr<FLayeredMoveInstance>& Move : From)
			{
				To.Add(Move);
			}
		};
		DeepCopyMovesFunc(Other.ActiveMoves, ActiveMoves);
		DeepCopyMovesFunc(Other.QueuedMoves, QueuedMoves);

		TagCancellationRequests = Other.TagCancellationRequests;
	}

	return *this;
}

bool FLayeredMoveInstanceGroup::operator==(const FLayeredMoveInstanceGroup& Other) const
{
	// Todo: Deep move-by-move comparison
	if (ActiveMoves.Num() != Other.ActiveMoves.Num())
	{
		return false;
	}
	if (QueuedMoves.Num() != Other.QueuedMoves.Num())
	{
		return false;
	}
	
	return true;
}

void FLayeredMoveInstanceGroup::QueueLayeredMove(const TSharedPtr<FLayeredMoveInstance>& Move)
{
	if (ensure(Move.IsValid() && Move->HasLogic()))
	{
		QueuedMoves.Add(Move);
		// TODO NS: Add simple string support and re add this log
		//UE_LOGF(LogMover, VeryVerbose, "LayeredMove queued move (%ls)", *Move->ToSimpleString());
	}
}
bool FLayeredMoveInstanceGroup::HasSameContents(const FLayeredMoveInstanceGroup& Other) const
{
	// Only compare the types of moves contained, not the state
	if (ActiveMoves.Num() != Other.ActiveMoves.Num() ||
		QueuedMoves.Num() != Other.QueuedMoves.Num())
	{
		return false;
	}
	for (int32 i = 0; i < ActiveMoves.Num(); ++i)
	{
		if (ActiveMoves[i]->GetDataStructType() != Other.ActiveMoves[i]->GetDataStructType())
		{
			return false;
		}
	}
	for (int32 i = 0; i < QueuedMoves.Num(); ++i)
	{
		if (QueuedMoves[i]->GetDataStructType() != Other.QueuedMoves[i]->GetDataStructType())
		{
			return false;
		}
	}
	return true;
}
void FLayeredMoveInstanceGroup::ApplyResidualVelocity(FProposedMove& ProposedMove)
{
	if (bApplyResidualVelocity)
	{
		ProposedMove.LinearVelocity = ResidualVelocity;
	}
	if (ResidualClamping >= 0.0f)
	{
		ProposedMove.LinearVelocity = ProposedMove.LinearVelocity.GetClampedToMaxSize(ResidualClamping);
	}
	ResetResidualVelocity();
}
bool FLayeredMoveInstanceGroup::GenerateMixedMove(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, UMovementMixer& MovementMixer, UMoverBlackboard* SimBlackboard, FProposedMove& OutMixedMove)
{
	// Tick and accumulate all active moves
	// Gather all proposed moves and distill this into a cumulative movement report. May include separate additive vs override moves.
	// TODO: may want to sort by priority or other factors
	bool bHasLayeredMoveContributions = false;
	for (const TSharedPtr<FLayeredMoveInstance>& Move : ActiveMoves)
	{
		FProposedMove MoveStep;
		if (Move->GenerateMove(StartState, TimeStep, SimBlackboard, MoveStep))
		{	
			bHasLayeredMoveContributions = true;
			
			MovementMixer.MixLayeredMove(*Move, MoveStep, OutMixedMove);
		}
	}
	
	return bHasLayeredMoveContributions;
}

void FLayeredMoveInstanceGroup::NetSerialize(FArchive& Ar, uint8 MaxNumMovesToSerialize/* = MAX_uint8*/)
{
	const auto NetSerializeMovesArrayFunc = [&Ar, this](TArray<TSharedPtr<FLayeredMoveInstance>>& MovesArray, uint8 MaxArraySize)
	{
		uint8 NumMovesToSerialize;
		if (Ar.IsSaving())
		{
			UE_CLOGF(MovesArray.Num() > MaxArraySize, LogMover, Warning, "Too many Layered Moves (%d!) to net serialize. Clamping to %d", MovesArray.Num(), MaxArraySize);
			NumMovesToSerialize = FMath::Min<int32>(MovesArray.Num(), MaxArraySize);
		}

		Ar << NumMovesToSerialize;

		if (Ar.IsLoading())
		{
			// Note that any instances of FLayeredMoveInstance added this way won't have their constructor called
			// They are not safe to use until after the NetSerialize() that immediately follows below (so don't add anything in between!) 
			MovesArray.SetNumZeroed(NumMovesToSerialize);
			
			for (int32 MoveIdx = 0; MoveIdx < NumMovesToSerialize && !Ar.IsError(); ++MoveIdx)
			{
				MovesArray[MoveIdx] = MakeShared<FLayeredMoveInstance>(MakeShared<FLayeredMoveInstancedData>(), nullptr);
				FLayeredMoveInstance* Move = MovesArray[MoveIdx].Get();
				Move->NetSerialize(Ar);
			}
		}
		else
		{
			for (int32 MoveIdx = 0; MoveIdx < NumMovesToSerialize && !Ar.IsError(); ++MoveIdx)
			{
				FLayeredMoveInstance* Move = MovesArray[MoveIdx].Get();
				Move->NetSerialize(Ar);
			}
		}
	};

	NetSerializeMovesArrayFunc(ActiveMoves, MaxNumMovesToSerialize);
	
	const uint8 MaxNumQueuedMovesToSerialize = FMath::Max(MaxNumMovesToSerialize - ActiveMoves.Num(), 0);
	NetSerializeMovesArrayFunc(QueuedMoves, MaxNumQueuedMovesToSerialize);
}


void FLayeredMoveInstanceGroup::AddStructReferencedObjects(FReferenceCollector& Collector) const
{
	for (const TSharedPtr<FLayeredMoveInstance>& Move : ActiveMoves)
	{
		Move->AddReferencedObjects(Collector);
	}
	for (const TSharedPtr<FLayeredMoveInstance>& Move : QueuedMoves)
	{
		Move->AddReferencedObjects(Collector);
	}
}

void FLayeredMoveInstanceGroup::ResetResidualVelocity()
{
	bApplyResidualVelocity = false;
	ResidualVelocity = FVector::ZeroVector;
	ResidualClamping = -1.f;
}

void FLayeredMoveInstanceGroup::Reset()
{
	ResetResidualVelocity();
	QueuedMoves.Empty();
	ActiveMoves.Empty();
	TagCancellationRequests.Empty();
}

void FLayeredMoveInstanceGroup::PopulateMissingActiveMoveLogic(const TArray<TObjectPtr<ULayeredMoveLogic>>& RegisteredMoves)
{
	for (TSharedPtr<FLayeredMoveInstance> ActiveMove : ActiveMoves)
	{
		if (ActiveMove && !ActiveMove->HasLogic())
		{
			bool bPopulatedActiveMoveLogic = ActiveMove->PopulateMissingActiveMoveLogic(RegisteredMoves);
		}
	}

	for (TSharedPtr<FLayeredMoveInstance> QueuedMove : QueuedMoves)
	{
		if (QueuedMove && !QueuedMove->HasLogic())
		{
			bool bPopulatedActiveMoveLogic = QueuedMove->PopulateMissingActiveMoveLogic(RegisteredMoves);
		}
	}
}

FString FLayeredMoveInstanceGroup::ToSimpleString() const
{
	return FString::Printf(TEXT("FLayeredMoveInstanceGroup. Active: %i Queued: %i"), ActiveMoves.Num(), QueuedMoves.Num());
}

const FLayeredMoveInstance* FLayeredMoveInstanceGroup::PrivateFindActiveMove(const TSubclassOf<ULayeredMoveLogic>& MoveLogicClass) const
{
	for (const TSharedPtr<FLayeredMoveInstance>& Move : ActiveMoves)
	{
		if (Move->GetLogicClass() == MoveLogicClass)
		{
			return Move.Get();
		}
	}

	return nullptr;
}

const FLayeredMoveInstance* FLayeredMoveInstanceGroup::PrivateFindActiveMove(const UScriptStruct* MoveDataType) const
{
	for (const TSharedPtr<FLayeredMoveInstance>& Move : ActiveMoves)
	{
		if (Move->GetDataStructType() == MoveDataType)
		{
			return Move.Get();
		}
	}

	return nullptr;
}

const FLayeredMoveInstance* FLayeredMoveInstanceGroup::PrivateFindQueuedMove(const TSubclassOf<ULayeredMoveLogic>& MoveLogicClass) const
{
	for (const TSharedPtr<FLayeredMoveInstance>& Move : QueuedMoves)
	{
		if (Move->GetLogicClass() == MoveLogicClass)
		{
			return Move.Get();
		}
	}

	return nullptr;
}

const FLayeredMoveInstance* FLayeredMoveInstanceGroup::PrivateFindQueuedMove(const UScriptStruct* MoveDataType) const
{
	for (const TSharedPtr<FLayeredMoveInstance>& Move : QueuedMoves)
	{
		if (Move->GetDataStructType() == MoveDataType)
		{
			return Move.Get();
		}
	}

	return nullptr;
}

void FLayeredMoveInstanceGroup::CancelMovesByTag(FGameplayTag Tag, bool bRequireExactMatch)
{
	// Queued moves can be removed now before they become active
	QueuedMoves.RemoveAll([Tag, bRequireExactMatch](const TSharedPtr<FLayeredMoveInstance>& Move)
		{
			return (!Move.IsValid() || Move->HasGameplayTag(Tag, bRequireExactMatch));
		});

	// Schedule a tag cancellation request, to handle any active moves during simulation
	TagCancellationRequests.Add(TPair<FGameplayTag, bool>(Tag, bRequireExactMatch));

}


void FLayeredMoveInstanceGroup::FlushMoveArrays(const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, bool bIsAsync)
{
	bool bResidualVelocityOverridden = false;
	bool bClampVelocityOverridden = false;

	// Process any cancellations
	{
		for (TPair<FGameplayTag, bool> CancelRequest : TagCancellationRequests)
		{
			const FGameplayTag TagToMatch = CancelRequest.Key;
			const bool bRequireExactMatch = CancelRequest.Value;



			// Process completion of any active moves that being canceled
			// Only removing active ones because queued ones were cancelled at the time the request was added
			ActiveMoves.RemoveAll([&TimeStep, &SimBlackboard, &bResidualVelocityOverridden, &bClampVelocityOverridden, TagToMatch, bRequireExactMatch, bIsAsync, this] (const TSharedPtr<FLayeredMoveInstance>& Move)
				{
					if (Move.IsValid() && Move->HasGameplayTag(TagToMatch, bRequireExactMatch))
					{
						ProcessFinishedMove(*Move, bResidualVelocityOverridden, bClampVelocityOverridden);
						if (bIsAsync)
						{
							Move->EndMove_Async(SimBlackboard, TimeStep.ToStartTime());
						}
						else
						{
							Move->EndMove(TimeStep, SimBlackboard);
						}
						return true;
					}
					return false;
				});
		}

		TagCancellationRequests.Empty();
	}

	{

		// Process completion of any active moves that are finished
		ActiveMoves.RemoveAll([&TimeStep, &SimBlackboard, &bResidualVelocityOverridden, &bClampVelocityOverridden, bIsAsync, this]
			(const TSharedPtr<FLayeredMoveInstance>& Move)
			{
				if (Move.IsValid() && Move->IsFinished(TimeStep, SimBlackboard))
				{
					ProcessFinishedMove(*Move, bResidualVelocityOverridden, bClampVelocityOverridden);
					if (bIsAsync)
					{
						Move->EndMove_Async(SimBlackboard, TimeStep.ToStartTime());
					}
					else
					{
						Move->EndMove(TimeStep, SimBlackboard);
					}
					return true;
				}
				return false;
			});
	}

	// Begin any queued moves
	for (TSharedPtr<FLayeredMoveInstance>& QueuedMove : QueuedMoves)
	{
		if (QueuedMove.IsValid())
		{
			if (QueuedMove->HasLogic())
			{
				ActiveMoves.Add(QueuedMove);
				if (bIsAsync)
				{
					QueuedMove->StartMove_Async(SimBlackboard, TimeStep.ToStartTime());
				}
				else
				{
					QueuedMove->StartMove(TimeStep, SimBlackboard);
				}
			}
			else
			{
				// We should've populated missing logic before this so let's just clear this and warn about it
				UE_LOGF(LogMover, Warning, "Queued Active Move (%ls) logic was not present. Move will not be activated.", *QueuedMove->GetDataStructType()->GetName());
			}
		}
	}

	QueuedMoves.Empty();
}

TArray<TSharedPtr<FLayeredMoveInstance>> FLayeredMoveInstanceGroup::GenerateActiveMoves_Async(
	const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard)
{
	FlushMoveArrays(TimeStep, SimBlackboard, /* bIsAsync = */ true);
	return ActiveMoves.FilterByPredicate([](const TSharedPtr<FLayeredMoveInstance>& Move)
	{
		return Move.IsValid() && Move->HasLogic() && Move->GetLogic()->SupportsAsync();
	});
}

void FLayeredMoveInstanceGroup::ProcessFinishedMove(const FLayeredMoveInstance& Move, bool& bResidualVelocityOverridden, bool& bClampVelocityOverridden)
{
	const FLayeredMoveFinishVelocitySettings& FinishVelocitySettings = Move.GetFinishVelocitySettings();
	const EMoveMixMode MixMode = Move.GetMixMode();
	
	if (FinishVelocitySettings.FinishVelocityMode == ELayeredMoveFinishVelocityMode::SetVelocity)
	{
		bApplyResidualVelocity = true;

		switch (MixMode)
		{
		case EMoveMixMode::AdditiveVelocity:
			if (!bResidualVelocityOverridden)
			{
				ResidualVelocity += FinishVelocitySettings.SetVelocity;
			}
			break;
		case EMoveMixMode::OverrideVelocity:
		case EMoveMixMode::OverrideAll:
			{
				UE_CLOGF(bClampVelocityOverridden, LogMover, Log, "Multiple LayeredMove residual settings have a MixMode that overrides. Only one will take effect.");
				bResidualVelocityOverridden = true;
				ResidualVelocity = FinishVelocitySettings.SetVelocity;
			}
			break;
		}
	}
	else if (FinishVelocitySettings.FinishVelocityMode == ELayeredMoveFinishVelocityMode::ClampVelocity)
	{
		switch (MixMode)
		{
		case EMoveMixMode::AdditiveVelocity:
			if (!bClampVelocityOverridden)
			{
				if (ResidualClamping < 0)
				{
					ResidualClamping = FinishVelocitySettings.ClampVelocity;
				}
				else if (ResidualClamping > FinishVelocitySettings.ClampVelocity)
				{
					// No way to really add clamping so we instead apply it if it was smaller
					ResidualClamping = FinishVelocitySettings.ClampVelocity;
				}
			}
			break;
		case EMoveMixMode::OverrideVelocity:
		case EMoveMixMode::OverrideAll:
			{
				UE_CLOGF(bClampVelocityOverridden, LogMover, Log, "Multiple LayeredMove residual settings have a MixMode that overrides. Only one will take effect.");
				bClampVelocityOverridden = true;
				ResidualClamping = FinishVelocitySettings.ClampVelocity;
			}
			break;
		}
	}
}
