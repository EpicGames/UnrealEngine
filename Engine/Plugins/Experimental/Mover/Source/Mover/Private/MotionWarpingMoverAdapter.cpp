// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotionWarpingMoverAdapter.h"
#include "Components/SkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MotionWarpingMoverAdapter)

void UMotionWarpingMoverAdapter::BeginDestroy()
{
	if (TargetMoverComp)
	{
		TargetMoverComp->ProcessLocalRootMotionDelegate.Unbind();
	}

	Super::BeginDestroy();
}

void UMotionWarpingMoverAdapter::SetMoverComp(UMoverComponent* InMoverComp)
{

	TargetMoverComp = InMoverComp;
	TargetMoverComp->ProcessLocalRootMotionDelegate.BindUObject(this, &UMotionWarpingMoverAdapter::WarpLocalRootMotionOnMoverComp);
}

AActor* UMotionWarpingMoverAdapter::GetActor() const
{
	return TargetMoverComp->GetOwner();
}

USkeletalMeshComponent* UMotionWarpingMoverAdapter::GetMesh() const
{
	return TargetMoverComp->GetPrimaryVisualComponent<USkeletalMeshComponent>();
}

FVector UMotionWarpingMoverAdapter::GetVisualRootLocation() const
{
	if (const USceneComponent* PrimaryVisualComp = TargetMoverComp->GetPrimaryVisualComponent())
	{
		return PrimaryVisualComp->GetComponentLocation();
	}

	if (const USceneComponent* UpdatedComponent = TargetMoverComp->GetUpdatedComponent())
	{
		const FVector ActorRootLocation = UpdatedComponent->GetComponentLocation();
		const FQuat ActorRootOrientation = UpdatedComponent->GetComponentQuat();
		FBoxSphereBounds ActorRootBounds = UpdatedComponent->GetLocalBounds();

		return ActorRootLocation - (ActorRootOrientation.GetUpVector() * ActorRootBounds.BoxExtent.Z);
	}

	return TargetMoverComp->GetOwner()->GetActorLocation();
}

FVector UMotionWarpingMoverAdapter::GetBaseVisualTranslationOffset() const
{
	// TODO: rework these GetBase****Offset functions once MoverComponent supports primary visual component offset (coming as part of mesh-based smoothing)
	if (const USceneComponent* VisualComp = TargetMoverComp->GetPrimaryVisualComponent())
	{
		return VisualComp->GetRelativeLocation();
	}

	return FVector::ZeroVector;
}

FQuat UMotionWarpingMoverAdapter::GetBaseVisualRotationOffset() const
{
	if (const USceneComponent* VisualComp = TargetMoverComp->GetPrimaryVisualComponent())
	{
		return VisualComp->GetRelativeRotation().Quaternion();
	}

	return FQuat::Identity;
}

FTransform UMotionWarpingMoverAdapter::WarpLocalRootMotionOnMoverComp(const FTransform& LocalRootMotionTransform, float DeltaSeconds, const FMotionWarpingUpdateContext* OptionalWarpingContext)
{
	if (WarpLocalRootMotionDelegate.IsBound())
	{
		return WarpLocalRootMotionDelegate.Execute(LocalRootMotionTransform, DeltaSeconds, OptionalWarpingContext);
	}

	return LocalRootMotionTransform;
}
