// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoveLibrary/AirMovementUtils.h"

#include "MoverComponent.h"
#include "MoveLibrary/AsyncMovementUtils.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/MovementUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AirMovementUtils)

FProposedMove UAirMovementUtils::ComputeControlledFreeMove(const FFreeMoveParams& InParams)
{
	FProposedMove OutMove;

	const FPlane MovementPlane(FVector::ZeroVector, FVector::UpVector);

	OutMove.DirectionIntent = UMovementUtils::ComputeDirectionIntent(InParams.MoveInput, InParams.MoveInputType, InParams.MaxSpeed);
	OutMove.bHasDirIntent = !OutMove.DirectionIntent.IsNearlyZero();

	FComputeVelocityParams ComputeVelocityParams;
	ComputeVelocityParams.DeltaSeconds = InParams.DeltaSeconds;
	ComputeVelocityParams.InitialVelocity = InParams.PriorVelocity;
	ComputeVelocityParams.MoveDirectionIntent = InParams.MoveInput;
	ComputeVelocityParams.MaxSpeed = InParams.MaxSpeed;
	ComputeVelocityParams.TurningBoost = InParams.TurningBoost;
	ComputeVelocityParams.Deceleration = InParams.Deceleration;
	ComputeVelocityParams.Acceleration = InParams.Acceleration;
	ComputeVelocityParams.MoveInputType = InParams.MoveInputType;
	ComputeVelocityParams.MoveInput = InParams.MoveInput; 
	ComputeVelocityParams.bUseAccelerationForVelocityMove = InParams.bUseAccelerationForVelocityMove;
	ComputeVelocityParams.Friction = InParams.Friction;
	
	OutMove.LinearVelocity = UMovementUtils::ComputeVelocity(ComputeVelocityParams);
	OutMove.AngularVelocityDegrees = UMovementUtils::ComputeAngularVelocityDegrees(InParams.PriorOrientation, InParams.OrientationIntent, InParams.DeltaSeconds, InParams.TurningRate);

	return OutMove;
}

bool UAirMovementUtils::IsValidLandingSpot(const FMovingComponentSet& MovingComps, const FVector& Location, const FHitResult& Hit, const FFloorCheckSettings& FloorCheckSettings, FFloorCheckResult& OutFloorResult)
{
	OutFloorResult.Clear();

	if (!Hit.bBlockingHit)
	{
		return false;
	}

	if (Hit.bStartPenetrating)
	{
		return false;
	}

	// Reject unwalkable floor normals.
	if (!UFloorQueryUtils::IsHitSurfaceWalkable(Hit, MovingComps.MoverComponent->GetUpDirection(), FloorCheckSettings.MaxWalkSlopeCosine))
	{
		return false;
	}

	// Make sure floor test passes here.
	UFloorQueryUtils::FindFloor(MovingComps, FloorCheckSettings, Location, OutFloorResult);

	if (!OutFloorResult.IsWalkableFloor())
	{
		return false;
	}

	return true;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UAirMovementUtils::IsValidLandingSpot(const FMovingComponentSet& MovingComps, const FVector& Location, const FHitResult& Hit, float FloorSweepDistance, float MaxWalkSlopeCosine, bool bUseFlatBaseForFloorChecks, FFloorCheckResult& OutFloorResult, float PerchRadiusThreshold)
{
	FFloorCheckSettings Settings;
	Settings.FloorSweepDistance = FloorSweepDistance;
	Settings.MaxWalkSlopeCosine = MaxWalkSlopeCosine;
	Settings.bUseFlatBaseForFloorChecks = bUseFlatBaseForFloorChecks;
	Settings.PerchRadiusThreshold = PerchRadiusThreshold;
	return IsValidLandingSpot(MovingComps, Location, Hit, Settings, OutFloorResult);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

float UAirMovementUtils::TryMoveToFallAlongSurface(const FMovingComponentSet& MovingComps, const FVector& Delta, float PctOfDeltaToMove, const FQuat Rotation, const FVector& Normal, FHitResult& Hit, bool bHandleImpact, const FFloorCheckSettings& FloorCheckSettings, FFloorCheckResult& OutFloorResult, FMovementRecord& MoveRecord)
{
	OutFloorResult.Clear();

	if (!Hit.bBlockingHit)
	{
		return 0.f;
	}

	float PctOfTimeUsed = 0.f;
	const FVector OldHitNormal = Normal;

	FVector SlideDelta = UMovementUtils::ComputeSlideDelta(MovingComps, Delta, PctOfDeltaToMove, Normal, Hit);

	if ((SlideDelta | Delta) > 0.f)
	{
		// First sliding attempt along surface
		UMovementUtils::TrySafeMoveUpdatedComponent(MovingComps, SlideDelta, Rotation, true, Hit, ETeleportType::None, MoveRecord);

		PctOfTimeUsed = Hit.Time;
		if (Hit.IsValidBlockingHit())
		{
			UMoverComponent* MoverComponent = MovingComps.MoverComponent.Get();
			UPrimitiveComponent* UpdatedPrimitive = MovingComps.UpdatedPrimitive.Get();

			// Notify first impact
			if (MoverComponent && bHandleImpact)
			{
				FMoverOnImpactParams ImpactParams(NAME_None, Hit, SlideDelta);
				MoverComponent->HandleImpact(ImpactParams);
			}

			// Check if we landed
			if (!IsValidLandingSpot(MovingComps, UpdatedPrimitive->GetComponentLocation(),
				Hit, FloorCheckSettings, OutFloorResult))
			{
				// We've hit another surface during our first move, so let's try to slide along both of them together

				// Compute new slide normal when hitting multiple surfaces.
				SlideDelta = UMovementUtils::ComputeTwoWallAdjustedDelta(MovingComps, SlideDelta, Hit, OldHitNormal);

				// Only proceed if the new direction is of significant length and not in reverse of original attempted move.
				if (!SlideDelta.IsNearlyZero(UE::MoverUtils::SMALL_MOVE_DISTANCE) && (SlideDelta | Delta) > 0.f)
				{
					// Perform second move, taking 2 walls into account
					UMovementUtils::TrySafeMoveUpdatedComponent(MovingComps, SlideDelta, Rotation, true, Hit, ETeleportType::None, MoveRecord);
					PctOfTimeUsed += (Hit.Time * (1.f - PctOfTimeUsed));

					// Notify second impact
					if (MoverComponent && bHandleImpact && Hit.bBlockingHit)
					{
						FMoverOnImpactParams ImpactParams(NAME_None, Hit, SlideDelta);
						MoverComponent->HandleImpact(ImpactParams);
					}

					// Check if we've landed, to acquire floor result
					IsValidLandingSpot(MovingComps, UpdatedPrimitive->GetComponentLocation(),
						Hit, FloorCheckSettings, OutFloorResult);
				}
			}
		}

		return FMath::Clamp(PctOfTimeUsed, 0.f, 1.f);
	}

	return 0.f;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
float UAirMovementUtils::TryMoveToFallAlongSurface(const FMovingComponentSet& MovingComps, const FVector& Delta, float PctOfDeltaToMove, const FQuat Rotation, const FVector& Normal, FHitResult& Hit, bool bHandleImpact, float FloorSweepDistance, float MaxWalkSlopeCosine, bool bUseFlatBaseForFloorChecks, FFloorCheckResult& OutFloorResult, FMovementRecord& MoveRecord, float PerchRadiusThreshold)
{
	FFloorCheckSettings Settings;
	Settings.FloorSweepDistance = FloorSweepDistance;
	Settings.MaxWalkSlopeCosine = MaxWalkSlopeCosine;
	Settings.bUseFlatBaseForFloorChecks = bUseFlatBaseForFloorChecks;
	Settings.PerchRadiusThreshold = PerchRadiusThreshold;
	return TryMoveToFallAlongSurface(MovingComps, Delta, PctOfDeltaToMove, Rotation, Normal, Hit, bHandleImpact, Settings, OutFloorResult, MoveRecord);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS


/* static */
float UAirMovementUtils::TestFallingMoveAlongHitSurface(const FMovingComponentSet& MovingComps, const FVector& OriginalMoveDelta, const FVector& LocationAtHit, const FQuat& TargetRotation, bool bHandleImpact, const FFloorCheckSettings& FloorCheckSettings, FHitResult& InOutHit, FFloorCheckResult& OutFloorResult, FMovementRecord& InOutMoveRecord)
{
	OutFloorResult.Clear();

	if (!InOutHit.bBlockingHit)
	{
		return 0.f;
	}

	float PctOfTimeUsed = 0.f;
	float PctOfOrigDeltaToSlide = 1.f - InOutHit.Time;
	const FVector OrigHitNormal = InOutHit.Normal;

	FVector SlideDelta = UMovementUtils::ComputeSlideDelta(MovingComps, OriginalMoveDelta, PctOfOrigDeltaToSlide, OrigHitNormal, InOutHit);

	if ((SlideDelta | OriginalMoveDelta) > 0.f)
	{
		// First sliding attempt along surface
		UAsyncMovementUtils::TestDepenetratingMove(MovingComps, LocationAtHit, LocationAtHit + SlideDelta, TargetRotation, TargetRotation, /*bShouldSweep*/ true, InOutHit, InOutMoveRecord);

		PctOfTimeUsed = InOutHit.Time;

		if (InOutHit.IsValidBlockingHit())
		{
			UMoverComponent* MoverComponent = MovingComps.MoverComponent.Get();
			const UPrimitiveComponent* UpdatedPrimitive = MovingComps.UpdatedPrimitive.Get();

			// Notify first impact
			if (MoverComponent && bHandleImpact)
			{
				FMoverOnImpactParams ImpactParams(NAME_None, InOutHit, SlideDelta);
				MoverComponent->HandleImpact(ImpactParams);
			}

			const FVector LocationAfter1stSlide = InOutHit.TraceStart + ((InOutHit.TraceEnd - InOutHit.TraceStart) * InOutHit.Time);

			// Check if we landed
			if (!UAirMovementUtils::IsValidLandingSpot(MovingComps, LocationAfter1stSlide,
				InOutHit, FloorCheckSettings, OutFloorResult))
			{
				// We've hit another surface during our first move, so let's try to slide along both of them together

				// Compute new slide normal when hitting multiple surfaces.
				SlideDelta = UMovementUtils::ComputeTwoWallAdjustedDelta(MovingComps, SlideDelta, InOutHit, OrigHitNormal);

				// Only proceed if the new direction is of significant length and not in reverse of original attempted move.
				if (!SlideDelta.IsNearlyZero(UE::MoverUtils::SMALL_MOVE_DISTANCE) && (SlideDelta | OriginalMoveDelta) > 0.f)
				{
					// Perform second move, taking 2 walls into account
					UAsyncMovementUtils::TestDepenetratingMove(MovingComps, LocationAfter1stSlide, LocationAfter1stSlide + SlideDelta, TargetRotation, TargetRotation, /*bShouldSweep*/ true, InOutHit, InOutMoveRecord);
					PctOfTimeUsed += (InOutHit.Time * (1.f - PctOfTimeUsed));

					// Notify second impact
					if (MoverComponent && bHandleImpact && InOutHit.bBlockingHit)
					{
						FMoverOnImpactParams ImpactParams(NAME_None, InOutHit, SlideDelta);
						MoverComponent->HandleImpact(ImpactParams);
					}

					const FVector LocationAfter2ndSlide = InOutHit.TraceStart + ((InOutHit.TraceEnd - InOutHit.TraceStart) * InOutHit.Time);

					// Check if we've landed, to acquire floor result
					UAirMovementUtils::IsValidLandingSpot(MovingComps, LocationAfter2ndSlide,
						InOutHit, FloorCheckSettings, OutFloorResult);
				}
			}
		}

		return FMath::Clamp(PctOfTimeUsed, 0.f, 1.f);
	}

	return 0.f;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
float UAirMovementUtils::TestFallingMoveAlongHitSurface(const FMovingComponentSet& MovingComps, const FVector& OriginalMoveDelta, const FVector& LocationAtHit, const FQuat& TargetRotation, bool bHandleImpact, float FloorSweepDistance, float MaxWalkSlopeCosine, bool bUseFlatBaseForFloorChecks, FHitResult& InOutHit, FFloorCheckResult& OutFloorResult, FMovementRecord& InOutMoveRecord, float PerchRadiusThreshold)
{
	FFloorCheckSettings Settings;
	Settings.FloorSweepDistance = FloorSweepDistance;
	Settings.MaxWalkSlopeCosine = MaxWalkSlopeCosine;
	Settings.bUseFlatBaseForFloorChecks = bUseFlatBaseForFloorChecks;
	Settings.PerchRadiusThreshold = PerchRadiusThreshold;
	return TestFallingMoveAlongHitSurface(MovingComps, OriginalMoveDelta, LocationAtHit, TargetRotation, bHandleImpact, Settings, InOutHit, OutFloorResult, InOutMoveRecord);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

