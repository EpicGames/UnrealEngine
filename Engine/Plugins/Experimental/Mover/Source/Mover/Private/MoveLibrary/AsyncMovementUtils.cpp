// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoveLibrary/AsyncMovementUtils.h"
#include "MoveLibrary/MovementRecord.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoverComponent.h"
#include "MoverLog.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncMovementUtils)



// TODO: replace with real cvars
static float HitDistanceToleranceCVar = 0.f;
static float InitialOverlapToleranceCVar = 0.f;
static float PenetrationOverlapCheckInflationCVar = 0.1f;

static const FName TestSafeMoveSubstepName = "TestSafeMove";
static const FName TestSafeDepenetrationSubstepName = "TestSafeResolvePenetration";

/* static */
bool UAsyncMovementUtils::TestDepenetratingMove(const FMovingComponentSet& MovingComps, const FVector& StartLocation, const FVector& TargetLocation, const FQuat& StartRotation, const FQuat& TargetRotation, bool bShouldSweep, FHitResult& OutHit, FMovementRecord& InOutMoveRecord)
{
	FMoverCollisionParams CollisionParams(MovingComps.UpdatedComponent.Get());
	return TestDepenetratingMove(MovingComps, StartLocation, TargetLocation, StartRotation, TargetRotation, bShouldSweep, CollisionParams, OutHit, InOutMoveRecord);
}

/* static */
bool UAsyncMovementUtils::TestDepenetratingMove(const FMovingComponentSet& MovingComps, const FVector& StartLocation, const FVector& TargetLocation, const FQuat& StartRotation, const FQuat& TargetRotation, bool bShouldSweep, FMoverCollisionParams& CollisionParams, FHitResult& OutHit, FMovementRecord& InOutMoveRecord)
{
	const USceneComponent* UpdatedComponent = MovingComps.UpdatedComponent.Get();

	if (!UpdatedComponent)
	{
		OutHit.Reset(1.f);
		return false;
	}

	bool bDidMove = false;

	FVector ResolvedStartLocation = StartLocation;
	FVector CurrentLocation = StartLocation;

	// Test full move first
	{
		const EMoveComponentFlags IncludeBlockingOverlapsWithoutEvents = (MOVECOMP_NeverIgnoreBlockingOverlaps | MOVECOMP_DisableBlockingOverlapDispatch);

		TGuardValue<EMoveComponentFlags> ScopedFlagRestore(CollisionParams.MoveComponentFlags, EMoveComponentFlags(CollisionParams.MoveComponentFlags | IncludeBlockingOverlapsWithoutEvents));

		bDidMove = TestMoveComponent_Internal(MovingComps, ResolvedStartLocation, TargetLocation, StartRotation, TargetRotation, bShouldSweep, CollisionParams, OutHit);

		CurrentLocation = ResolvedStartLocation + ((TargetLocation-ResolvedStartLocation) * OutHit.Time);

		UE_LOGF(LogMover, VeryVerbose, "TestDepenetratingMove: %ls (role %i) Delta=%ls DidMove=%i",
			*GetNameSafe(UpdatedComponent->GetOwner()), UpdatedComponent->GetOwnerRole(), *(TargetLocation- ResolvedStartLocation).ToCompactString(), bDidMove);
	}

	// If we are starting out in penetration, try to resolve it and then retry
	if (OutHit.bStartPenetrating)
	{
		FVector AdjustmentToResolve = FVector::ZeroVector;
		if (FindMoveToResolveInitialPenetration_Internal(MovingComps, StartLocation, StartRotation, OutHit, CollisionParams, AdjustmentToResolve))
		{
			ResolvedStartLocation = StartLocation + AdjustmentToResolve;
			CurrentLocation = ResolvedStartLocation;	// keeping track of where we've moved so far

			InOutMoveRecord.Append(FMovementSubstep(TestSafeDepenetrationSubstepName, AdjustmentToResolve, false));

			// Retry original move
			bDidMove |= TestMoveComponent_Internal(MovingComps, ResolvedStartLocation, TargetLocation, StartRotation, TargetRotation, bShouldSweep, CollisionParams, OutHit);

			UE_LOGF(LogMover, VeryVerbose, "TestDepenetratingMove retry: %ls (role %i) Delta=%ls DidMove=%i",
				*GetNameSafe(UpdatedComponent->GetOwner()), UpdatedComponent->GetOwnerRole(), *(TargetLocation - ResolvedStartLocation).ToCompactString(), bDidMove);

			if (bDidMove)
			{
				CurrentLocation = ResolvedStartLocation + ((TargetLocation - ResolvedStartLocation) * OutHit.Time);
			}
		}
	}

	if (bDidMove)
	{
		InOutMoveRecord.Append(FMovementSubstep(TestSafeMoveSubstepName, CurrentLocation - ResolvedStartLocation, true));
	}

	return bDidMove;
}

/* static */
float UAsyncMovementUtils::TestSlidingMoveAlongHitSurface(const FMovingComponentSet& MovingComps, const FVector& OriginalMoveDelta, const FVector& LocationAtHit, const FQuat& TargetRotation, FHitResult& InOutHit, FMovementRecord& InOutMoveRecord)
{
	FMoverCollisionParams CollisionParams(MovingComps.UpdatedComponent.Get());
	return TestSlidingMoveAlongHitSurface(MovingComps, OriginalMoveDelta, LocationAtHit, TargetRotation, CollisionParams, InOutHit, InOutMoveRecord);

}

/* static */ 
float UAsyncMovementUtils::TestSlidingMoveAlongHitSurface(const FMovingComponentSet& MovingComps, const FVector& OriginalMoveDelta, const FVector& LocationAtHit, const FQuat& TargetRotation, FMoverCollisionParams& CollisionParams, FHitResult& InOutHit, FMovementRecord& InOutMoveRecord)
{
	if (!InOutHit.bBlockingHit)
	{
		return 0.f;
	}

	float PctOfTimeUsed = 0.f;
	float PctOfOrigDeltaToSlide = 1.f - InOutHit.Time;
	const FVector OrigHitNormal = InOutHit.Normal;

	FVector SlideDelta = UMovementUtils::ComputeSlideDelta(MovingComps, OriginalMoveDelta, PctOfOrigDeltaToSlide, OrigHitNormal, InOutHit);

	if ((SlideDelta | OriginalMoveDelta) > 0.f)
	{
		TestDepenetratingMove(MovingComps, LocationAtHit, LocationAtHit+SlideDelta, TargetRotation, TargetRotation, /*bShouldSweep*/ true, CollisionParams, InOutHit, InOutMoveRecord);

		PctOfTimeUsed = InOutHit.Time;

		if (InOutHit.IsValidBlockingHit())
		{
			// TODO: maybe record impact here

			// Compute new slide normal when hitting multiple surfaces.
			SlideDelta = UMovementUtils::ComputeTwoWallAdjustedDelta(MovingComps, SlideDelta, InOutHit, OrigHitNormal);

			// Only proceed if the new direction is of significant length and not in reverse of original attempted move.
			if (!SlideDelta.IsNearlyZero(UE::MoverUtils::SMALL_MOVE_DISTANCE) && (SlideDelta | OriginalMoveDelta) > 0.f)
			{
				const FVector LocationAfter1stSlide = InOutHit.TraceStart + ((InOutHit.TraceEnd-InOutHit.TraceStart) * InOutHit.Time);

				// Perform second move
				TestDepenetratingMove(MovingComps, LocationAfter1stSlide, LocationAfter1stSlide + SlideDelta, TargetRotation, TargetRotation, /*bShouldSweep*/ true, CollisionParams, InOutHit, InOutMoveRecord);
				PctOfTimeUsed += (InOutHit.Time * (1.f - PctOfTimeUsed));

				// TODO: maybe record second impact here
			}
		}

		return FMath::Clamp(PctOfTimeUsed, 0.f, 1.f);
	}

	return 0.f;

}


/* static */
bool UAsyncMovementUtils::FindMoveToResolveInitialPenetration_Internal(const FMovingComponentSet& MovingComps, const FVector& StartLocation, const FQuat& StartRotation, const FHitResult& PenetratingHit, FMoverCollisionParams& CollisionParams, FVector& OutAdjustmentDelta)
{
	if (ensureMsgf(PenetratingHit.bStartPenetrating, TEXT("Expected a hit that started penetrating. Will not attempt adjustment.")))
	{
		if (!MovingComps.UpdatedComponent.IsValid() || !MovingComps.UpdatedPrimitive.IsValid())
		{
			return false;
		}

		const USceneComponent* UpdatedComponent = MovingComps.UpdatedComponent.Get();
		const UPrimitiveComponent* UpdatedPrimitive = MovingComps.UpdatedPrimitive.Get();

		const AActor* ActorOwner = UpdatedPrimitive->GetOwner();
		if (!ActorOwner)
		{
			return false;
		}

		FVector ProposedAdjustment = UMovementUtils::ComputePenetrationAdjustment(PenetratingHit);
		ProposedAdjustment = UPlanarConstraintUtils::ConstrainDirectionToPlane(MovingComps.MoverComponent->GetPlanarConstraint(), ProposedAdjustment);

		// SceneComponent can't be in penetration, so this function really only applies to PrimitiveComponent.
		if (!ProposedAdjustment.IsZero())
		{
			// Attempts to adjust:
			//	1) Check if our component will fit with the proposed adjustment. If so, accept the proposed adjustment.
			//  2) Try sweep moving out while ignoring blocking overlaps. If move is allowed, capture how far we actually moved and accept that as the proposed adjustment.
			//  3) If still stuck in penetration, add a second penetration adjustment
			//  4) If still stuck, try moving the entire amount

			// See if we can fit at the adjusted location without overlapping anything.


			// We really want to make sure that precision differences or differences between the overlap test and sweep tests don't put us into another overlap,
			// so make the overlap test a bit more restrictive.
			const float OverlapInflation = PenetrationOverlapCheckInflationCVar;
			bool bEncroached = UMovementUtils::OverlapTest(UpdatedComponent, UpdatedPrimitive, PenetratingHit.TraceStart + ProposedAdjustment, StartRotation, UpdatedPrimitive->GetCollisionObjectType(), UpdatedPrimitive->GetCollisionShape(OverlapInflation), ActorOwner);

			if (!bEncroached)
			{
				// No more overlapping, so we've got an acceptable adjustment
				OutAdjustmentDelta = ProposedAdjustment;
				return true;
			}

			// Disable MOVECOMP_NeverIgnoreBlockingOverlaps if it is enabled, otherwise we wouldn't be able to sweep out of the object to fix the penetration.
			TGuardValue<EMoveComponentFlags> ScopedFlagRestore(CollisionParams.MoveComponentFlags, EMoveComponentFlags(CollisionParams.MoveComponentFlags & (~MOVECOMP_NeverIgnoreBlockingOverlaps)));

			// Try sweeping out as far as possible...

			FHitResult SweepOutHit(1.f);

			bool bMoved = TestMoveComponent_Internal(MovingComps, StartLocation, StartLocation + ProposedAdjustment, StartRotation, StartRotation, /* bShouldSweep */ true, CollisionParams, SweepOutHit);

			UE_LOGF(LogMover, VeryVerbose, "FindMoveToResolveInitialPenetration_Internal: %ls (role %i) Adjustment=%ls DidMove=%i",
				*GetNameSafe(UpdatedComponent->GetOwner()), UpdatedComponent->GetOwnerRole(), *ProposedAdjustment.ToCompactString(), bMoved);

			// Still stuck in penetration?
			if (!bMoved && SweepOutHit.bStartPenetrating)
			{
				// Combine two MTD results to get a new direction that gets out of multiple surfaces.
				const FVector SecondMTD = UMovementUtils::ComputePenetrationAdjustment(SweepOutHit);
				const FVector CombinedMTD = ProposedAdjustment + SecondMTD;

				if (SecondMTD != ProposedAdjustment && !CombinedMTD.IsZero())
				{
					bMoved = TestMoveComponent_Internal(MovingComps, StartLocation, StartLocation+CombinedMTD, StartRotation, StartRotation, /*bShouldSweep*/ true, CollisionParams, SweepOutHit);

					UE_LOGF(LogMover, VeryVerbose, "FindMoveToResolveInitialPenetration_Internal combined: %ls (role %i) CombinedAdjustment=%ls DidMove=%i",
						*GetNameSafe(UpdatedComponent->GetOwner()), UpdatedComponent->GetOwnerRole(), *CombinedMTD.ToCompactString(), bMoved);
				}

			}

			// Still stuck?
			if (!bMoved)
			{
				const UMoverComponent* MoverComp = MovingComps.MoverComponent.Get();

				// Try moving the proposed adjustment plus the attempted move direction. This can sometimes get out of penetrations with multiple objects
				const FVector FullMoveDelta = UPlanarConstraintUtils::ConstrainDirectionToPlane(MoverComp->GetPlanarConstraint(), (PenetratingHit.TraceEnd - PenetratingHit.TraceStart));
				if (!FullMoveDelta.IsZero())
				{
					const FVector TargetLocationWithAdjustment = StartLocation + ProposedAdjustment + FullMoveDelta;
					bMoved = TestMoveComponent_Internal(MovingComps, StartLocation, TargetLocationWithAdjustment, StartRotation, StartRotation, /*bShouldSweep*/ true, CollisionParams, SweepOutHit);

					UE_LOGF(LogMover, VeryVerbose, "FindMoveToResolveInitialPenetration_Internal multiple: %ls (role %i) TargetLocationWithAdjustment=%ls DidMove=%i",
						*GetNameSafe(UpdatedComponent->GetOwner()), UpdatedComponent->GetOwnerRole(), *TargetLocationWithAdjustment.ToCompactString(), bMoved);
				}
			}

			if (bMoved)
			{
				const FVector FinalAdjustmentLocation = SweepOutHit.TraceStart + ((SweepOutHit.TraceEnd-SweepOutHit.TraceStart) * SweepOutHit.Time);
				OutAdjustmentDelta = FinalAdjustmentLocation - StartLocation;
				return true;
			}
		}
	}

	OutAdjustmentDelta = FVector::ZeroVector;
	return false;
}

static bool ShouldIgnoreHitResult(const UWorld* InWorld, FHitResult const& TestHit, FVector const& MovementDirDenormalized, const AActor* MovingActor, EMoveComponentFlags MoveFlags)
{
	if (TestHit.bBlockingHit)
	{
		// check "ignore bases" functionality
		if ((MoveFlags & MOVECOMP_IgnoreBases) && MovingActor)	//we let overlap components go through because their overlap is still needed and will cause beginOverlap/endOverlap events
		{
			// ignore if there's a base relationship between moving actor and hit actor
			AActor const* const HitActor = TestHit.HitObjectHandle.FetchActor();
			if (HitActor)
			{
				if (MovingActor->IsBasedOnActor(HitActor) || HitActor->IsBasedOnActor(MovingActor))
				{
					return true;
				}
			}
		}

		// If we started penetrating, we may want to ignore it if we are moving out of penetration.
		// This helps prevent getting stuck in walls.
		if ((TestHit.Distance < HitDistanceToleranceCVar || TestHit.bStartPenetrating) && !(MoveFlags & MOVECOMP_NeverIgnoreBlockingOverlaps))
		{
			const float DotTolerance = InitialOverlapToleranceCVar;

			// Dot product of movement direction against 'exit' direction
			const FVector MovementDir = MovementDirDenormalized.GetSafeNormal();
			const float MoveDot = (TestHit.ImpactNormal | MovementDir);

			const bool bMovingOut = MoveDot > DotTolerance;

			// If we are moving out, ignore this result!
			if (bMovingOut)
			{
				return true;
			}
		}
	}

	return false;
}

// Adjusts the time of the hit result to pull it back slightly from the actual hit
static void PullBackHit(FHitResult& Hit, const float OrigMoveDist)
{
	const float DesiredTimeBack = FMath::Clamp(0.1f, 0.1f / OrigMoveDist, 1.f / OrigMoveDist) + 0.001f;
	Hit.Time = FMath::Clamp(Hit.Time - DesiredTimeBack, 0.f, 1.f);
}


static const FName TestSweepTraceTagName = "SweepTestMoverComponent";

// Returns true if any of the sweeping movement could occur
bool UAsyncMovementUtils::TestMoveComponent_Internal(const FMovingComponentSet& MovingComps, const FVector& StartLocation, const FVector& TargetLocation, const FQuat& StartRotation, const FQuat& TargetRotation, bool bShouldSweep, FMoverCollisionParams& CollisionParams, FHitResult& OutHit)
{
	UPrimitiveComponent* UpdatedPrimitive = MovingComps.UpdatedPrimitive.Get();
	if (!UpdatedPrimitive)
	{
		OutHit.Reset(1.f);
		return false;
	}

	FVector ConstrainedDelta = TargetLocation - StartLocation;

	if (MovingComps.MoverComponent.IsValid())
	{
		ConstrainedDelta = UPlanarConstraintUtils::ConstrainDirectionToPlane(MovingComps.MoverComponent->GetPlanarConstraint(), ConstrainedDelta);
	}

	const AActor* OwningActor = UpdatedPrimitive->GetOwner();

	const FVector TraceStart = StartLocation;
	const FVector TraceEnd = StartLocation + ConstrainedDelta;
	float DeltaSizeSq = (TraceEnd - TraceStart).SizeSquared();				// Recalc here to account for precision loss of float addition
	const FQuat InitialRotationQuat = StartRotation;

	// If we aren't sweeping, always consider the full move allowed
	if (!bShouldSweep)
	{
		OutHit.Init(TraceStart, TraceEnd);
		return true;
	}

	// ComponentSweepMulti does nothing if moving < KINDA_SMALL_NUMBER in distance, so it's important to not try to sweep distances smaller than that. 
	constexpr float MinMovementDistSq = FMath::Square(4.f * UE_KINDA_SMALL_NUMBER);
	if (DeltaSizeSq <= MinMovementDistSq)
	{
		// Skip if no vector or rotation.
		if (TargetRotation.Equals(InitialRotationQuat, SCENECOMPONENT_QUAT_TOLERANCE))
		{
			OutHit.Init(TraceStart, TraceEnd);
			return true;
		}

		DeltaSizeSq = 0.f;
	}

	// Perform movement collision checking if needed for this actor.
	const bool bCollisionEnabled = UpdatedPrimitive->IsQueryCollisionEnabled();
	UWorld* const MyWorld = UpdatedPrimitive->GetWorld();
	if (MyWorld && bCollisionEnabled && (DeltaSizeSq > 0.f))
	{
		TArray<FHitResult> AllHits;

		TGuardValue<bool> GuardIgnoreTouches(CollisionParams.QueryParams.bIgnoreTouches, true);

		const bool bHadBlockingHit = MyWorld->SweepMultiByChannel(AllHits, TraceStart, TraceEnd, InitialRotationQuat, 
			CollisionParams.Channel, CollisionParams.Shape, CollisionParams.QueryParams, CollisionParams.ResponseParams);

		if (AllHits.Num() > 0)
		{
			const float DeltaSize = FMath::Sqrt(DeltaSizeSq);
			for (int32 HitIdx = 0; HitIdx < AllHits.Num(); HitIdx++)
			{
				PullBackHit(AllHits[HitIdx], DeltaSize);
			}
		}

		// Find the best blocking hit from AllHits
		if (bHadBlockingHit)
		{
			int32 BlockingHitIndex = INDEX_NONE;
			float BlockingHitNormalDotDelta = UE_BIG_NUMBER;

			for (int32 HitIdx = 0; HitIdx < AllHits.Num(); ++HitIdx)
			{
				const FHitResult& TestHit = AllHits[HitIdx];
				if (TestHit.bBlockingHit)
				{
					if (!ShouldIgnoreHitResult(MyWorld, TestHit, ConstrainedDelta, OwningActor, CollisionParams.MoveComponentFlags) && 
					    !UpdatedPrimitive->ShouldComponentIgnoreHitResult(TestHit, CollisionParams.MoveComponentFlags))
					{
						if (TestHit.bStartPenetrating)
						{
							// We may have multiple initial hits, and want to choose the one with the normal most opposed to our movement.
							const float NormalDotDelta = (TestHit.ImpactNormal | ConstrainedDelta);
							if (NormalDotDelta < BlockingHitNormalDotDelta)
							{
								BlockingHitNormalDotDelta = NormalDotDelta;
								BlockingHitIndex = HitIdx;
							}
						}
						else if (BlockingHitIndex == INDEX_NONE)
						{
							// First non-overlapping blocking hit should be used, if an overlapping hit was not.
							// This should be the only non-overlapping blocking hit, and last in the results.
							BlockingHitIndex = HitIdx;
							break;
						}
					}
				}
			}

			if (BlockingHitIndex != INDEX_NONE)
			{
				OutHit = AllHits[BlockingHitIndex];
				return OutHit.Time > 0.f;	// consider there to be no movement if the blocking hit started immediately (aka start in penetration)
			}
		}
	}

	// No blocking hits occurred, so the full movement was allowed
	OutHit.Init(TraceStart, TraceEnd);
	return true;

}

