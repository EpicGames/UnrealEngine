// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoveLibrary/BasedMovementUtils.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/RollbackBlackboardLibrary.h"
#include "Components/PrimitiveComponent.h"
#include "MoverComponent.h"
#include "MoverLog.h"
#include "Kismet/KismetMathLibrary.h"
#include "DefaultMovementSet/Settings/CommonLegacyMovementSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BasedMovementUtils)

void FRelativeBaseInfo::Clear()
{
	MovementBase = nullptr;
	BoneName = NAME_None;
	Location = FVector::ZeroVector;
	Rotation = FQuat::Identity;
	ContactLocalPosition = FVector::ZeroVector;
	WorldspaceOffsetFromContactPos = FVector::ZeroVector;
}

bool FRelativeBaseInfo::HasRelativeInfo() const
{
	return MovementBase != nullptr;
}

bool FRelativeBaseInfo::UsesSameBase(const FRelativeBaseInfo& Other) const
{
	return UsesSameBase(Other.MovementBase.Get(), Other.BoneName);
}

bool FRelativeBaseInfo::UsesSameBase(const UPrimitiveComponent* OtherComp, FName OtherBoneName) const
{
	return HasRelativeInfo()
		&& (MovementBase == OtherComp)
		&& (BoneName == OtherBoneName);
}

void FRelativeBaseInfo::SetFromFloorResult(const FFloorCheckResult& FloorTestResult, const FVector ObjectWorldLocation, const FVector UpDir)
{
	bool bDidSucceed = false;

	if (FloorTestResult.bWalkableFloor)
	{
		MovementBase = FloorTestResult.HitResult.GetComponent();

		if (MovementBase.IsValid())
		{
			BoneName = FloorTestResult.HitResult.BoneName;

			if (UBasedMovementUtils::GetMovementBaseTransform(MovementBase.Get(), BoneName, OUT Location, OUT Rotation))
			{
				// Projecting onto the trace line, to account for a flat bottom base that does not have a reliable impact point directly under the object
				FVector ContactPositionInWorldSpace = FMath::ClosestPointOnInfiniteLine(FloorTestResult.HitResult.TraceStart, 
																						FloorTestResult.HitResult.TraceEnd,
																						FloorTestResult.HitResult.ImpactPoint);

				if (UBasedMovementUtils::TransformWorldLocationToBased(MovementBase.Get(), BoneName, ContactPositionInWorldSpace, OUT ContactLocalPosition))
				{
					WorldspaceOffsetFromContactPos = (ObjectWorldLocation - ContactPositionInWorldSpace).ProjectOnToNormal(UpDir);
					bDidSucceed = true;
				}
			}
		}
	}

	if (!bDidSucceed)
	{
		Clear();
	}
}

void FRelativeBaseInfo::UpdateToLatestBaseTransform()
{
	if (!UBasedMovementUtils::GetMovementBaseTransform(MovementBase.Get(), BoneName, OUT Location, OUT Rotation))
	{
		Clear();
	}
}


FString FRelativeBaseInfo::ToString() const
{
	if (MovementBase.IsValid())
	{
		return FString::Printf(TEXT("Base: %s, Loc: %s, Rot: %s, LocalContact: %s, WSContactOffset: %s"),
			*GetNameSafe(MovementBase->GetOwner()),
			*Location.ToCompactString(),
			*Rotation.Rotator().ToCompactString(),
			*ContactLocalPosition.ToCompactString(),
			*WorldspaceOffsetFromContactPos.ToCompactString());
	}

	return FString(TEXT("Base: NULL"));
}

bool UBasedMovementUtils::IsADynamicBase(const UPrimitiveComponent* MovementBase)
{
	return (MovementBase && MovementBase->Mobility == EComponentMobility::Movable);
}

bool UBasedMovementUtils::IsBaseSimulatingPhysics(const UPrimitiveComponent* MovementBase)
{
	bool bBaseIsSimulatingPhysics = false;
	const USceneComponent* AttachParent = MovementBase;
	while (!bBaseIsSimulatingPhysics && AttachParent)
	{
		bBaseIsSimulatingPhysics = AttachParent->IsSimulatingPhysics();
		AttachParent = AttachParent->GetAttachParent();
	}
	return bBaseIsSimulatingPhysics;
}


bool UBasedMovementUtils::GetMovementBaseTransform(const UPrimitiveComponent* MovementBase, const FName BoneName, FVector& OutLocation, FQuat& OutQuat)
{
	if (MovementBase)
	{
		bool bBoneNameIsInvalid = false;

		if (BoneName != NAME_None)
		{
			// Check if this socket or bone exists (DoesSocketExist checks for either, as does requesting the transform).
			if (MovementBase->DoesSocketExist(BoneName))
			{
				MovementBase->GetSocketWorldLocationAndRotation(BoneName, OutLocation, OutQuat);
				return true;
			}

			bBoneNameIsInvalid = true;
			UE_LOGF(LogMover, Warning, "GetMovementBaseTransform(): Invalid bone or socket '%ls' for PrimitiveComponent base %ls. Using component's root transform instead.", *BoneName.ToString(), *GetPathNameSafe(MovementBase));
		}

		OutLocation = MovementBase->GetComponentLocation();
		OutQuat = MovementBase->GetComponentQuat();
		return !bBoneNameIsInvalid;
	}

	// nullptr MovementBase
	OutLocation = FVector::ZeroVector;
	OutQuat = FQuat::Identity;
	return false;
}

bool UBasedMovementUtils::GetMovementBaseVelocity(const UPrimitiveComponent* MovementBase, FVector& OutWorldVelocity)
{
	if (MovementBase)
	{
		if (const UMoverComponent* BaseMoverComp = MovementBase->GetOwner()->GetComponentByClass<UMoverComponent>())
		{
			OutWorldVelocity = BaseMoverComp->GetVelocity();
		}
		else
		{
			OutWorldVelocity = MovementBase->GetComponentVelocity();
		}
		// TODO: prefer IMovementInterface once it exists, as well as UMovementComponent

		return true;
	}

	OutWorldVelocity = FVector::ZeroVector;
	return false;
}

bool UBasedMovementUtils::GetMovementBaseVelocityAtPoint(const UPrimitiveComponent* MovementBase, const FVector& WorldPoint, FVector& OutWorldVelocity)
{
	if (MovementBase)
	{
		OutWorldVelocity = MovementBase->GetPhysicsLinearVelocityAtPoint(WorldPoint);
		return true;
	}

	OutWorldVelocity = FVector::ZeroVector;
	return false;
}


bool UBasedMovementUtils::TransformBasedLocationToWorld(const UPrimitiveComponent* MovementBase, const FName BoneName, FVector LocalLocation, FVector& OutLocationWorldSpace)
{
	FVector BaseLocation;
	FQuat BaseQuat;
	
	if (GetMovementBaseTransform(MovementBase, BoneName, /*out*/ BaseLocation, /*out*/ BaseQuat))
	{ 
		TransformLocationToWorld(BaseLocation, BaseQuat, LocalLocation, OutLocationWorldSpace);
		return true;
	}
	
	return false;
}


bool UBasedMovementUtils::TransformWorldLocationToBased(const UPrimitiveComponent* MovementBase, const FName BoneName, FVector WorldSpaceLocation, FVector& OutLocalLocation)
{
	FVector BaseLocation;
	FQuat BaseQuat;
	if (GetMovementBaseTransform(MovementBase, BoneName, /*out*/ BaseLocation, /*out*/ BaseQuat))
	{
		TransformLocationToLocal(BaseLocation, BaseQuat, WorldSpaceLocation, OutLocalLocation);
		return true;
	}

	return false;
}


bool UBasedMovementUtils::TransformBasedDirectionToWorld(const UPrimitiveComponent* MovementBase, const FName BoneName, FVector LocalDirection, FVector& OutDirectionWorldSpace)
{
	FVector IgnoredLocation;
	FQuat BaseQuat;
	if (GetMovementBaseTransform(MovementBase, BoneName, /*out*/ IgnoredLocation, /*out*/ BaseQuat))
	{
		TransformDirectionToWorld(BaseQuat, LocalDirection, OutDirectionWorldSpace);
		return true;
	}

	return false;
}


bool UBasedMovementUtils::TransformWorldDirectionToBased(const UPrimitiveComponent* MovementBase, const FName BoneName, FVector WorldSpaceDirection, FVector& OutLocalDirection)
{
	FVector IgnoredLocation;
	FQuat BaseQuat;
	if (GetMovementBaseTransform(MovementBase, BoneName, /*out*/ IgnoredLocation, /*out*/ BaseQuat))
	{
		TransformDirectionToLocal(BaseQuat, WorldSpaceDirection, OutLocalDirection);
		return true;
	}

	return false;
}


bool UBasedMovementUtils::TransformBasedRotatorToWorld(const UPrimitiveComponent* MovementBase, const FName BoneName, FRotator LocalRotator, FRotator& OutWorldSpaceRotator)
{
	FVector IgnoredLocation;
	FQuat BaseQuat;
	
	if (GetMovementBaseTransform(MovementBase, BoneName, /*out*/ IgnoredLocation, /*out*/ BaseQuat))
	{
		TransformRotatorToWorld(BaseQuat, LocalRotator, OutWorldSpaceRotator);
		return true;
	}

	return false;
}


bool UBasedMovementUtils::TransformWorldRotatorToBased(const UPrimitiveComponent* MovementBase, const FName BoneName, FRotator WorldSpaceRotator, FRotator& OutLocalRotator)
{
	FVector IgnoredLocation;
	FQuat BaseQuat;
	if (GetMovementBaseTransform(MovementBase, BoneName, /*out*/ IgnoredLocation, /*out*/ BaseQuat))
	{
		TransformRotatorToLocal(BaseQuat, WorldSpaceRotator, OutLocalRotator);
		return true;
	}
	return false;
}


void UBasedMovementUtils::TransformLocationToWorld(FVector BasePos, FQuat BaseQuat, FVector LocalLocation, FVector& OutLocationWorldSpace)
{
	OutLocationWorldSpace = FTransform(BaseQuat, BasePos).TransformPositionNoScale(LocalLocation);
}

void UBasedMovementUtils::TransformLocationToLocal(FVector BasePos, FQuat BaseQuat, FVector WorldSpaceLocation, FVector& OutLocalLocation)
{
	OutLocalLocation = FTransform(BaseQuat, BasePos).InverseTransformPositionNoScale(WorldSpaceLocation);
}

void UBasedMovementUtils::TransformDirectionToWorld(FQuat BaseQuat, FVector LocalDirection, FVector& OutDirectionWorldSpace)
{
	OutDirectionWorldSpace = BaseQuat.RotateVector(LocalDirection);
}

void UBasedMovementUtils::TransformDirectionToLocal(FQuat BaseQuat, FVector WorldSpaceDirection, FVector& OutLocalDirection)
{
	OutLocalDirection = BaseQuat.UnrotateVector(WorldSpaceDirection);
}

void UBasedMovementUtils::TransformRotatorToWorld(FQuat BaseQuat, FRotator LocalRotator, FRotator& OutWorldSpaceRotator)
{
	FQuat LocalQuat(LocalRotator);
	OutWorldSpaceRotator = (BaseQuat * LocalQuat).Rotator();
}

void UBasedMovementUtils::TransformRotatorToLocal(FQuat BaseQuat, FRotator WorldSpaceRotator, FRotator& OutLocalRotator)
{
	FQuat WorldQuat(WorldSpaceRotator);
	OutLocalRotator = (BaseQuat.Inverse() * WorldQuat).Rotator();
}

FRelativeBaseInfo UBasedMovementUtils::ApplyMovementBaseChange(UMoverComponent* TargetMoverComp, const FRelativeBaseInfo& PriorBaseInfo, bool bIgnoreBaseRotation)
{
	// Early out with no changes if this isn't a valid base
	if (!PriorBaseInfo.HasRelativeInfo())
	{
		return PriorBaseInfo;
	}

	FRelativeBaseInfo CurrentBaseInfo = PriorBaseInfo;
	CurrentBaseInfo.UpdateToLatestBaseTransform();

	if (!CurrentBaseInfo.HasRelativeInfo())
	{
		return CurrentBaseInfo;
	}

	// Calculate new transform matrix of base actor (ignoring scale).
	const FQuatRotationTranslationMatrix CurrentBaseLocalToWorld(CurrentBaseInfo.Rotation, CurrentBaseInfo.Location);

	// Find where the actor *should* be
	// NOTE that we are using a combination of the relative floor contact location, plus a worldspace offset. 
	// This is so the location we're touching the base is unchanging, and the actor location always stays above the 
	// base, even if the base tilts and heaves like a rocking boat.
	const FVector CurrentWorldBaseContactPos = CurrentBaseLocalToWorld.TransformPosition(CurrentBaseInfo.ContactLocalPosition);
	const FVector TargetWorldPos = CurrentWorldBaseContactPos + CurrentBaseInfo.WorldspaceOffsetFromContactPos;

	USceneComponent* UpdatedComponent = TargetMoverComp->GetUpdatedComponent();
	const FVector OldWorldPos = UpdatedComponent->GetComponentLocation();
	const FVector WorldBaseMovementDelta = TargetWorldPos - OldWorldPos;

	FQuat WorldTargetQuat = UpdatedComponent->GetComponentQuat();

	// Find any change in local rotation if a rotating base should cause a similar rotation in the pawn
	if (!bIgnoreBaseRotation && !CurrentBaseInfo.Rotation.Equals(PriorBaseInfo.Rotation, UE_SMALL_NUMBER))
	{
		// Attempting to match the platform's rotation as best as possible, but respect the up direction so the character
		// stays oriented upright even if the platform pitches / heaves / tilts like a rocking boat.

		FQuat DeltaQuat = CurrentBaseInfo.Rotation * PriorBaseInfo.Rotation.Inverse();
		WorldTargetQuat = DeltaQuat * WorldTargetQuat;

		FVector TargetForwVector = WorldTargetQuat.GetForwardVector();
		TargetForwVector = FVector::VectorPlaneProject(TargetForwVector, -TargetMoverComp->GetUpDirection());
		TargetForwVector.Normalize();

		FVector TargetRightVector = WorldTargetQuat.GetRightVector();
		TargetRightVector = FVector::VectorPlaneProject(TargetRightVector, -TargetMoverComp->GetUpDirection());
		TargetRightVector.Normalize();

		WorldTargetQuat = UKismetMathLibrary::MakeRotFromXY(TargetForwVector, TargetRightVector).Quaternion();
	}

	const FTransform OldTransform = UpdatedComponent->GetComponentTransform();
	FTransform NewTransform = OldTransform;

	if (!WorldBaseMovementDelta.IsZero() || WorldTargetQuat != UpdatedComponent->GetComponentQuat())
	{
		EMoveComponentFlags MoveComponentFlags = MOVECOMP_IgnoreBases | MOVECOMP_DisableBlockingOverlapDispatch;
		const bool bSweep = true;	// we need this to be a sweeping movement in case the base movement pushes us into an obstacle... we need to get scraped off
		FHitResult MoveHitResult;

		bool bDidMove = UMovementUtils::TryMoveUpdatedComponent_Internal(FMovingComponentSet(TargetMoverComp), WorldBaseMovementDelta, WorldTargetQuat, bSweep, MoveComponentFlags, &MoveHitResult, ETeleportType::None);

		NewTransform = UpdatedComponent->GetComponentTransform();
		const FVector NewWorldPos = NewTransform.GetLocation();

		// If we did not move the full amount, that means we are blocked and the base is moving along without us.
		// We're getting scraped along our base, so adjust our contact point backwards to account for it.
		const FVector RemainingWorldDelta = (OldWorldPos + WorldBaseMovementDelta) - NewWorldPos;

		if (!RemainingWorldDelta.IsNearlyZero())
		{
			// Find the remaining delta that wasn't achieved
			// Convert the remaining delta to current base space
			FVector RemainingLocalDelta;
			UBasedMovementUtils::TransformDirectionToLocal(CurrentBaseInfo.Rotation, RemainingWorldDelta, OUT RemainingLocalDelta);

			// Subtract the remaining delta to reflect the change in the contact position
			CurrentBaseInfo.ContactLocalPosition -= RemainingLocalDelta;
		}
		
		// Track accumulated based movement transform deltas as they occur
		FRollbackBlackboardExternalWrapper RollbackBlackboard = TargetMoverComp->GetRollbackBlackboardExternal();

		FTransform AccumulatedTransformDelta = FTransform::Identity;

		if (!RollbackBlackboard.TryGet(CommonBlackboard::AccumulatedBasedTransformDelta, AccumulatedTransformDelta))
		{
			if (!RollbackBlackboard.HasEntry(CommonBlackboard::AccumulatedBasedTransformDelta))
			{
				URollbackBlackboard::EntrySettings AccumTransformDeltaSettings = URollbackBlackboardLibrary::MakeSingleFrameEntrySettings();
				AccumTransformDeltaSettings.PersistencePolicy = EBlackboardPersistencePolicy::ThroughNextFrame;
				RollbackBlackboard.CreateEntry<FTransform>(CommonBlackboard::AccumulatedBasedTransformDelta, AccumTransformDeltaSettings);
			}
		}

		const FVector DiffInWorldPos = NewTransform.GetLocation() - OldTransform.GetLocation();
		const FQuat DiffInWorldQuat = NewTransform.GetRotation() * OldTransform.GetRotation().Inverse();

		AccumulatedTransformDelta.AddToTranslation(DiffInWorldPos);
		AccumulatedTransformDelta.SetRotation((DiffInWorldQuat * AccumulatedTransformDelta.GetRotation()).GetNormalized());

		RollbackBlackboard.TrySet(CommonBlackboard::AccumulatedBasedTransformDelta, AccumulatedTransformDelta);
	}

	return CurrentBaseInfo;
}

void UBasedMovementUtils::AddTickDependency(FTickFunction& BasedObjectTick, UPrimitiveComponent* NewBase)
{
	if (NewBase && IsADynamicBase(NewBase))
	{
		if (NewBase->PrimaryComponentTick.bCanEverTick)
		{
			BasedObjectTick.AddPrerequisite(NewBase, NewBase->PrimaryComponentTick);
		}

		AActor* NewBaseOwner = NewBase->GetOwner();
		if (NewBaseOwner)
		{
			if (NewBaseOwner->PrimaryActorTick.bCanEverTick)
			{
				BasedObjectTick.AddPrerequisite(NewBaseOwner, NewBaseOwner->PrimaryActorTick);
			}

			// @TODO: We need to find a more efficient way of finding all ticking components in an actor.
			for (UActorComponent* Component : NewBaseOwner->GetComponents())
			{
				// Dont allow a based component (e.g. a particle system) to push us into a different tick group
				if (Component && Component->PrimaryComponentTick.bCanEverTick && Component->PrimaryComponentTick.TickGroup <= BasedObjectTick.TickGroup)
				{
					BasedObjectTick.AddPrerequisite(Component, Component->PrimaryComponentTick);
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogMover, Warning, "Attempted to AddTickDependency on an invalid or non-dynamic base: %ls", *GetNameSafe(NewBase));
	}
}

void UBasedMovementUtils::RemoveTickDependency(FTickFunction& BasedObjectTick, UPrimitiveComponent* OldBase)
{
	if (OldBase)
	{
		BasedObjectTick.RemovePrerequisite(OldBase, OldBase->PrimaryComponentTick);
		
		if (AActor* OldBaseOwner = OldBase->GetOwner())
		{
			BasedObjectTick.RemovePrerequisite(OldBaseOwner, OldBaseOwner->PrimaryActorTick);

			// @TODO: We need to find a more efficient way of finding all ticking components in an actor.
			for (UActorComponent* Component : OldBaseOwner->GetComponents())
			{
				if (Component && Component->PrimaryComponentTick.bCanEverTick)
				{
					BasedObjectTick.RemovePrerequisite(Component, Component->PrimaryComponentTick);
				}
			}
		}
	}
}


void UBasedMovementUtils::UpdateSimpleBasedMovement(UMoverComponent* TargetMoverComp)
{
	if (!TargetMoverComp)
	{
		return;
	}

	UMoverBlackboard* SimBlackboard = TargetMoverComp->GetSimBlackboard_Mutable();
	USceneComponent* UpdatedComponent = TargetMoverComp->UpdatedComponent;

	bool bIgnoreBaseRotation = false;

	if (const UCommonLegacyMovementSettings* CommonSettings = TargetMoverComp->FindSharedSettings<UCommonLegacyMovementSettings>())
	{
		bIgnoreBaseRotation = CommonSettings->bIgnoreBaseRotation;
	}

	bool bDidPerformBasedMovement = false;
	ON_SCOPE_EXIT
	{
		if (!bDidPerformBasedMovement)
		{
			SimBlackboard->Invalidate(CommonBlackboard::LastFoundDynamicMovementBase);
		}
	};



	FRelativeBaseInfo PriorBaseInfo;		// This is the snapshot of the base where we last used it

	const bool bHasPriorBaseInfo = SimBlackboard->TryGet(CommonBlackboard::LastFoundDynamicMovementBase, PriorBaseInfo);

	
	if (!ensureMsgf(bHasPriorBaseInfo, TEXT("No information about the prior movement base info. Nothing we can do.")))
	{
		return;
	}



	FRelativeBaseInfo CurrentBaseInfo = UBasedMovementUtils::ApplyMovementBaseChange(TargetMoverComp, PriorBaseInfo, bIgnoreBaseRotation);


	// Propagate the movement changes to the backend's state, if supported

	// Note that this is occurring out-of-band with the movement simulation, in order to support based movement regardless of update order or
	// whether the movement base is also simulated through Mover.

	FMoverSyncState PendingSimSyncState;
	if (TargetMoverComp->BackendLiaisonComp->ReadPendingSyncState(OUT PendingSimSyncState))
	{
		// Modify the PENDING sync state that has not yet been committed to simulation history nor replicated
		if (FMoverDefaultSyncState* PendingMoverState = PendingSimSyncState.SyncStateCollection.FindMutableDataByType<FMoverDefaultSyncState>())
		{
			FTransform OldSyncTransformWs = PendingMoverState->GetTransform_WorldSpace();
			FTransform NewSyncTransformWs = UpdatedComponent->GetComponentTransform();

			PendingMoverState->SetTransforms_WorldSpace(
				NewSyncTransformWs.GetLocation(),
				NewSyncTransformWs.GetRotation().Rotator(),
				PendingMoverState->GetVelocity_WorldSpace(),	// keep same velocity and base
				PendingMoverState->GetAngularVelocityDegrees_WorldSpace(),
				PendingMoverState->GetMovementBase(), PendingMoverState->GetMovementBaseBoneName());

			TargetMoverComp->BackendLiaisonComp->WritePendingSyncState(PendingSimSyncState);	// writes pending Simulation state

			// If smoothing, modify presentation-related states as well so that the visual offset location stays anchored to the movement base
			if (TargetMoverComp->SmoothingMode != EMoverSmoothingMode::None)
			{
				const FTransform OldToNewTransform = NewSyncTransformWs.GetRelativeTransform(OldSyncTransformWs);

				// Modify the PRESENTATION sync state that we're smoothing TO
				FMoverSyncState PresentationSyncState;
				if (TargetMoverComp->BackendLiaisonComp->ReadPresentationSyncState(OUT PresentationSyncState))
				{
					if (FMoverDefaultSyncState* PresentationMoverState = PresentationSyncState.SyncStateCollection.FindMutableDataByType<FMoverDefaultSyncState>())
					{
						OldSyncTransformWs = PresentationMoverState->GetTransform_WorldSpace();
						NewSyncTransformWs = OldToNewTransform * OldSyncTransformWs;

						PresentationMoverState->SetTransforms_WorldSpace(
							NewSyncTransformWs.GetLocation(),
							NewSyncTransformWs.GetRotation().Rotator(),
							PresentationMoverState->GetVelocity_WorldSpace(),	// keep same velocity and base
							PresentationMoverState->GetAngularVelocityDegrees_WorldSpace(),
							PresentationMoverState->GetMovementBase(), PresentationMoverState->GetMovementBaseBoneName());

						TargetMoverComp->BackendLiaisonComp->WritePresentationSyncState(PresentationSyncState);
					}
				}

				// Modify the PREV PRESENTATION sync state that we're smoothing FROM
				FMoverSyncState PrevPresentationSyncState;
				if (TargetMoverComp->BackendLiaisonComp->ReadPrevPresentationSyncState(OUT PrevPresentationSyncState))
				{
					if (FMoverDefaultSyncState* PrevPresentationMoverState = PrevPresentationSyncState.SyncStateCollection.FindMutableDataByType<FMoverDefaultSyncState>())
					{
						OldSyncTransformWs = PrevPresentationMoverState->GetTransform_WorldSpace();
						NewSyncTransformWs = OldToNewTransform * OldSyncTransformWs;

						PrevPresentationMoverState->SetTransforms_WorldSpace(
							NewSyncTransformWs.GetLocation(),
							NewSyncTransformWs.GetRotation().Rotator(),
							PrevPresentationMoverState->GetVelocity_WorldSpace(),	// keep same velocity and base
							PrevPresentationMoverState->GetAngularVelocityDegrees_WorldSpace(),
							PrevPresentationMoverState->GetMovementBase(), PrevPresentationMoverState->GetMovementBaseBoneName());

						TargetMoverComp->BackendLiaisonComp->WritePrevPresentationSyncState(PrevPresentationSyncState);
					}
				}
			}
		}
	}

	SimBlackboard->Set(CommonBlackboard::LastFoundDynamicMovementBase, CurrentBaseInfo);
	bDidPerformBasedMovement = true;
	


	// Emit an event on the MoverComponent to surface the accumulated transform delta of the actor applied solely by based movement, once per sim frame
	FRollbackBlackboardExternalWrapper RollbackBlackboard = TargetMoverComp->GetRollbackBlackboardExternal(); 	
	FMoverTimeStep CurrentTimeStep = TargetMoverComp->GetLastTimeStep();
	FMoverTimeStep LastBasedMovementAppliedEventTimeStep;

	if (!RollbackBlackboard.TryGet(CommonBlackboard::LastBasedMovementAppliedEventTime, LastBasedMovementAppliedEventTimeStep))
	{
		URollbackBlackboard::EntrySettings LastAppliedEvtSettings = URollbackBlackboardLibrary::MakeSingleFrameEntrySettings();
		LastAppliedEvtSettings.PersistencePolicy = EBlackboardPersistencePolicy::Forever;
		RollbackBlackboard.CreateEntry<FMoverTimeStep>(CommonBlackboard::LastBasedMovementAppliedEventTime, LastAppliedEvtSettings);

		RollbackBlackboard.TrySet(CommonBlackboard::LastBasedMovementAppliedEventTime, CurrentTimeStep);
	}

	// Emit the event and clear accumulation only when the frame has advanced
	if (CurrentTimeStep.ServerFrame > LastBasedMovementAppliedEventTimeStep.ServerFrame)
	{
		FTransform AccumulatedTransformDelta = FTransform::Identity;
		RollbackBlackboard.TryGet(CommonBlackboard::AccumulatedBasedTransformDelta, AccumulatedTransformDelta);

		TargetMoverComp->OnBasedMovementApplied.Broadcast(AccumulatedTransformDelta, CurrentTimeStep);

		RollbackBlackboard.TrySet(CommonBlackboard::AccumulatedBasedTransformDelta, FTransform::Identity);
		RollbackBlackboard.TrySet(CommonBlackboard::LastBasedMovementAppliedEventTime, CurrentTimeStep);
	}
}



// FMoverDynamicBasedMovementTickFunction ////////////////////////////////////

void FMoverDynamicBasedMovementTickFunction::ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	FActorComponentTickFunction::ExecuteTickHelper(TargetMoverComp, /*bTickInEditor=*/ false, DeltaTime, TickType, [this](float DilatedTime)
		{
			UBasedMovementUtils::UpdateSimpleBasedMovement(TargetMoverComp);
		});

	if (bAutoDisableAfterTick)
	{
		SetTickFunctionEnable(false);
	}
}
FString FMoverDynamicBasedMovementTickFunction::DiagnosticMessage()
{
	return TargetMoverComp->GetFullName() + TEXT("[FMoverDynamicBasedMovementTickFunction]");
}
FName FMoverDynamicBasedMovementTickFunction::DiagnosticContext(bool bDetailed)
{
	if (bDetailed)
	{
		return FName(*FString::Printf(TEXT("UMoverComponent/%s"), *GetFullNameSafe(TargetMoverComp)));
	}
	return FName(TEXT("FMoverDynamicBasedMovementTickFunction"));
}

