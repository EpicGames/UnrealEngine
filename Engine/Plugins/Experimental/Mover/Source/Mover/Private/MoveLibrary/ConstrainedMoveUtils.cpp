// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoveLibrary/ConstrainedMoveUtils.h"
#include "MoverLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConstrainedMoveUtils)


void UPlanarConstraintUtils::SetPlanarConstraintEnabled(UPARAM(ref) FPlanarConstraint& Constraint, bool bEnabled)
{
	Constraint.bConstrainToPlane = bEnabled;
}


void UPlanarConstraintUtils::SetPlanarConstraintNormal(UPARAM(ref) FPlanarConstraint& Constraint, FVector PlaneNormal)
{
	PlaneNormal = PlaneNormal.GetSafeNormal();

	if (PlaneNormal.IsNearlyZero())
	{
		UE_LOGF(LogMover, Warning, "Can't use SetPlanarConstraintNormal with a zero-length normal. Leaving normal as %ls", *Constraint.PlaneConstraintNormal.ToCompactString());
	}
	else
	{
		Constraint.PlaneConstraintNormal = PlaneNormal;
	}
}


void UPlanarConstraintUtils::SetPlaneConstraintOrigin(UPARAM(ref) FPlanarConstraint& Constraint, FVector PlaneOrigin)
{
	Constraint.PlaneConstraintOrigin = PlaneOrigin;
}


FVector UPlanarConstraintUtils::ConstrainDirectionToPlane(const FPlanarConstraint& Constraint, FVector Direction, bool bMaintainMagnitude)
{
	if (Constraint.bConstrainToPlane)
	{
		float OrigSize = Direction.Size();

		Direction = FVector::VectorPlaneProject(Direction, Constraint.PlaneConstraintNormal);

		if (bMaintainMagnitude)
		{
			Direction = Direction.GetSafeNormal() * OrigSize;
		}
	}

	return Direction;
}

FVector UPlanarConstraintUtils::ConstrainLocationToPlane(const FPlanarConstraint& Constraint, FVector Location)
{
	if (Constraint.bConstrainToPlane)
	{
		Location = FVector::PointPlaneProject(Location, Constraint.PlaneConstraintOrigin, Constraint.PlaneConstraintNormal);
	}

	return Location;
}

FVector UPlanarConstraintUtils::ConstrainNormalToPlane(const FPlanarConstraint& Constraint, FVector Normal)
{
	if (Constraint.bConstrainToPlane)
	{
		Normal = FVector::VectorPlaneProject(Normal, Constraint.PlaneConstraintNormal).GetSafeNormal();
	}

	return Normal;
}
