// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoveLibrary/FloorQueryUtils.h"

#include "Engine/World.h"
#include "Components/CapsuleComponent.h"
#include "DefaultMovementSet/Settings/CommonLegacyMovementSettings.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoverComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FloorQueryUtils)

namespace UE::FloorQueryUtility
{

	const float MIN_FLOOR_DIST = 1.9f;	// Smallest distance we want our primitive floating above walkable floors while in ground-based movement.
	const float MAX_FLOOR_DIST = 2.4f;	// Largest distance we want our primitive floating above walkable floors while in ground-based movement.

	const float SWEEP_EDGE_REJECT_DISTANCE = 0.15f;
}

void UFloorQueryUtils::FindFloor(const FMovingComponentSet& MovingComps, const FFloorCheckSettings& FloorCheckSettings, const FVector& Location, FFloorCheckResult& OutFloorResult)
{
	if (!MovingComps.UpdatedComponent->IsQueryCollisionEnabled())
	{
		OutFloorResult.Clear();
		return;
	}

	// Sweep for the floor
	// TODO: Might need to plug in a different value for LineTraceDistance - using the same value as FloorSweepDistance for now - function takes both so we can plug in different values if needed
	ComputeFloorDist(MovingComps, FloorCheckSettings, FloorCheckSettings.FloorSweepDistance, Location, OutFloorResult);

	// Perch validation: when using a capsule sweep (not flat base), the sweep uses the full capsule radius
	// and doesn't account for PerchRadiusThreshold. We need to do a second sweep with a reduced radius
	// to verify the character can actually stand at this location. This approximates CMC's ComputePerchResult approach.
	// When using flat base checks, the box shape is already shrunk by PerchRadiusThreshold in FloorSweepTest,
	// so no additional validation is needed.
	if (FloorCheckSettings.PerchRadiusThreshold > 0.f
		&& !FloorCheckSettings.bUseFlatBaseForFloorChecks
		&& OutFloorResult.bBlockingHit
		&& !OutFloorResult.bLineTrace)
	{
		float PawnRadius = 0.0f;
		float PawnHalfHeight = 0.0f;
		if (const UCapsuleComponent* CapsuleComponent = Cast<UCapsuleComponent>(MovingComps.UpdatedComponent))
		{
			CapsuleComponent->GetScaledCapsuleSize(PawnRadius, PawnHalfHeight);
		}
		else if (MovingComps.UpdatedPrimitive.IsValid())
		{
			MovingComps.UpdatedPrimitive->CalcBoundingCylinder(PawnRadius, PawnHalfHeight);
		}

		const float ValidPerchRadius = FMath::Clamp(PawnRadius - FloorCheckSettings.PerchRadiusThreshold, 0.1f, PawnRadius);
		const FVector UpDirection = MovingComps.MoverComponent->GetUpDirection();

		// Check if the impact point is outside the valid perch radius (i.e., near the capsule edge).
		// Use HitResult.Location (capsule position at hit time) for the distance calculation.
		const float DistFromCenterSq = FVector::VectorPlaneProject((OutFloorResult.HitResult.ImpactPoint - OutFloorResult.HitResult.Location), -UpDirection).SizeSquared();

		if (DistFromCenterSq > FMath::Square(ValidPerchRadius))
		{
			// The main sweep hit something near the capsule edge. Do a second sweep with the reduced
			// perch radius to see if there's actually walkable floor under the character's footprint.
			const float MaxPerchFloorDist = FMath::Max(UE::FloorQueryUtility::MAX_FLOOR_DIST, FloorCheckSettings.FloorSweepDistance);
			const float PerchSweepDist = MaxPerchFloorDist + PawnRadius;

			FCollisionQueryParams QueryParams(SCENE_QUERY_STAT(ComputePerchResult), false, MovingComps.UpdatedPrimitive->GetOwner());
			FCollisionResponseParams ResponseParam;
			UMovementUtils::InitCollisionParams(MovingComps.UpdatedPrimitive.Get(), QueryParams, ResponseParam);
			const ECollisionChannel CollisionChannel = MovingComps.UpdatedComponent->GetCollisionObjectType();

			const float ShrinkHeight = FMath::Max(0.f, PawnHalfHeight - ValidPerchRadius);
			const FCollisionShape PerchCapsuleShape = FCollisionShape::MakeCapsule(ValidPerchRadius, PawnHalfHeight - ShrinkHeight);
			const FVector SweepStart = OutFloorResult.HitResult.Location;
			const FVector SweepEnd = SweepStart + UpDirection * -(PerchSweepDist + ShrinkHeight);

			FHitResult PerchHit(1.f);
			const bool bPerchHit = MovingComps.UpdatedComponent->GetWorld()->SweepSingleByChannel(
				PerchHit, SweepStart, SweepEnd,
				FRotationMatrix::MakeFromZX(UpDirection, FVector::ForwardVector).ToQuat(),
				CollisionChannel, PerchCapsuleShape, QueryParams, ResponseParam);

			if (!bPerchHit || !PerchHit.IsValidBlockingHit() || !IsHitSurfaceWalkable(PerchHit, UpDirection, FloorCheckSettings.MaxWalkSlopeCosine))
			{
				OutFloorResult.bWalkableFloor = false;
			}
		}
	}
}

void UFloorQueryUtils::FindFloor(const FMovingComponentSet& MovingComps, float FloorSweepDistance, float MaxWalkSlopeCosine, bool bUseFlatBaseForFloorChecks, const FVector& Location, FFloorCheckResult& OutFloorResult, float PerchRadiusThreshold)
{
	FFloorCheckSettings Settings;
	Settings.FloorSweepDistance = FloorSweepDistance;
	Settings.MaxWalkSlopeCosine = MaxWalkSlopeCosine;
	Settings.bUseFlatBaseForFloorChecks = bUseFlatBaseForFloorChecks;
	Settings.PerchRadiusThreshold = PerchRadiusThreshold;
	FindFloor(MovingComps, Settings, Location, OutFloorResult);
}

void UFloorQueryUtils::ComputeFloorDist(const FMovingComponentSet& MovingComps, const FFloorCheckSettings& FloorCheckSettings, float LineTraceDistance, const FVector& Location, FFloorCheckResult& OutFloorResult)
{
	OutFloorResult.Clear();

	FCollisionQueryParams QueryParams(SCENE_QUERY_STAT(ComputeFloorDist), false, MovingComps.UpdatedPrimitive->GetOwner());
	FCollisionResponseParams ResponseParam;
	UMovementUtils::InitCollisionParams(MovingComps.UpdatedPrimitive.Get(), QueryParams, ResponseParam);
	const ECollisionChannel CollisionChannel = MovingComps.UpdatedComponent->GetCollisionObjectType();

	// TODO: pluggable shapes
	float PawnRadius = 0.0f;
	float PawnHalfHeight = 0.0f;
	if (UCapsuleComponent* CapsuleComponent = Cast<UCapsuleComponent>(MovingComps.UpdatedComponent))
	{
		CapsuleComponent->GetScaledCapsuleSize(PawnRadius, PawnHalfHeight);
	}
	else if (MovingComps.UpdatedPrimitive.IsValid())
	{
		MovingComps.UpdatedPrimitive->CalcBoundingCylinder(PawnRadius, PawnHalfHeight);
	}

	FVector UpDirection = MovingComps.MoverComponent->GetUpDirection();

	bool bBlockingHit = false;
	
	// Sweep test
	if (FloorCheckSettings.FloorSweepDistance > 0.f)
	{
		// Use a shorter height to avoid sweeps giving weird results if we start on a surface.
		// This also allows us to adjust out of penetrations.
		const float ShrinkScale = 0.9f;
		const float ShrinkScaleOverlap = 0.1f;
		float ShrinkHeight = (PawnHalfHeight - PawnRadius) * (1.f - ShrinkScale);
		float TraceDist = FloorCheckSettings.FloorSweepDistance + ShrinkHeight;
		FCollisionShape CapsuleShape = FCollisionShape::MakeCapsule(PawnRadius, PawnHalfHeight - ShrinkHeight);

		FHitResult Hit(1.f);
		FVector SweepDirection = UpDirection * -TraceDist;
		bBlockingHit = FloorSweepTest(MovingComps, Hit, Location, Location + SweepDirection, CollisionChannel, CapsuleShape, QueryParams, ResponseParam, FloorCheckSettings.bUseFlatBaseForFloorChecks, FloorCheckSettings.PerchRadiusThreshold);

		if (bBlockingHit)
		{
			// Reject hits adjacent to us, we only care about hits on the bottom portion of our capsule.
			// Check 2D distance to impact point, reject if within a tolerance from radius.
			if (Hit.bStartPenetrating || !IsWithinEdgeTolerance(Location, Hit.ImpactPoint, CapsuleShape.Capsule.Radius, UpDirection))
			{
				// Use a capsule with a slightly smaller radius and shorter height to avoid the adjacent object.
				// Capsule must not be nearly zero or the trace will fall back to a line trace from the start point and have the wrong length.
				CapsuleShape.Capsule.Radius = FMath::Max(0.f, CapsuleShape.Capsule.Radius - UE::FloorQueryUtility::SWEEP_EDGE_REJECT_DISTANCE - KINDA_SMALL_NUMBER);
				if (!CapsuleShape.IsNearlyZero())
				{
					ShrinkHeight = (PawnHalfHeight - PawnRadius) * (1.f - ShrinkScaleOverlap);
					TraceDist = FloorCheckSettings.FloorSweepDistance + ShrinkHeight;
					SweepDirection = UpDirection * -TraceDist;
					CapsuleShape.Capsule.HalfHeight = FMath::Max(PawnHalfHeight - ShrinkHeight, CapsuleShape.Capsule.Radius);
					Hit.Reset(1.f, false);

					bBlockingHit = FloorSweepTest(MovingComps, Hit, Location, Location + SweepDirection, CollisionChannel, CapsuleShape, QueryParams, ResponseParam, FloorCheckSettings.bUseFlatBaseForFloorChecks, FloorCheckSettings.PerchRadiusThreshold);
				}
			}

			// Reduce hit distance by ShrinkHeight because we shrank the capsule for the trace.
			// We allow negative distances here, because this allows us to pull out of penetrations.
			// JAH TODO: move magic numbers to a common location
			const float MaxPenetrationAdjust = FMath::Max(UE::FloorQueryUtility::MAX_FLOOR_DIST, PawnRadius);
			const float SweepResult = FMath::Max(-MaxPenetrationAdjust, Hit.Time * TraceDist - ShrinkHeight);

			OutFloorResult.SetFromSweep(Hit, SweepResult, false);
			if (Hit.IsValidBlockingHit() && IsHitSurfaceWalkable(Hit, UpDirection, FloorCheckSettings.MaxWalkSlopeCosine))
			{
				if (SweepResult <= FloorCheckSettings.FloorSweepDistance)
				{
					// Hit within test distance.
					OutFloorResult.bWalkableFloor = true;
					return;
				}
			}
		}
	}

	// Since we require a longer sweep than line trace, we don't want to run the line trace if the sweep missed everything.
	// We do however want to try a line trace if the sweep was stuck in penetration.
	if (!OutFloorResult.bBlockingHit && !OutFloorResult.HitResult.bStartPenetrating)
	{
		OutFloorResult.FloorDist = FloorCheckSettings.FloorSweepDistance;
		return;
	}

	// Line trace
	if (LineTraceDistance > 0.f)
	{
		const float ShrinkHeight = PawnHalfHeight;
		const FVector LineTraceStart = Location;	
		const float TraceDist = LineTraceDistance + ShrinkHeight;
		const FVector Down = UpDirection * -TraceDist;
		QueryParams.TraceTag = SCENE_QUERY_STAT_NAME_ONLY(FloorLineTrace);

		FHitResult Hit(1.f);
		bBlockingHit = MovingComps.UpdatedComponent->GetWorld()->LineTraceSingleByChannel(Hit, LineTraceStart, LineTraceStart + Down, CollisionChannel, QueryParams, ResponseParam);
		
		if (bBlockingHit && Hit.Time > 0.f)
		{
			// Reduce hit distance by ShrinkHeight because we started the trace higher than the base.
			// We allow negative distances here, because this allows us to pull out of penetrations.
			const float MaxPenetrationAdjust = FMath::Max(UE::FloorQueryUtility::MAX_FLOOR_DIST, PawnRadius);
			const float LineResult = FMath::Max(-MaxPenetrationAdjust, Hit.Time * TraceDist - ShrinkHeight);
			
			OutFloorResult.bBlockingHit = true;
			if (LineResult <= LineTraceDistance && IsHitSurfaceWalkable(Hit, UpDirection, FloorCheckSettings.MaxWalkSlopeCosine))
			{
				OutFloorResult.SetFromLineTrace(Hit, OutFloorResult.FloorDist, LineResult, true);
				return;
			}
		}
	}

	// No hits were acceptable.
	OutFloorResult.bWalkableFloor = false;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UFloorQueryUtils::ComputeFloorDist(const FMovingComponentSet& MovingComps, float LineTraceDistance, float FloorSweepDistance, float MaxWalkSlopeCosine, const FVector& Location, bool bUseFlatBaseForFloorChecks, FFloorCheckResult& OutFloorResult, float PerchRadiusThreshold)
{
	FFloorCheckSettings Settings;
	Settings.FloorSweepDistance = FloorSweepDistance;
	Settings.MaxWalkSlopeCosine = MaxWalkSlopeCosine;
	Settings.bUseFlatBaseForFloorChecks = bUseFlatBaseForFloorChecks;
	Settings.PerchRadiusThreshold = PerchRadiusThreshold;
	ComputeFloorDist(MovingComps, Settings, LineTraceDistance, Location, OutFloorResult);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool UFloorQueryUtils::FloorSweepTest(const FMovingComponentSet& MovingComps, FHitResult& OutHit, const FVector& Start, const FVector& End, ECollisionChannel TraceChannel, const struct FCollisionShape& CollisionShape, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParam, bool bUseFlatBaseForFloorChecks, float PerchRadiusThreshold)
{
	const FVector DirAwayFromFloor = (Start - End).GetSafeNormal();
	if (!MovingComps.UpdatedPrimitive.IsValid() || !DirAwayFromFloor.IsNormalized())
	{
		return false;
	}

	UWorld* World = MovingComps.UpdatedPrimitive->GetWorld();

	const FQuat UpDirOrientation = FRotationMatrix::MakeFromZX(DirAwayFromFloor, FVector::ForwardVector).ToQuat();

	bool bBlockingHit = false;

	if (bUseFlatBaseForFloorChecks)
	{
		// Test with a box that is enclosed by the capsule: 2 checks at different rotations to get a good approximation of a circular flat bottom.
		const float CapsuleRadius = CollisionShape.GetCapsuleRadius();
		const float CapsuleHeight = CollisionShape.GetCapsuleHalfHeight();
		const float EffectiveRadius = FMath::Max(0.1f, CapsuleRadius - PerchRadiusThreshold);
		const FCollisionShape BoxShape = FCollisionShape::MakeBox(FVector(EffectiveRadius * 0.707f, EffectiveRadius * 0.707f, CapsuleHeight));

		// First test with the box rotated so the corners are along the major axes (ie rotated 45 degrees).
		const FQuat Rotate45LocalYaw = FQuat::MakeFromEuler(FVector(0.0f, 0.0f, 45.0f));

		bBlockingHit = World->SweepSingleByChannel(OutHit, Start, End, UpDirOrientation * Rotate45LocalYaw, TraceChannel, BoxShape, Params, ResponseParam);

		if (!bBlockingHit)
		{
			// Test again with the same box, not rotated.
			OutHit.Reset(1.f, false);
			bBlockingHit = World->SweepSingleByChannel(OutHit, Start, End, UpDirOrientation, TraceChannel, BoxShape, Params, ResponseParam);
		}
	}
	else
	{
		bBlockingHit = World->SweepSingleByChannel(OutHit, Start, End, UpDirOrientation, TraceChannel, CollisionShape, Params, ResponseParam);
	}

	return bBlockingHit;
}

bool UFloorQueryUtils::IsHitSurfaceWalkable(const FHitResult& Hit, const FVector& UpDirection, float MaxWalkSlopeCosine)
{
	if (!Hit.IsValidBlockingHit())
	{
		// No hit, or starting in penetration
		return false;
	}

	// Never walk up vertical surfaces.
	if (Hit.ImpactNormal.Dot(UpDirection) < KINDA_SMALL_NUMBER)
	{
		return false;
	}

	float TestWalkableSlopeCosine = MaxWalkSlopeCosine;

	// See if this component overrides the walkable floor slope cosine.
	const UPrimitiveComponent* HitComponent = Hit.Component.Get();
	if (HitComponent)
	{
		const FWalkableSlopeOverride& SlopeOverride = HitComponent->GetWalkableSlopeOverride();
		TestWalkableSlopeCosine = SlopeOverride.ModifyWalkableFloorZ(TestWalkableSlopeCosine);
	}

	// Can't walk on this surface if it is too steep.
	if (Hit.ImpactNormal.Dot(UpDirection) < TestWalkableSlopeCosine)
	{
		return false;
	}

	return true;
}

bool UFloorQueryUtils::IsWithinEdgeTolerance(const FVector& CapsuleLocation, const FVector& TestImpactPoint, float CapsuleRadius, const FVector& UpDirection)
{
	const float DistFromCenterSq = FVector::VectorPlaneProject((TestImpactPoint - CapsuleLocation), -UpDirection).SizeSquared();
	const float ReducedRadiusSq = FMath::Square(FMath::Max(UE::FloorQueryUtility::SWEEP_EDGE_REJECT_DISTANCE + UE_KINDA_SMALL_NUMBER, CapsuleRadius - UE::FloorQueryUtility::SWEEP_EDGE_REJECT_DISTANCE));
	return DistFromCenterSq < ReducedRadiusSq;
}


void UFloorQueryUtils::TryFindFloor(UMoverComponent* MoverComp, OUT bool& DidFindWalkableFloor, OUT FFloorCheckResult& OutFloorResult)
{
	DidFindWalkableFloor = false;

	if (MoverComp && MoverComp->GetUpdatedComponent())
	{
		if (const UCommonLegacyMovementSettings* CommonLegacySettings = MoverComp->FindSharedSettings<UCommonLegacyMovementSettings>())
		{
			const FVector Location = MoverComp->GetUpdatedComponent()->GetComponentLocation();
			const FMovingComponentSet MovingComps(MoverComp);

			FFloorCheckSettings FloorCheckSettings;
			FloorCheckSettings.FloorSweepDistance = CommonLegacySettings->FloorSweepDistance;
			FloorCheckSettings.MaxWalkSlopeCosine = CommonLegacySettings->MaxWalkSlopeCosine;
			FloorCheckSettings.bUseFlatBaseForFloorChecks = CommonLegacySettings->bUseFlatBaseForFloorChecks;
			FloorCheckSettings.PerchRadiusThreshold = CommonLegacySettings->PerchRadiusThreshold;

			UFloorQueryUtils::FindFloor(MovingComps, FloorCheckSettings, Location, OutFloorResult);

			DidFindWalkableFloor = OutFloorResult.IsWalkableFloor();
		}
	}
}


void FFloorCheckResult::SetFromSweep(const FHitResult& InHit, const float InSweepFloorDist, const bool bIsWalkableFloor)
{
	bBlockingHit = InHit.IsValidBlockingHit();
	bWalkableFloor = bIsWalkableFloor;
	FloorDist = InSweepFloorDist;
	HitResult = InHit;
	bLineTrace = false;
	LineDist = 0.f;
}

void FFloorCheckResult::SetFromLineTrace(const FHitResult& InHit, const float InSweepFloorDist, const float InLineDist, const bool bIsWalkableFloor)
{
	// We require a sweep that hit if we are going to use a line result.
	ensure(HitResult.bBlockingHit);
	if (HitResult.bBlockingHit && InHit.bBlockingHit)
	{
		// Override most of the sweep result with the line result, but save some values
		FHitResult OldHit(HitResult);
		HitResult = InHit;

		// Restore some of the old values. We want the new normals and hit actor, however.
		HitResult.Time = OldHit.Time;
		HitResult.ImpactPoint = OldHit.ImpactPoint;
		HitResult.Location = OldHit.Location;
		HitResult.TraceStart = OldHit.TraceStart;
		HitResult.TraceEnd = OldHit.TraceEnd;
		bLineTrace = true;
		LineDist = InLineDist;
		
		FloorDist = InSweepFloorDist;
		bWalkableFloor = bIsWalkableFloor;
	}
}
