// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoveLibrary/GroundMovementUtils.h"

#include "MoverComponent.h"
#include "MoverLog.h"
#include "Components/CapsuleComponent.h"
#include "GameFramework/Pawn.h"
#include "MoveLibrary/AsyncMovementUtils.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/MovementRecord.h"
#include "MoveLibrary/MovementUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroundMovementUtils)

FProposedMove UGroundMovementUtils::ComputeControlledGroundMove(const FGroundMoveParams& InParams)
{
	FProposedMove OutMove;

	const FVector MoveDirIntent = UMovementUtils::ComputeDirectionIntent(InParams.MoveInput, InParams.MoveInputType, InParams.MaxSpeed);

	const FPlane MovementPlane(FVector::ZeroVector, InParams.UpDirection);
	FVector MoveDirIntentInMovementPlane = UMovementUtils::ConstrainToPlane(MoveDirIntent, MovementPlane, true);

	const FPlane GroundSurfacePlane(FVector::ZeroVector, InParams.GroundNormal);
	OutMove.DirectionIntent = UMovementUtils::ConstrainToPlane(MoveDirIntentInMovementPlane, GroundSurfacePlane, true);
	
	OutMove.bHasDirIntent = !OutMove.DirectionIntent.IsNearlyZero();

	FComputeVelocityParams ComputeVelocityParams;
	ComputeVelocityParams.DeltaSeconds = InParams.DeltaSeconds;
	ComputeVelocityParams.InitialVelocity = InParams.PriorVelocity;
	ComputeVelocityParams.MoveDirectionIntent = MoveDirIntentInMovementPlane;
	ComputeVelocityParams.MaxSpeed = InParams.MaxSpeed;
	ComputeVelocityParams.TurningBoost = InParams.TurningBoost;
	ComputeVelocityParams.Deceleration = InParams.Deceleration;
	ComputeVelocityParams.Acceleration = InParams.Acceleration;
	ComputeVelocityParams.Friction = InParams.Friction;
	ComputeVelocityParams.MoveInputType = InParams.MoveInputType;
	ComputeVelocityParams.MoveInput = InParams.MoveInput;
	ComputeVelocityParams.bUseAccelerationForVelocityMove = InParams.bUseAccelerationForVelocityMove;
	
	// Figure out linear velocity
	const FVector Velocity = UMovementUtils::ComputeVelocity(ComputeVelocityParams);
	OutMove.LinearVelocity = UMovementUtils::ConstrainToPlane(Velocity, GroundSurfacePlane, true);

	// Linearly rotate in place
	OutMove.AngularVelocityDegrees = UMovementUtils::ComputeAngularVelocityDegrees(InParams.PriorOrientation, InParams.OrientationIntent, InParams.DeltaSeconds, InParams.TurningRate);

	return OutMove;
}

static const FName StepUpSubstepName = "StepUp";
static const FName StepFwdSubstepName = "StepFwd";
static const FName StepDownSubstepName = "StepDown";
static const FName SlideSubstepName = "SlideFromStep";

float UGroundMovementUtils::TryMoveToStepUp(const FMovingComponentSet& MovingComps, const FVector& GravDir, float MaxStepHeight, const FFloorCheckSettings& FloorCheckSettings, const FVector& MoveDelta, const FHitResult& MoveHitResult, const FFloorCheckResult& CurrentFloor, bool bIsFalling, FOptionalFloorCheckResult* OutFloorTestResult, FMovementRecord& MoveRecord)
{
	float PercentMovementUsed = 0;
	if (!CanStepUpOnHitSurface(MoveHitResult) || MaxStepHeight <= 0.f)
	{
		return 0;
	}

	float PawnRadius(0.0f), PawnHalfHeight(0.0f);
	if (const UCapsuleComponent* CapsuleComponent = Cast<UCapsuleComponent>(MovingComps.UpdatedPrimitive.Get()))
	{
		CapsuleComponent->GetScaledCapsuleSize(PawnRadius, PawnHalfHeight);
	}
	else if (MovingComps.UpdatedPrimitive.IsValid())
	{
		MovingComps.UpdatedPrimitive->CalcBoundingCylinder(PawnRadius, PawnHalfHeight);
	}

	USceneComponent* RootComponent = MovingComps.UpdatedComponent.Get();

	const FVector OldLocation = RootComponent->GetComponentLocation();

	TArray<FMovementSubstep> QueuedSubsteps;	// keeping track of substeps before committing, because some moves can be backed out

	FVector UpDirection = MovingComps.MoverComponent->GetUpDirection();

	FVector LastComponentLocation = OldLocation;

	// Don't bother stepping up if top of shape is hitting something.
	const float InitialImpactDot = MoveHitResult.ImpactPoint.Dot(UpDirection);
	const float OldLocationDot = OldLocation.Dot(UpDirection);
	if (InitialImpactDot > OldLocationDot + (PawnHalfHeight - PawnRadius))
	{
		UE_LOGF(LogMover, VeryVerbose, "Not stepping up due to top of shape hitting something");
		return 0;
	}

	// TODO: We should rely on movement plane normal, rather than gravity direction
	if (GravDir.IsZero())
	{
		UE_LOGF(LogMover, VeryVerbose, "Not stepping up because there's no gravity");
		return 0;
	}

	// Gravity should be a normalized direction
	ensure(GravDir.IsNormalized());

	float StepTravelUpHeight = MaxStepHeight;
	float StepTravelDownHeight = StepTravelUpHeight;
	const float StepSideDot = -1.f * FVector::DotProduct(MoveHitResult.ImpactNormal, GravDir);
	float PawnInitialFloorBaseDot = OldLocationDot - PawnHalfHeight;
	float PawnFloorPointDot = PawnInitialFloorBaseDot;

	//if (IsMovingOnGround() && CurrentFloor.IsWalkableFloor())
	if (CurrentFloor.IsWalkableFloor())
	{
		// Since we float a variable amount off the floor, we need to enforce max step height off the actual point of impact with the floor.
		const float FloorDist = FMath::Max(0.f, CurrentFloor.GetDistanceToFloor());
		PawnInitialFloorBaseDot -= FloorDist;
		StepTravelUpHeight = FMath::Max(StepTravelUpHeight - FloorDist, 0.f);
		StepTravelDownHeight = (MaxStepHeight + UE::FloorQueryUtility::MAX_FLOOR_DIST * 2.f);

		const bool bHitVerticalFace = !UFloorQueryUtils::IsWithinEdgeTolerance(MoveHitResult.Location, MoveHitResult.ImpactPoint, PawnRadius, UpDirection);
		if (!CurrentFloor.bLineTrace && !bHitVerticalFace)
		{
			PawnFloorPointDot = CurrentFloor.HitResult.ImpactPoint.Dot(UpDirection);
		}
		else
		{
			// Base floor point is the base of the shape moved down by how far we are hovering over the surface we are hitting.
			PawnFloorPointDot -= CurrentFloor.FloorDist;
		}
	}

	// Don't step up if the impact is below us, accounting for distance from floor.
	if (InitialImpactDot <= PawnInitialFloorBaseDot)
	{
		UE_LOGF(LogMover, VeryVerbose, "Not stepping up because the impact is below us");
		return 0;
	}

	// Scope our movement updates, and do not apply them until all intermediate moves are completed.
	FScopedMovementUpdate ScopedStepUpMovement(RootComponent, EScopedUpdate::DeferredUpdates);

	// step up - treat as vertical wall
	FHitResult SweepUpHit(1.f);
	const FQuat PawnRotation = RootComponent->GetComponentQuat();

	const FVector UpAdjustment = -GravDir * StepTravelUpHeight;
	const bool bDidStepUp = UMovementUtils::TryMoveUpdatedComponent_Internal(MovingComps, UpAdjustment, PawnRotation, true, MOVECOMP_NoFlags, &SweepUpHit, ETeleportType::None);

	UE_LOGF(LogMover, VeryVerbose, "TryMoveToStepUp Up: %ls (role %i) UpAdjustment=%ls DidMove=%i",
		*GetNameSafe(RootComponent->GetOwner()), RootComponent->GetOwnerRole(), *UpAdjustment.ToCompactString(), bDidStepUp);

	if (SweepUpHit.bStartPenetrating)
	{
		// Undo movement
		UE_LOGF(LogMover, VeryVerbose, "Reverting step-up attempt because we started in a penetrating state");
		ScopedStepUpMovement.RevertMove();
		return 0;
	}

	// Cache upwards substep
	QueuedSubsteps.Add(FMovementSubstep(StepUpSubstepName, RootComponent->GetComponentLocation()-LastComponentLocation, false));
	LastComponentLocation = RootComponent->GetComponentLocation();

	// step fwd
	FHitResult StepFwdHit(1.f);
	const bool bDidStepFwd = UMovementUtils::TryMoveUpdatedComponent_Internal(MovingComps, MoveDelta, PawnRotation, true, MOVECOMP_NoFlags, &StepFwdHit, ETeleportType::None);

	UE_LOGF(LogMover, VeryVerbose, "TryMoveToStepUp Fwd: %ls (role %i) MoveDelta=%ls DidMove=%i",
		*GetNameSafe(RootComponent->GetOwner()), RootComponent->GetOwnerRole(), *MoveDelta.ToCompactString(), bDidStepFwd);

	PercentMovementUsed += StepFwdHit.Time;
	
	// Check result of forward movement
	if (StepFwdHit.bBlockingHit)
	{
		if (StepFwdHit.bStartPenetrating)
		{
			// Undo movement
			UE_LOGF(LogMover, VeryVerbose, "Reverting step-fwd attempt during step-up, because we started in a penetrating state");
			ScopedStepUpMovement.RevertMove();
			return 0;
		}

		UMoverComponent* MoverComp = MovingComps.MoverComponent.Get();
		// If we hit something above us and also something ahead of us, we should notify about the upward hit as well.
		// The forward hit will be handled later (in the bSteppedOver case below).
		// In the case of hitting something above but not forward, we are not blocked from moving so we don't need the notification.
		if (MoverComp && SweepUpHit.bBlockingHit && StepFwdHit.bBlockingHit)
		{
			FMoverOnImpactParams ImpactParams(NAME_None, SweepUpHit, MoveDelta);
			MoverComp->HandleImpact(ImpactParams);
		}

		// pawn ran into a wall
		if (MoverComp)
		{
			FMoverOnImpactParams ImpactParams(NAME_None, StepFwdHit, MoveDelta);
			MoverComp->HandleImpact(ImpactParams);
		}
		
		if (bIsFalling)
		{
			QueuedSubsteps.Add( FMovementSubstep(StepFwdSubstepName, RootComponent->GetComponentLocation() - LastComponentLocation, true) );

			// Commit queued substeps to movement record
			for (FMovementSubstep Substep : QueuedSubsteps)
			{
				MoveRecord.Append(Substep);
			}

			return PercentMovementUsed;
		}

		// Cache forwards substep before the slide attempt
		QueuedSubsteps.Add(FMovementSubstep(StepFwdSubstepName, RootComponent->GetComponentLocation() - LastComponentLocation, true));
		LastComponentLocation = RootComponent->GetComponentLocation();

		// adjust and try again
		const float ForwardHitTime = StepFwdHit.Time;

		// locking relevancy so velocity isn't added until it is needed to (adding it to the QueuedSubsteps so it can get added later)
		MoveRecord.LockRelevancy(false);
		const float ForwardSlideAmount = TryWalkToSlideAlongSurface(MovingComps, MoveDelta, 1.f - StepFwdHit.Time, PawnRotation, StepFwdHit.Normal, StepFwdHit, true, MoveRecord, FloorCheckSettings.MaxWalkSlopeCosine, MaxStepHeight);
		QueuedSubsteps.Add( FMovementSubstep(SlideSubstepName, RootComponent->GetComponentLocation() - LastComponentLocation, true) );
		LastComponentLocation = RootComponent->GetComponentLocation();
		MoveRecord.UnlockRelevancy();

		PercentMovementUsed += ForwardSlideAmount * (1 - PercentMovementUsed);
		
		// If both the forward hit and the deflection got us nowhere, there is no point in this step up.
		if (ForwardHitTime == 0.f && ForwardSlideAmount == 0.f)
		{
			UE_LOGF(LogMover, VeryVerbose, "Reverting step-fwd attempt during step-up, because no movement differences occurred");
			ScopedStepUpMovement.RevertMove();
			return 0;
		}
	}
	else
	{
		// Our forward move attempt was unobstructed - cache it
		QueuedSubsteps.Add(FMovementSubstep(StepFwdSubstepName, RootComponent->GetComponentLocation() - LastComponentLocation, true));
		LastComponentLocation = RootComponent->GetComponentLocation();
		PercentMovementUsed = 1.f;
	}


	// Step down
	FHitResult StepDownHit(1.f);
	const FVector StepDownAdjustment = GravDir * StepTravelDownHeight;
	const bool bDidStepDown = UMovementUtils::TryMoveUpdatedComponent_Internal(MovingComps, StepDownAdjustment, RootComponent->GetComponentQuat(), true, MOVECOMP_NoFlags, &StepDownHit, ETeleportType::None);

	UE_LOGF(LogMover, VeryVerbose, "TryMoveToStepUp Down: %ls (role %i) StepDownAdjustment=%ls DidMove=%i",
		*GetNameSafe(RootComponent->GetOwner()), RootComponent->GetOwnerRole(), *StepDownAdjustment.ToCompactString(), bDidStepDown);


	// If step down was initially penetrating abort the step up
	if (StepDownHit.bStartPenetrating)
	{
		UE_LOGF(LogMover, VeryVerbose, "Reverting step-down attempt during step-up/step-fwd, because we started in a penetrating state");
		ScopedStepUpMovement.RevertMove();
		return 0;
	}

	FOptionalFloorCheckResult StepDownResult;
	if (StepDownHit.IsValidBlockingHit())
	{
		// See if this step sequence would have allowed us to travel higher than our max step height allows.
		const float DeltaDot = (StepDownHit.ImpactPoint.Dot(UpDirection)) - PawnFloorPointDot;
		if (DeltaDot > MaxStepHeight)
		{
			UE_LOGF(LogMover, VeryVerbose, "Reject step-down attempt during step-up/step-fwd, because it made us travel too high (too high Height %.3f) up from floor base %f to %f", DeltaDot, PawnInitialFloorBaseDot, StepDownHit.ImpactPoint.Z);
			ScopedStepUpMovement.RevertMove();
			return 0;
		}

		// Reject unwalkable surface normals here.
		if (!UFloorQueryUtils::IsHitSurfaceWalkable(StepDownHit, UpDirection, FloorCheckSettings.MaxWalkSlopeCosine))
		{
			// Reject if normal opposes movement direction
			const bool bNormalTowardsMe = (MoveDelta | StepDownHit.ImpactNormal) < 0.f;
			if (bNormalTowardsMe)
			{
				UE_LOGF(LogMover, VeryVerbose, "Reject step-down attempt during step-up/step-fwd, due to unwalkable normal %ls opposed to movement", *StepDownHit.ImpactNormal.ToString());
				ScopedStepUpMovement.RevertMove();
				return 0;
			}

			// Also reject if we would end up being higher than our starting location by stepping down.
			// It's fine to step down onto an unwalkable normal below us, we will just slide off. Rejecting those moves would prevent us from being able to walk off the edge.
			if (StepDownHit.Location.Dot(UpDirection) > OldLocationDot)
			{
				UE_LOGF(LogMover, VeryVerbose, "Reject step-down attempt during step-up/step-fwd, due to unwalkable normal %ls above old position)", *StepDownHit.ImpactNormal.ToString());
				ScopedStepUpMovement.RevertMove();
				return 0;
			}
		}

		// Reject moves where the downward sweep hit something very close to the edge of the shape. This maintains consistency with FindFloor as well.
		if (!UFloorQueryUtils::IsWithinEdgeTolerance(StepDownHit.Location, StepDownHit.ImpactPoint, PawnRadius, UpDirection))
		{
			UE_LOGF(LogMover, VeryVerbose, "Reject step-down attempt during step-up/step-fwd, due to being outside edge tolerance");
			ScopedStepUpMovement.RevertMove();
			return 0;
		}

		// Don't step up onto invalid surfaces if traveling higher.
		if (DeltaDot > 0.f && !CanStepUpOnHitSurface(StepDownHit))
		{
			UE_LOGF(LogMover, VeryVerbose, "Reject step-down attempt during step-up/step-fwd, due to being up onto surface with !CanStepUpOnHitSurface"); 
			ScopedStepUpMovement.RevertMove();
			return 0;
		}

		// See if we can validate the floor as a result of this step down. In almost all cases this should succeed, and we can avoid computing the floor outside this method.
		if (OutFloorTestResult != NULL)
		{

			UFloorQueryUtils::FindFloor(MovingComps, FloorCheckSettings, RootComponent->GetComponentLocation(), StepDownResult.FloorTestResult);

			// Reject unwalkable normals if we end up higher than our initial height.
			// It's fine to walk down onto an unwalkable surface, don't reject those moves.
			if (StepDownHit.Location.Dot(UpDirection) > OldLocationDot)
			{
				// We should reject the floor result if we are trying to step up an actual step where we are not able to perch (this is rare).
				// In those cases we should instead abort the step up and try to slide along the stair.
				const float MAX_STEP_SIDE_DOT = 0.08f; // TODO: Move magic numbers elsewhere
				if (!StepDownResult.FloorTestResult.bBlockingHit && StepSideDot < MAX_STEP_SIDE_DOT)
				{
					UE_LOGF(LogMover, VeryVerbose, "Reject step-down attempt during step-up/step-fwd, due to it being an unperchable step"); 
					ScopedStepUpMovement.RevertMove();
					return 0;
				}
			}

			StepDownResult.bHasFloorResult = true;
		}
	}

	// Cache downwards substep
	QueuedSubsteps.Add(FMovementSubstep(StepDownSubstepName, RootComponent->GetComponentLocation() - LastComponentLocation, false));
	LastComponentLocation = RootComponent->GetComponentLocation();

	// Copy step down result.
	if (OutFloorTestResult != NULL)
	{
		*OutFloorTestResult = StepDownResult;
	}

	// Don't recalculate velocity based on this height adjustment, if considering vertical adjustments.
	//bJustTeleported |= !bMaintainHorizontalGroundVelocity;

	// Commit queued substeps to movement record
	for (FMovementSubstep Substep : QueuedSubsteps)
	{
		MoveRecord.Append(Substep);
	}

	return PercentMovementUsed;

}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
float UGroundMovementUtils::TryMoveToStepUp(const FMovingComponentSet& MovingComps, const FVector& GravDir, float MaxStepHeight, float MaxWalkSlopeCosine, bool bUseFlatBaseForFloorChecks, float FloorSweepDistance, const FVector& MoveDelta, const FHitResult& MoveHitResult, const FFloorCheckResult& CurrentFloor, bool bIsFalling, FOptionalFloorCheckResult* OutFloorTestResult, FMovementRecord& MoveRecord, float PerchRadiusThreshold)
{
	FFloorCheckSettings Settings;
	Settings.FloorSweepDistance = FloorSweepDistance;
	Settings.MaxWalkSlopeCosine = MaxWalkSlopeCosine;
	Settings.bUseFlatBaseForFloorChecks = bUseFlatBaseForFloorChecks;
	Settings.PerchRadiusThreshold = PerchRadiusThreshold;
	return TryMoveToStepUp(MovingComps, GravDir, MaxStepHeight, Settings, MoveDelta, MoveHitResult, CurrentFloor, bIsFalling, OutFloorTestResult, MoveRecord);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

static bool TryMoveToAdjustHeightAboveFloorInternal(const FMovingComponentSet& MovingComps, FFloorCheckResult& CurrentFloor, float MaxWalkSlopeCosine, bool bPushAwayOnly, FMovementRecord& MoveRecord)
{
	// If we have a floor check that hasn't hit anything, don't adjust height.
	if (!CurrentFloor.IsWalkableFloor())
	{
		return false;
	}

	const FVector UpDirection = MovingComps.MoverComponent->GetUpDirection();
	
	float OldFloorDist = CurrentFloor.FloorDist;
	if (CurrentFloor.bLineTrace)
	{
		if (OldFloorDist < UE::FloorQueryUtility::MIN_FLOOR_DIST && CurrentFloor.LineDist >= UE::FloorQueryUtility::MIN_FLOOR_DIST)
		{
			// This would cause us to scale unwalkable walls
			return false;
		}
		// Falling back to a line trace means the sweep was unwalkable (or in penetration). Use the line distance for the vertical adjustment.
		OldFloorDist = CurrentFloor.LineDist;
	}

	// Move up or down to maintain floor height.
	if (OldFloorDist < UE::FloorQueryUtility::MIN_FLOOR_DIST || OldFloorDist > UE::FloorQueryUtility::MAX_FLOOR_DIST)
	{
		const float AvgFloorDist = (UE::FloorQueryUtility::MIN_FLOOR_DIST + UE::FloorQueryUtility::MAX_FLOOR_DIST) * 0.5f;
		const float MoveDist = AvgFloorDist - OldFloorDist;

		// Don't adjust if we're only supposed to be staying away and the adjustment would bring us closer
		if (bPushAwayOnly && MoveDist <= 0.0)
		{
			return false;
		}

		USceneComponent* UpdatedComponent = MovingComps.UpdatedComponent.Get();
		const float InitialUp = UpdatedComponent->GetComponentLocation().Dot(UpDirection);
		FHitResult AdjustHit(1.f);
		
		MoveRecord.LockRelevancy(false);

		UMovementUtils::TrySafeMoveUpdatedComponent(MovingComps,
			UpDirection * MoveDist, UpdatedComponent->GetComponentQuat(),
			true, AdjustHit, ETeleportType::None, MoveRecord);

		MoveRecord.UnlockRelevancy();

		if (!AdjustHit.IsValidBlockingHit())
		{
			CurrentFloor.FloorDist += MoveDist;
		}
		else if (MoveDist > 0.f)
		{
			const float CurrentUp = UpdatedComponent->GetComponentLocation().Dot(UpDirection);
			CurrentFloor.FloorDist += CurrentUp - InitialUp;
		}
		else
		{
			checkSlow(MoveDist < 0.f);
			const float CurrentUp = UpdatedComponent->GetComponentLocation().Dot(UpDirection);
			CurrentFloor.FloorDist = CurrentUp - AdjustHit.Location.Dot(UpDirection);
			if (UFloorQueryUtils::IsHitSurfaceWalkable(AdjustHit, UpDirection, MaxWalkSlopeCosine))
			{
				CurrentFloor.SetFromSweep(AdjustHit, CurrentFloor.FloorDist, true);
			}
		}

		return true;
	}

	return false;
}


bool UGroundMovementUtils::TryMoveToAdjustHeightAboveFloor(const FMovingComponentSet& MovingComps, FFloorCheckResult& CurrentFloor, float MaxWalkSlopeCosine, FMovementRecord& MoveRecord)
{
	return TryMoveToAdjustHeightAboveFloorInternal(MovingComps, CurrentFloor, MaxWalkSlopeCosine, /*bPushAwayOnly*/false, MoveRecord);
}


bool UGroundMovementUtils::TryMoveToKeepMinHeightAboveFloor(const FMovingComponentSet& MovingComps, FFloorCheckResult& CurrentFloor, float MaxWalkSlopeCosine, FMovementRecord& MoveRecord)
{
	return TryMoveToAdjustHeightAboveFloorInternal(MovingComps, CurrentFloor, MaxWalkSlopeCosine, /*bPushAwayOnly*/true, MoveRecord);
}


float UGroundMovementUtils::TryWalkToSlideAlongSurface(const FMovingComponentSet& MovingComps, const FVector& Delta, float PctOfDeltaToMove, const FQuat Rotation, const FVector& Normal, FHitResult& Hit, bool bHandleImpact, FMovementRecord& MoveRecord, float MaxWalkSlopeCosine, float MaxStepHeight)
{
	if (!Hit.bBlockingHit)
	{
		return 0.f;
	}
	
	if (UMoverComponent* MoverComponent = MovingComps.MoverComponent.Get())
	{
		const FVector UpDirection = MoverComponent->GetUpDirection();
		FVector SafeWalkNormal(Normal);
		const FVector::FReal NormalDot = Normal.Dot(UpDirection);

		// We don't want to be pushed up an unwalkable surface.
		if (NormalDot > 0.f && !UFloorQueryUtils::IsHitSurfaceWalkable(Hit, UpDirection, MaxWalkSlopeCosine))
		{
			SafeWalkNormal = FVector::VectorPlaneProject(SafeWalkNormal, -UpDirection).GetSafeNormal();
		}

		float PctOfTimeUsed = 0.f;
		const FVector OldSafeHitNormal = SafeWalkNormal;

		FVector SlideDelta = UMovementUtils::ComputeSlideDelta(MovingComps, Delta, PctOfDeltaToMove, SafeWalkNormal, Hit);
		FVector OriginalSlideDelta = SlideDelta;
	
		if (SlideDelta.Dot(Delta) > 0.f)
		{
			UMovementUtils::TrySafeMoveUpdatedComponent(MovingComps, SlideDelta, Rotation, true, Hit, ETeleportType::None, MoveRecord);

			PctOfTimeUsed = Hit.Time;

			if (Hit.IsValidBlockingHit())
			{
				// Notify first impact
				if (MoverComponent && bHandleImpact)
				{
					FMoverOnImpactParams ImpactParams(NAME_None, Hit, SlideDelta);
					MoverComponent->HandleImpact(ImpactParams);
				}

				// Compute new slide normal when hitting multiple surfaces.
				SlideDelta = UMovementUtils::ComputeTwoWallAdjustedDelta(MovingComps, SlideDelta, Hit, OldSafeHitNormal);
				const float SlideDeltaDot = SlideDelta.Dot(UpDirection);
				const float HitNormalDot = Hit.Normal.Dot(UpDirection);
				if (SlideDeltaDot > 0.f && UFloorQueryUtils::IsHitSurfaceWalkable(Hit, UpDirection, MaxWalkSlopeCosine) && HitNormalDot > UE_KINDA_SMALL_NUMBER)
				{
					// Maintain horizontal velocity
					const float Time = (1.f - Hit.Time);
					const FVector ScaledDelta = SlideDelta.GetSafeNormal() * SlideDelta.Size();
					const FVector::FReal NewDeltaDot = (ScaledDelta.Dot(UpDirection) / HitNormalDot);
					FVector SlideProjectedToGrav = FVector::VectorPlaneProject(OriginalSlideDelta, -UpDirection);
					SlideDelta = (SlideProjectedToGrav + NewDeltaDot * UpDirection) * Time;
					// Should never exceed MaxStepHeight in vertical component, so rescale if necessary.
					// This should be rare (HitNormalDot above would have been very small) but we'd rather lose horizontal velocity than go too high.
					if (SlideDeltaDot > MaxStepHeight)
					{
						const float Rescale = MaxStepHeight / SlideDeltaDot;
						SlideDelta *= Rescale;
					}
				}
				else
				{
					SlideDelta = FVector::VectorPlaneProject(SlideDelta, -UpDirection);
				}
			
				// Only proceed if the new direction is of significant length and not in reverse of original attempted move.
				if (!SlideDelta.IsNearlyZero(UE::MoverUtils::SMALL_MOVE_DISTANCE) && (SlideDelta | Delta) > 0.f)
				{
					// Perform second move
					UMovementUtils::TrySafeMoveUpdatedComponent(MovingComps, SlideDelta, Rotation, true, Hit, ETeleportType::None, MoveRecord);
					PctOfTimeUsed += (Hit.Time * (1.f - PctOfTimeUsed));

					// Notify second impact
					if (MoverComponent && bHandleImpact && Hit.bBlockingHit)
					{
						FMoverOnImpactParams ImpactParams(NAME_None, Hit, SlideDelta);
						MoverComponent->HandleImpact(ImpactParams);
					}
				}
			}

			return FMath::Clamp(PctOfTimeUsed, 0.f, 1.f);
		}
	}

	return 0.f;
}

FVector UGroundMovementUtils::ComputeDeflectedMoveOntoRamp(const FVector& OrigMoveDelta, const FVector& UpDirection, const FHitResult& RampHitResult, float MaxWalkSlopeCosine, const bool bHitFromLineTrace)
{
	const FVector FloorNormal = RampHitResult.ImpactNormal;
	const FVector::FReal FloorNormalDot = FloorNormal.Dot(UpDirection);
	const FVector::FReal ContactNormalDot = RampHitResult.Normal.Dot(UpDirection);

	if (FloorNormalDot < (1.f - UE_KINDA_SMALL_NUMBER) && FloorNormalDot > UE_KINDA_SMALL_NUMBER && 
		ContactNormalDot > UE_KINDA_SMALL_NUMBER && 
		UFloorQueryUtils::IsHitSurfaceWalkable(RampHitResult, UpDirection, MaxWalkSlopeCosine) && !bHitFromLineTrace)
	{
		// Compute a vector that moves parallel to the surface, by projecting the horizontal movement direction onto the ramp.
		const FPlane RampSurfacePlane(FVector::ZeroVector, FloorNormal);
		return UMovementUtils::ConstrainToPlane(OrigMoveDelta, RampSurfacePlane, true);
	}

	return OrigMoveDelta;
}

bool UGroundMovementUtils::CanStepUpOnHitSurface(const FHitResult& Hit)
{
	if (!Hit.IsValidBlockingHit())
	{
		return false;
	}

	// No component for "fake" hits when we are on a known good base.
	const UPrimitiveComponent* HitComponent = Hit.Component.Get();
	if (!HitComponent)
	{
		return true;
	}

	APawn* PawnOwner = Cast<APawn>(Hit.GetActor());

	if (!HitComponent->CanCharacterStepUp(PawnOwner))
	{
		return false;
	}

	// No actor for "fake" hits when we are on a known good base.

	if (!Hit.HitObjectHandle.IsValid())
	{
		return true;
	}

	const AActor* HitActor = Hit.HitObjectHandle.GetManagingActor();
	if (!HitActor->CanBeBaseForCharacter(PawnOwner))
	{
		return false;
	}

	return true;
}

float UGroundMovementUtils::TestGroundedMoveAlongHitSurface(const FMovingComponentSet& MovingComps, const FVector& OriginalMoveDelta, const FVector& LocationAtHit, const FQuat& TargetRotation, bool bHandleImpact, float MaxStepHeight, float MaxWalkSlopeCosine, FHitResult& InOutHit, FMovementRecord& InOutMoveRecord)
{
	if (!InOutHit.bBlockingHit)
	{
		return 0.f;
	}

	float PctOfOrigDeltaToSlide = 1.f - InOutHit.Time;

	if (UMoverComponent* MoverComponent = MovingComps.MoverComponent.Get())
	{
		const FVector UpDirection = MoverComponent->GetUpDirection();
		FVector SafeWalkNormal(InOutHit.Normal);
		const FVector::FReal NormalDot = InOutHit.Normal.Dot(UpDirection);

		// We don't want to be pushed up an unwalkable surface.
		if (NormalDot > 0.f && !UFloorQueryUtils::IsHitSurfaceWalkable(InOutHit, UpDirection, MaxWalkSlopeCosine))
		{
			SafeWalkNormal = FVector::VectorPlaneProject(SafeWalkNormal, -UpDirection).GetSafeNormal();
		}

		float PctOfTimeUsed = 0.f;
		const FVector OldSafeHitNormal = SafeWalkNormal;

		FVector SlideDelta = UMovementUtils::ComputeSlideDelta(MovingComps, OriginalMoveDelta, PctOfOrigDeltaToSlide, SafeWalkNormal, InOutHit);
		const FVector OriginalSlideDelta = SlideDelta;

		if (SlideDelta.Dot(OriginalMoveDelta) > 0.f)
		{
			UAsyncMovementUtils::TestDepenetratingMove(MovingComps, LocationAtHit, LocationAtHit + SlideDelta, TargetRotation, TargetRotation, /*bShouldSweep*/ true, InOutHit, InOutMoveRecord);

			FVector LocationInProgress = InOutHit.TraceStart + ((InOutHit.TraceEnd - InOutHit.TraceStart) * InOutHit.Time);

			PctOfTimeUsed = InOutHit.Time;

			if (InOutHit.IsValidBlockingHit())
			{
				// Notify first impact
				if (MoverComponent && bHandleImpact)
				{
					FMoverOnImpactParams ImpactParams(NAME_None, InOutHit, SlideDelta);
					MoverComponent->HandleImpact(ImpactParams);
				}

				// Compute new slide normal when hitting multiple surfaces.
				SlideDelta = UMovementUtils::ComputeTwoWallAdjustedDelta(MovingComps, SlideDelta, InOutHit, OldSafeHitNormal);
				const float SlideDeltaDot = SlideDelta.Dot(UpDirection);
				const float HitNormalDot = InOutHit.Normal.Dot(UpDirection);
				if (SlideDeltaDot > 0.f && UFloorQueryUtils::IsHitSurfaceWalkable(InOutHit, UpDirection, MaxWalkSlopeCosine) && HitNormalDot > UE_KINDA_SMALL_NUMBER)
				{
					// Maintain horizontal velocity
					const float Time = (1.f - InOutHit.Time);
					const FVector ScaledDelta = SlideDelta.GetSafeNormal() * SlideDelta.Size();
					const FVector::FReal NewDeltaDot = (ScaledDelta.Dot(UpDirection) / HitNormalDot);
					FVector SlideProjectedToGrav = FVector::VectorPlaneProject(OriginalSlideDelta, -UpDirection);
					SlideDelta = (SlideProjectedToGrav + NewDeltaDot * UpDirection) * Time;
					// Should never exceed MaxStepHeight in vertical component, so rescale if necessary.
					// This should be rare (HitNormalDot above would have been very small) but we'd rather lose horizontal velocity than go too high.
					if (SlideDeltaDot > MaxStepHeight)
					{
						const float Rescale = MaxStepHeight / SlideDeltaDot;
						SlideDelta *= Rescale;
					}
				}
				else
				{
					SlideDelta = FVector::VectorPlaneProject(SlideDelta, -UpDirection);
				}

				// Only proceed if the new direction is of significant length and not in reverse of original attempted move.
				if (!SlideDelta.IsNearlyZero(UE::MoverUtils::SMALL_MOVE_DISTANCE) && (SlideDelta | OriginalMoveDelta) > 0.f)
				{
					// Perform second move
					UAsyncMovementUtils::TestDepenetratingMove(MovingComps, LocationInProgress, LocationInProgress + SlideDelta, TargetRotation, TargetRotation, /*bShouldSweep*/ true, InOutHit, InOutMoveRecord);
					PctOfTimeUsed += (InOutHit.Time * (1.f - PctOfTimeUsed));

					// Notify second impact
					if (MoverComponent && bHandleImpact && InOutHit.bBlockingHit)
					{
						FMoverOnImpactParams ImpactParams(NAME_None, InOutHit, SlideDelta);
						MoverComponent->HandleImpact(ImpactParams);
					}
				}
			}

			return FMath::Clamp(PctOfTimeUsed, 0.f, 1.f);
		}
	}

	return 0.f;
}


static FVector TestMoveToAdjustToFloorInternal(const FMovingComponentSet& MovingComps, const FVector& Location, const FQuat& Rotation, float MaxWalkSlopeCosine, bool bPushAwayOnly, FFloorCheckResult& InOutCurrentFloor, FMovementRecord& InOutMoveRecord)
{
	// If we have a floor check that hasn't hit anything, don't adjust height.
	if (!InOutCurrentFloor.IsWalkableFloor())
	{
		return Location;	// no adjustment
	}

	const FVector UpDirection = MovingComps.MoverComponent->GetUpDirection();

	float OldFloorDist = InOutCurrentFloor.FloorDist;
	if (InOutCurrentFloor.bLineTrace)
	{
		if (OldFloorDist < UE::FloorQueryUtility::MIN_FLOOR_DIST && InOutCurrentFloor.LineDist >= UE::FloorQueryUtility::MIN_FLOOR_DIST)
		{
			// This would cause us to scale unwalkable walls
			return Location;	// no adjustment
		}
		// Falling back to a line trace means the sweep was unwalkable (or in penetration). Use the line distance for the vertical adjustment.
		OldFloorDist = InOutCurrentFloor.LineDist;
	}

	// Move up or down to maintain floor height.
	if (OldFloorDist < UE::FloorQueryUtility::MIN_FLOOR_DIST || OldFloorDist > UE::FloorQueryUtility::MAX_FLOOR_DIST)
	{
		const float AvgFloorDist = (UE::FloorQueryUtility::MIN_FLOOR_DIST + UE::FloorQueryUtility::MAX_FLOOR_DIST) * 0.5f;
		const float MoveDist = AvgFloorDist - OldFloorDist;

		if (bPushAwayOnly && MoveDist <= 0.0f)
		{
			return Location;	// we are far enough away from the floor already, so no adjustment is needed
		}

		FHitResult AdjustHit(1.f);
		const float InitialUp = Location.Dot(UpDirection);
		const FVector TargetAdjustedLocation = Location + (UpDirection * MoveDist);


		InOutMoveRecord.LockRelevancy(false);	// Makes it so any height adjustment will not count towards velocity changes

		UAsyncMovementUtils::TestDepenetratingMove(MovingComps, Location, TargetAdjustedLocation, Rotation, Rotation, /*bShouldSweep*/ true, AdjustHit, InOutMoveRecord);

		InOutMoveRecord.UnlockRelevancy();

		const FVector AdjustedLocation = AdjustHit.TraceStart + ((AdjustHit.TraceEnd - AdjustHit.TraceStart) * AdjustHit.Time);

		if (!AdjustHit.IsValidBlockingHit())
		{
			InOutCurrentFloor.FloorDist += MoveDist;
		}
		else if (MoveDist > 0.f)
		{
			const float CurrentUp = AdjustedLocation.Dot(UpDirection);
			InOutCurrentFloor.FloorDist += CurrentUp - InitialUp;
		}
		else
		{
			checkSlow(MoveDist < 0.f);
			const float CurrentUp = AdjustedLocation.Dot(UpDirection);
			InOutCurrentFloor.FloorDist = CurrentUp - AdjustHit.Location.Dot(UpDirection);
			if (UFloorQueryUtils::IsHitSurfaceWalkable(AdjustHit, UpDirection, MaxWalkSlopeCosine))
			{
				InOutCurrentFloor.SetFromSweep(AdjustHit, InOutCurrentFloor.FloorDist, true);
			}
		}

		return AdjustedLocation;
	}

	return Location; // no adjustment
}


FVector UGroundMovementUtils::TestMoveToAdjustToFloor(const FMovingComponentSet& MovingComps, const FVector& Location, const FQuat& Rotation, float MaxWalkSlopeCosine, FFloorCheckResult& InOutCurrentFloor, FMovementRecord& InOutMoveRecord)
{
	return TestMoveToAdjustToFloorInternal(MovingComps, Location, Rotation, MaxWalkSlopeCosine, /*bPushAwayOnly*/false, InOutCurrentFloor, InOutMoveRecord);
}


FVector UGroundMovementUtils::TestMoveToKeepMinHeightAboveFloor(const FMovingComponentSet& MovingComps, const FVector& Location, const FQuat& Rotation, float MaxWalkSlopeCosine, UPARAM(ref) FFloorCheckResult& InOutCurrentFloor, UPARAM(ref) FMovementRecord& InOutMoveRecord)
{
	return TestMoveToAdjustToFloorInternal(MovingComps, Location, Rotation, MaxWalkSlopeCosine, /*bPushAwayOnly*/true, InOutCurrentFloor, InOutMoveRecord);
}


/* static */
float UGroundMovementUtils::TestMoveToStepOver(const FMovingComponentSet& MovingComps, const FVector& GravDir, float MaxStepHeight, const FFloorCheckSettings& FloorCheckSettings, const FVector& MoveDelta, const FQuat& Rotation, const FHitResult& MoveHitResult, const FFloorCheckResult& CurrentFloor, bool bIsFalling, FOptionalFloorCheckResult* OutFloorTestResult, FVector& OutFinalLocation, FMovementRecord& InOutMoveRecord)
{
	if (!UGroundMovementUtils::CanStepUpOnHitSurface(MoveHitResult) || MaxStepHeight <= 0.f)
	{
		return 0;
	}

	float PercentMovementUsed = 0;
	
	TArray<FMovementSubstep> QueuedSubsteps;	// keeping track of substeps before committing, because some moves can be backed out

	FVector UpDirection = MovingComps.MoverComponent->GetUpDirection();

	const FVector OldLocation = MoveHitResult.TraceStart + ((MoveHitResult.TraceEnd - MoveHitResult.TraceStart) * MoveHitResult.Time);
	FVector LocationInProgress = OldLocation;

	float PawnRadius(0.0f), PawnHalfHeight(0.0f);
	if (const UCapsuleComponent* CapsuleComponent = Cast<UCapsuleComponent>(MovingComps.UpdatedPrimitive.Get()))
	{
		CapsuleComponent->GetScaledCapsuleSize(PawnRadius, PawnHalfHeight);
	}
	else if (MovingComps.UpdatedPrimitive.IsValid())
	{
		MovingComps.UpdatedPrimitive->CalcBoundingCylinder(PawnRadius, PawnHalfHeight);
	}

	USceneComponent* RootComponent = MovingComps.UpdatedComponent.Get();

	// Don't bother stepping up if top of shape is hitting something.
	const float InitialImpactDot = MoveHitResult.ImpactPoint.Dot(UpDirection);
	const float OldLocationDot = OldLocation.Dot(UpDirection);
	if (InitialImpactDot > OldLocationDot + (PawnHalfHeight - PawnRadius))
	{
		UE_LOGF(LogMover, VeryVerbose, "Not stepping up due to top of shape hitting something");
		return 0;
	}

	// TODO: We should rely on movement plane normal, rather than gravity direction
	if (GravDir.IsZero())
	{
		UE_LOGF(LogMover, VeryVerbose, "Not stepping up because there's no gravity");
		return 0;
	}

	// Gravity should be a normalized direction
	ensure(GravDir.IsNormalized());

	float StepTravelUpHeight = MaxStepHeight;
	float StepTravelDownHeight = StepTravelUpHeight;
	const float StepSideDot = -1.f * FVector::DotProduct(MoveHitResult.ImpactNormal, GravDir);
	float PawnInitialFloorBaseDot = OldLocationDot - PawnHalfHeight;
	float PawnFloorPointDot = PawnInitialFloorBaseDot;

	if (CurrentFloor.IsWalkableFloor())
	{
		// Since we float a variable amount off the floor, we need to enforce max step height off the actual point of impact with the floor.
		const float FloorDist = FMath::Max(0.f, CurrentFloor.GetDistanceToFloor());
		PawnInitialFloorBaseDot -= FloorDist;
		StepTravelUpHeight = FMath::Max(StepTravelUpHeight - FloorDist, 0.f);
		StepTravelDownHeight = (MaxStepHeight + UE::FloorQueryUtility::MAX_FLOOR_DIST * 2.f);

		const bool bHitVerticalFace = !UFloorQueryUtils::IsWithinEdgeTolerance(MoveHitResult.Location, MoveHitResult.ImpactPoint, PawnRadius, UpDirection);
		if (!CurrentFloor.bLineTrace && !bHitVerticalFace)
		{
			PawnFloorPointDot = CurrentFloor.HitResult.ImpactPoint.Dot(UpDirection);
		}
		else
		{
			// Base floor point is the base of the shape moved down by how far we are hovering over the surface we are hitting.
			PawnFloorPointDot -= CurrentFloor.FloorDist;
		}
	}

	// Don't step up if the impact is below us, accounting for distance from floor.
	if (InitialImpactDot <= PawnInitialFloorBaseDot)
	{
		UE_LOGF(LogMover, VeryVerbose, "Not stepping up because the impact is below us");
		return 0;
	}

	FMoverCollisionParams CollisionParams(MovingComps.UpdatedComponent.Get());
	// step up - treat as vertical wall
	FHitResult SweepUpHit(1.f);

	const FVector UpAdjustment = -GravDir * StepTravelUpHeight;

	const bool bDidStepUp = UAsyncMovementUtils::TestMoveComponent_Internal(MovingComps, LocationInProgress, LocationInProgress + UpAdjustment, Rotation, Rotation, /*bShouldSweep=*/ true, CollisionParams, SweepUpHit);

	UE_LOGF(LogMover, VeryVerbose, "TestMoveToStepOver Up: %ls (role %i) UpAdjustment=%ls DidMove=%i",
		*GetNameSafe(RootComponent->GetOwner()), RootComponent->GetOwnerRole(), *UpAdjustment.ToCompactString(), bDidStepUp);

	if (SweepUpHit.bStartPenetrating)
	{
		// Undo movement
		UE_LOGF(LogMover, VeryVerbose, "Reverting step-up attempt because we started in a penetrating state");
		return 0;
	}

	FVector UpStepDelta = (SweepUpHit.TraceStart + ((SweepUpHit.TraceEnd - SweepUpHit.TraceStart) * SweepUpHit.Time)) - LocationInProgress;

	// Cache upwards substep
	QueuedSubsteps.Add(FMovementSubstep(StepUpSubstepName, UpStepDelta, false));
	LocationInProgress += UpStepDelta;

	// step fwd
	FHitResult StepFwdHit(1.f);
	const bool bDidStepFwd = UAsyncMovementUtils::TestMoveComponent_Internal(MovingComps, LocationInProgress, LocationInProgress + MoveDelta, Rotation, Rotation, /*bShouldSweep=*/ true, CollisionParams, StepFwdHit);

	FVector FwdStepDelta = (StepFwdHit.TraceStart + ((StepFwdHit.TraceEnd - StepFwdHit.TraceStart) * StepFwdHit.Time)) - LocationInProgress;

	UE_LOGF(LogMover, VeryVerbose, "TestMoveToStepOver Fwd: %ls (role %i) MoveDelta=%ls DidMove=%i",
		*GetNameSafe(RootComponent->GetOwner()), RootComponent->GetOwnerRole(), *MoveDelta.ToCompactString(), bDidStepFwd);

	// Check result of forward movement
	if (StepFwdHit.bBlockingHit)
	{
		if (StepFwdHit.bStartPenetrating)
		{
			// Undo movement
			UE_LOGF(LogMover, VeryVerbose, "Reverting step-fwd attempt during step-up, because we started in a penetrating state");
			return 0;
		}

		UMoverComponent* MoverComp = MovingComps.MoverComponent.Get();
		// If we hit something above us and also something ahead of us, we should notify about the upward hit as well.
		// The forward hit will be handled later (in the bSteppedOver case below).
		// In the case of hitting something above but not forward, we are not blocked from moving so we don't need the notification.
		if (MoverComp && SweepUpHit.bBlockingHit && StepFwdHit.bBlockingHit)
		{
			FMoverOnImpactParams ImpactParams(NAME_None, SweepUpHit, MoveDelta);
			MoverComp->HandleImpact(ImpactParams);
		}

		// pawn ran into a wall
		if (MoverComp)
		{
			FMoverOnImpactParams ImpactParams(NAME_None, StepFwdHit, MoveDelta);
			MoverComp->HandleImpact(ImpactParams);
		}

		// Cache forwards substep before the slide attempt
		QueuedSubsteps.Add(FMovementSubstep(StepFwdSubstepName, FwdStepDelta, true));
		LocationInProgress += FwdStepDelta;

		PercentMovementUsed = StepFwdHit.Time;

		// If falling, we only need to try to reach up and fwd. We do not need to continue searching downward.
		if (bIsFalling)
		{
			// Commit queued substeps to movement record
			for (FMovementSubstep Substep : QueuedSubsteps)
			{
				InOutMoveRecord.Append(Substep);
			}

			OutFinalLocation = LocationInProgress;
			return PercentMovementUsed;
		}


		// adjust and try again
		const float ForwardHitTime = StepFwdHit.Time;

		// locking relevancy so velocity isn't added until it is needed to (adding it to the QueuedSubsteps so it can get added later)
		InOutMoveRecord.LockRelevancy(false);

		const float ForwardSlideAmount = TestGroundedMoveAlongHitSurface(MovingComps, MoveDelta, LocationInProgress, Rotation, /*bHandleImpact=*/true, MaxStepHeight, FloorCheckSettings.MaxWalkSlopeCosine, StepFwdHit, IN OUT InOutMoveRecord);

		const FVector FwdSlideDelta = (StepFwdHit.TraceStart + ((StepFwdHit.TraceEnd - StepFwdHit.TraceStart) * StepFwdHit.Time)) - LocationInProgress;

		QueuedSubsteps.Add(FMovementSubstep(SlideSubstepName, FwdSlideDelta, true));
		LocationInProgress += FwdSlideDelta;

		PercentMovementUsed += ForwardSlideAmount * (1 - PercentMovementUsed);
		
		InOutMoveRecord.UnlockRelevancy();

		if (bIsFalling)
		{
			UE_LOGF(LogMover, VeryVerbose, "Reverting step-fwd attempt during step-up, because we could not adjust without falling");
			return 0;
		}

		// If both the forward hit and the deflection got us nowhere, there is no point in this step up.
		if (ForwardHitTime == 0.f && ForwardSlideAmount == 0.f)
		{
			UE_LOGF(LogMover, VeryVerbose, "Reverting step-fwd attempt during step-up, because no movement differences occurred");
			return 0;
		}
	}
	else
	{
		// Our forward move attempt was unobstructed - cache it
		QueuedSubsteps.Add(FMovementSubstep(StepFwdSubstepName, FwdStepDelta, true));
		LocationInProgress += FwdStepDelta;
		PercentMovementUsed = 1.f;
	}


	// Step down
	const FVector StepDownAdjustment = GravDir * StepTravelDownHeight;

	const bool bDidStepDown = UAsyncMovementUtils::TestMoveComponent_Internal(MovingComps, LocationInProgress, LocationInProgress + StepDownAdjustment, Rotation, Rotation, /*bShouldSweep=*/ true, CollisionParams, StepFwdHit);


	UE_LOGF(LogMover, VeryVerbose, "TestMoveToStepOver Down: %ls (role %i) StepDownAdjustment=%ls DidMove=%i",
		*GetNameSafe(RootComponent->GetOwner()), RootComponent->GetOwnerRole(), *StepDownAdjustment.ToCompactString(), bDidStepDown);


	// If step down was initially penetrating abort the step up
	if (StepFwdHit.bStartPenetrating)
	{
		UE_LOGF(LogMover, VeryVerbose, "Reverting step-down attempt during step-up/step-fwd, because we started in a penetrating state");
		return 0;
	}

	FOptionalFloorCheckResult StepDownResult;
	if (StepFwdHit.IsValidBlockingHit())
	{
		// See if this step sequence would have allowed us to travel higher than our max step height allows.
		const float DeltaDot = (StepFwdHit.ImpactPoint.Dot(UpDirection)) - PawnFloorPointDot;
		if (DeltaDot > MaxStepHeight)
		{
			UE_LOGF(LogMover, VeryVerbose, "Reject step-down attempt during step-up/step-fwd, because it made us travel too high (too high Height %.3f) up from floor base %f to %f", DeltaDot, PawnInitialFloorBaseDot, StepFwdHit.ImpactPoint.Z);
			return 0;
		}

		// Reject unwalkable surface normals here.
		if (!UFloorQueryUtils::IsHitSurfaceWalkable(StepFwdHit, UpDirection, FloorCheckSettings.MaxWalkSlopeCosine))
		{
			// Reject if normal opposes movement direction
			const bool bNormalTowardsMe = (MoveDelta | StepFwdHit.ImpactNormal) < 0.f;
			if (bNormalTowardsMe)
			{
				UE_LOGF(LogMover, VeryVerbose, "Reject step-down attempt during step-up/step-fwd, due to unwalkable normal %ls opposed to movement", *StepFwdHit.ImpactNormal.ToString());
				return 0;
			}

			// Also reject if we would end up being higher than our starting location by stepping down.
			// It's fine to step down onto an unwalkable normal below us, we will just slide off. Rejecting those moves would prevent us from being able to walk off the edge.
			if (StepFwdHit.Location.Dot(UpDirection) > OldLocationDot)
			{
				UE_LOGF(LogMover, VeryVerbose, "Reject step-down attempt during step-up/step-fwd, due to unwalkable normal %ls above old position)", *StepFwdHit.ImpactNormal.ToString());
				return 0;
			}
		}

		// Reject moves where the downward sweep hit something very close to the edge of the shape. This maintains consistency with FindFloor as well.
		if (!UFloorQueryUtils::IsWithinEdgeTolerance(StepFwdHit.Location, StepFwdHit.ImpactPoint, PawnRadius, UpDirection))
		{
			UE_LOGF(LogMover, VeryVerbose, "Reject step-down attempt during step-up/step-fwd, due to being outside edge tolerance");
			return 0;
		}

		// Don't step up onto invalid surfaces if traveling higher.
		if (DeltaDot > 0.f && !UGroundMovementUtils::CanStepUpOnHitSurface(StepFwdHit))
		{
			UE_LOGF(LogMover, VeryVerbose, "Reject step-down attempt during step-up/step-fwd, due to being up onto surface with !CanStepUpOnHitSurface");
			return 0;
		}

		// See if we can validate the floor as a result of this step down. In almost all cases this should succeed, and we can avoid computing the floor outside this method.
		if (OutFloorTestResult != NULL)
		{

			UFloorQueryUtils::FindFloor(MovingComps, FloorCheckSettings, LocationInProgress, StepDownResult.FloorTestResult);

			// Reject unwalkable normals if we end up higher than our initial height.
			// It's fine to walk down onto an unwalkable surface, don't reject those moves.
			if (StepFwdHit.Location.Dot(UpDirection) > OldLocationDot)
			{
				// We should reject the floor result if we are trying to step up an actual step where we are not able to perch (this is rare).
				// In those cases we should instead abort the step up and try to slide along the stair.
				const float MAX_STEP_SIDE_DOT = 0.08f; // TODO: Move magic numbers elsewhere
				if (!StepDownResult.FloorTestResult.bBlockingHit && StepSideDot < MAX_STEP_SIDE_DOT)
				{
					UE_LOGF(LogMover, VeryVerbose, "Reject step-down attempt during step-up/step-fwd, due to it being an unperchable step");
					return 0;
				}
			}

			StepDownResult.bHasFloorResult = true;
		}
	}

	const FVector StepDownDelta = (StepFwdHit.TraceStart + ((StepFwdHit.TraceEnd - StepFwdHit.TraceStart) * StepFwdHit.Time)) - LocationInProgress;


	// Cache downwards substep
	QueuedSubsteps.Add(FMovementSubstep(StepDownSubstepName, StepDownDelta, false));
	LocationInProgress += StepDownDelta;

	// Copy step down result.
	if (OutFloorTestResult != NULL)
	{
		*OutFloorTestResult = StepDownResult;
	}

	// Commit queued substeps to movement record
	for (FMovementSubstep Substep : QueuedSubsteps)
	{
		InOutMoveRecord.Append(Substep);
	}

	OutFinalLocation = LocationInProgress;
	return PercentMovementUsed;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
float UGroundMovementUtils::TestMoveToStepOver(const FMovingComponentSet& MovingComps, const FVector& GravDir, float MaxStepHeight, float MaxWalkSlopeCosine, bool bUseFlatBaseForFloorChecks, float FloorSweepDistance, const FVector& MoveDelta, const FQuat& Rotation, const FHitResult& MoveHitResult, const FFloorCheckResult& CurrentFloor, bool bIsFalling, FOptionalFloorCheckResult* OutFloorTestResult, FVector& OutFinalLocation, FMovementRecord& InOutMoveRecord, float PerchRadiusThreshold)
{
	FFloorCheckSettings Settings;
	Settings.FloorSweepDistance = FloorSweepDistance;
	Settings.MaxWalkSlopeCosine = MaxWalkSlopeCosine;
	Settings.bUseFlatBaseForFloorChecks = bUseFlatBaseForFloorChecks;
	Settings.PerchRadiusThreshold = PerchRadiusThreshold;
	return TestMoveToStepOver(MovingComps, GravDir, MaxStepHeight, Settings, MoveDelta, Rotation, MoveHitResult, CurrentFloor, bIsFalling, OutFloorTestResult, OutFinalLocation, InOutMoveRecord);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
