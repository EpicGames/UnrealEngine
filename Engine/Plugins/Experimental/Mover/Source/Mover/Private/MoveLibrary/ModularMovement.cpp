// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoveLibrary/ModularMovement.h"
#include "MoverTypes.h"
#include "MoverSimulationTypes.h"
#include "MoveLibrary/MovementUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModularMovement)


FVector ULinearTurnGenerator::GetTurn_Implementation(FRotator TargetOrientation, const FMoverTickStartData& FullStartState, const FMoverDefaultSyncState& MoverState, const FMoverTimeStep& TimeStep, const FProposedMove& ProposedMove, UMoverBlackboard* SimBlackboard)
{
	FVector AngularVelocityDpS(FVector::ZeroVector);

	const float DeltaSeconds = TimeStep.StepMs * 0.001f;

	if (DeltaSeconds > 0.f)
	{
		FRotator AngularDelta = (TargetOrientation - MoverState.GetOrientation_WorldSpace());
		FRotator Winding, Remainder;

		AngularDelta.GetWindingAndRemainder(Winding, Remainder);	// to find the fastest turn, just get the (-180,180] remainder

		FRotator AngularVelocityRotator = Remainder * (1.f / DeltaSeconds);

		if (HeadingRate >= 0.f)
		{
			AngularVelocityRotator.Yaw = FMath::Clamp(AngularVelocityRotator.Yaw, -HeadingRate, HeadingRate);
		}

		if (PitchRate >= 0.f)
		{
			AngularVelocityRotator.Pitch = FMath::Clamp(AngularVelocityRotator.Pitch, -PitchRate, PitchRate);
		}

		if (RollRate >= 0.f)
		{
			AngularVelocityRotator.Roll = FMath::Clamp(AngularVelocityRotator.Roll, -RollRate, RollRate);
		}
		
		AngularDelta = AngularVelocityRotator * DeltaSeconds;
		AngularVelocityDpS = FMath::RadiansToDegrees(AngularDelta.Quaternion().ToRotationVector()) / DeltaSeconds;
	}

	return AngularVelocityDpS;
}


// Note the lack of argument range checking.  Value and Time arguments can be in any units, as long as they're consistent.
static float CalcExactDampedInterpolation(float CurrentVal, float TargetVal, float HalflifeTime, float DeltaTime)
{
	return FMath::Lerp(CurrentVal, TargetVal, 1.f - FMath::Pow(2.f, (-DeltaTime / HalflifeTime)));
}

FVector UExactDampedTurnGenerator::GetTurn_Implementation(FRotator TargetOrientation, const FMoverTickStartData& FullStartState, const FMoverDefaultSyncState& MoverState, const FMoverTimeStep& TimeStep, const FProposedMove& ProposedMove, UMoverBlackboard* SimBlackboard)
{
	FVector AngularVelocityDpS(FVector::ZeroVector);

	const float DeltaSeconds = TimeStep.StepMs * 0.001f;

	if (DeltaSeconds > 0.f && HalfLifeSeconds > UE_KINDA_SMALL_NUMBER)
	{
		FRotator AngularDelta = (TargetOrientation - MoverState.GetOrientation_WorldSpace());
		FRotator Winding, Remainder;

		AngularDelta.GetWindingAndRemainder(Winding, Remainder);	// to find the fastest turn, just get the (-180,180] remainder

		const float OneOverDeltaSecs = 1.f / DeltaSeconds;
		FRotator AngularVelocityRotator;

		AngularVelocityRotator.Yaw   = CalcExactDampedInterpolation(0.f, Remainder.Yaw,   HalfLifeSeconds, DeltaSeconds) * OneOverDeltaSecs;
		AngularVelocityRotator.Pitch = CalcExactDampedInterpolation(0.f, Remainder.Pitch, HalfLifeSeconds, DeltaSeconds) * OneOverDeltaSecs;
		AngularVelocityRotator.Roll  = CalcExactDampedInterpolation(0.f, Remainder.Roll,  HalfLifeSeconds, DeltaSeconds) * OneOverDeltaSecs;

		AngularDelta = AngularVelocityRotator * DeltaSeconds;
		AngularVelocityDpS = FMath::RadiansToDegrees(AngularDelta.Quaternion().ToRotationVector()) / DeltaSeconds;
	}

	return AngularVelocityDpS;
}
