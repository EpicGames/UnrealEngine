// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoveLibrary/MovementMixer.h"
#include "LayeredMove.h"
#include "LayeredMoveBase.h"
#include "MoverLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovementMixer)

UMovementMixer::UMovementMixer()
	: CurrentHighestPriority(0)
	, CurrentLayeredMoveStartTime(TNumericLimits<float>::Max())
{
}

void UMovementMixer::MixLayeredMove(const FLayeredMoveBase& ActiveMove, const FProposedMove& MoveStep, FProposedMove& OutCumulativeMove)
{
	if (OutCumulativeMove.PreferredMode != MoveStep.PreferredMode && !OutCumulativeMove.PreferredMode.IsNone() && !MoveStep.PreferredMode.IsNone())
	{
		UE_LOGF(LogMover, Log, "Multiple LayeredMoves are conflicting with preferred moves. %ls will override %ls",
			*MoveStep.PreferredMode.ToString(), *OutCumulativeMove.PreferredMode.ToString());
	}

	if (MoveStep.bHasDirIntent && OutCumulativeMove.MixMode != EMoveMixMode::OverrideAll && ActiveMove.Priority >= CurrentHighestPriority)
	{
		if (OutCumulativeMove.bHasDirIntent)
		{
			UE_LOGF(LogMover, Log, "Multiple LayeredMoves are setting direction intent and the layered move with highest priority will be used.");
		}
				
		OutCumulativeMove.bHasDirIntent = MoveStep.bHasDirIntent;
		OutCumulativeMove.DirectionIntent = MoveStep.DirectionIntent;
	}

	if (MoveStep.MixMode == EMoveMixMode::OverrideVelocity)
	{
		if (CheckPriority(&ActiveMove, CurrentHighestPriority, CurrentLayeredMoveStartTime))
		{
			if (OutCumulativeMove.MixMode == EMoveMixMode::OverrideVelocity || OutCumulativeMove.MixMode == EMoveMixMode::OverrideAll)
			{
				UE_LOGF(LogMover, Log, "Multiple LayeredMoves with Override mix mode are active simultaneously. Layered move with the highest priority will take effect.");
			}

			if (!MoveStep.PreferredMode.IsNone() && OutCumulativeMove.MixMode != EMoveMixMode::OverrideAll)
			{
				OutCumulativeMove.PreferredMode = MoveStep.PreferredMode;
			}
				
			OutCumulativeMove.MixMode = EMoveMixMode::OverrideVelocity;
			OutCumulativeMove.LinearVelocity  = MoveStep.LinearVelocity;
			OutCumulativeMove.AngularVelocityDegrees = MoveStep.AngularVelocityDegrees;
		}
	}
	else if (MoveStep.MixMode == EMoveMixMode::AdditiveVelocity)
	{
		if (OutCumulativeMove.MixMode != EMoveMixMode::OverrideVelocity && OutCumulativeMove.MixMode != EMoveMixMode::OverrideAll)
		{
			if (!MoveStep.PreferredMode.IsNone())
			{
				OutCumulativeMove.PreferredMode = MoveStep.PreferredMode;
			}

			OutCumulativeMove.LinearVelocity += MoveStep.LinearVelocity;
			OutCumulativeMove.AngularVelocityDegrees += MoveStep.AngularVelocityDegrees;
		}
	}
	else if (MoveStep.MixMode == EMoveMixMode::OverrideAll)
	{
		if (CheckPriority(&ActiveMove, CurrentHighestPriority, CurrentLayeredMoveStartTime))
		{
			if (OutCumulativeMove.MixMode == EMoveMixMode::OverrideVelocity || OutCumulativeMove.MixMode == EMoveMixMode::OverrideAll)
			{
				UE_LOGF(LogMover, Log, "Multiple LayeredMoves with Override mix mode are active simultaneously. Layered move with the highest priority will take effect.");
			}
				
			OutCumulativeMove = MoveStep;
			OutCumulativeMove.MixMode = EMoveMixMode::OverrideAll;
		}
	}
	else if (MoveStep.MixMode == EMoveMixMode::OverrideAllExceptVerticalVelocity)
	{
		if (CheckPriority(&ActiveMove, CurrentHighestPriority, CurrentLayeredMoveStartTime))
		{
			if (OutCumulativeMove.MixMode == EMoveMixMode::OverrideVelocity || OutCumulativeMove.MixMode == EMoveMixMode::OverrideAll || OutCumulativeMove.MixMode == EMoveMixMode::OverrideAllExceptVerticalVelocity)
			{
				UE_LOGF(LogMover, Log, "Multiple LayeredMoves with Override mix mode are active simultaneously. Layered move with the highest priority will take effect.");
			}

			OutCumulativeMove = MoveStep;
			OutCumulativeMove.MixMode = EMoveMixMode::OverrideAllExceptVerticalVelocity;
		}
	}
	else
	{
		ensureMsgf(false, TEXT("Unhandled move mix mode was found."));
	}
}

void UMovementMixer::MixLayeredMove(const FLayeredMoveInstance& ActiveMove, const FProposedMove& MoveStep, FProposedMove& OutCumulativeMove)
{
	if (OutCumulativeMove.PreferredMode != MoveStep.PreferredMode && !OutCumulativeMove.PreferredMode.IsNone() && !MoveStep.PreferredMode.IsNone())
	{
		UE_LOGF(LogMover, Log, "Multiple LayeredMoves are conflicting with preferred moves. %ls will override %ls",
			*MoveStep.PreferredMode.ToString(), *OutCumulativeMove.PreferredMode.ToString());
	}

	if (MoveStep.bHasDirIntent && OutCumulativeMove.MixMode != EMoveMixMode::OverrideAll && ActiveMove.GetPriority() >= CurrentHighestPriority)
	{
		if (OutCumulativeMove.bHasDirIntent)
		{
			UE_LOGF(LogMover, Log, "Multiple LayeredMoves are setting direction intent and the layered move with highest priority will be used.");
		}
				
		OutCumulativeMove.bHasDirIntent = MoveStep.bHasDirIntent;
		OutCumulativeMove.DirectionIntent = MoveStep.DirectionIntent;
	}

	if (MoveStep.MixMode == EMoveMixMode::OverrideVelocity)
	{
		if (CheckPriority(ActiveMove.GetPriority(), ActiveMove.GetStartingTimeMs(), CurrentHighestPriority, CurrentLayeredMoveStartTime))
		{
			if (OutCumulativeMove.MixMode == EMoveMixMode::OverrideVelocity || OutCumulativeMove.MixMode == EMoveMixMode::OverrideAll)
			{
				UE_LOGF(LogMover, Log, "Multiple LayeredMoves with Override mix mode are active simultaneously. Layered move with the highest priority will take effect.");
			}

			if (!MoveStep.PreferredMode.IsNone() && OutCumulativeMove.MixMode != EMoveMixMode::OverrideAll)
			{
				OutCumulativeMove.PreferredMode = MoveStep.PreferredMode;
			}
				
			OutCumulativeMove.MixMode = EMoveMixMode::OverrideVelocity;
			OutCumulativeMove.LinearVelocity  = MoveStep.LinearVelocity;
			OutCumulativeMove.AngularVelocityDegrees = MoveStep.AngularVelocityDegrees;
		}
	}
	else if (MoveStep.MixMode == EMoveMixMode::AdditiveVelocity)
	{
		if (OutCumulativeMove.MixMode != EMoveMixMode::OverrideVelocity && OutCumulativeMove.MixMode != EMoveMixMode::OverrideAll)
		{
			if (!MoveStep.PreferredMode.IsNone())
			{
				OutCumulativeMove.PreferredMode = MoveStep.PreferredMode;
			}

			OutCumulativeMove.LinearVelocity += MoveStep.LinearVelocity;
			OutCumulativeMove.AngularVelocityDegrees += MoveStep.AngularVelocityDegrees;
		}
	}
	else if (MoveStep.MixMode == EMoveMixMode::OverrideAll)
	{
		if (CheckPriority(ActiveMove.GetPriority(), ActiveMove.GetStartingTimeMs(), CurrentHighestPriority, CurrentLayeredMoveStartTime))
		{
			if (OutCumulativeMove.MixMode == EMoveMixMode::OverrideVelocity || OutCumulativeMove.MixMode == EMoveMixMode::OverrideAll)
			{
				UE_LOGF(LogMover, Log, "Multiple LayeredMoves with Override mix mode are active simultaneously. Layered move with the highest priority will take effect.");
			}
				
			OutCumulativeMove = MoveStep;
			OutCumulativeMove.MixMode = EMoveMixMode::OverrideAll;
		}
	}
	else if (MoveStep.MixMode == EMoveMixMode::OverrideAllExceptVerticalVelocity)
	{
		if (CheckPriority(ActiveMove.GetPriority(), ActiveMove.GetStartingTimeMs(), CurrentHighestPriority, CurrentLayeredMoveStartTime))
		{
			if (OutCumulativeMove.MixMode == EMoveMixMode::OverrideVelocity || OutCumulativeMove.MixMode == EMoveMixMode::OverrideAll || OutCumulativeMove.MixMode == EMoveMixMode::OverrideAllExceptVerticalVelocity)
			{
				UE_LOGF(LogMover, Log, "Multiple LayeredMoves with Override mix mode are active simultaneously. Layered move with the highest priority will take effect.");
			}

			OutCumulativeMove = MoveStep;
			OutCumulativeMove.MixMode = EMoveMixMode::OverrideAllExceptVerticalVelocity;
		}
	}
	else
	{
		ensureMsgf(false, TEXT("Unhandled move mix mode was found."));
	}
}

void UMovementMixer::MixProposedMoves(const FProposedMove& MoveToMix, FVector UpDirection, FProposedMove& OutCumulativeMove)
{
	if (MoveToMix.bHasDirIntent && OutCumulativeMove.MixMode != EMoveMixMode::OverrideAll)
	{
		OutCumulativeMove.bHasDirIntent = MoveToMix.bHasDirIntent;
		OutCumulativeMove.DirectionIntent = MoveToMix.DirectionIntent;
	}

	// Combine movement parameters from layered moves into what the mode wants to do
	if (MoveToMix.MixMode == EMoveMixMode::OverrideAll)
	{
		OutCumulativeMove = MoveToMix;
	}
	else if (MoveToMix.MixMode == EMoveMixMode::AdditiveVelocity)
	{
		OutCumulativeMove.LinearVelocity += MoveToMix.LinearVelocity;
		OutCumulativeMove.AngularVelocityDegrees += MoveToMix.AngularVelocityDegrees;
	}
	else if (MoveToMix.MixMode == EMoveMixMode::OverrideVelocity)
	{
		OutCumulativeMove.LinearVelocity = MoveToMix.LinearVelocity;
		OutCumulativeMove.AngularVelocityDegrees = MoveToMix.AngularVelocityDegrees;
	}
	else if (MoveToMix.MixMode == EMoveMixMode::OverrideAllExceptVerticalVelocity)
	{
		const FVector IncomingVerticalVelocity = MoveToMix.LinearVelocity.ProjectOnToNormal(UpDirection);
		const FVector IncomingNonVerticalVelocity = MoveToMix.LinearVelocity - IncomingVerticalVelocity;
		const FVector ExistingVerticalVelocity = OutCumulativeMove.LinearVelocity.ProjectOnToNormal(UpDirection);

		OutCumulativeMove = MoveToMix;
		OutCumulativeMove.LinearVelocity = IncomingNonVerticalVelocity + ExistingVerticalVelocity;
	}
	else
	{
		ensureMsgf(false, TEXT("Unhandled move mix mode was found."));
	}
}

void UMovementMixer::ResetMixerState()
{
	CurrentHighestPriority = 0;
	CurrentLayeredMoveStartTime = TNumericLimits<double>::Max();
}

bool UMovementMixer::CheckPriority(const FLayeredMoveBase* LayeredMove, uint8& InOutHighestPriority, double& InOutCurrentLayeredMoveStartTimeMs)
{
	if (LayeredMove->Priority > InOutHighestPriority)
	{
		InOutHighestPriority = LayeredMove->Priority;
		InOutCurrentLayeredMoveStartTimeMs = LayeredMove->StartSimTimeMs;
		return true;
	}
	if (LayeredMove->Priority == InOutHighestPriority && LayeredMove->StartSimTimeMs < InOutCurrentLayeredMoveStartTimeMs)
	{
		InOutCurrentLayeredMoveStartTimeMs = LayeredMove->StartSimTimeMs;
		return true;
	}

	return false;
}

bool UMovementMixer::CheckPriority(const uint8 LayeredMovePriority, const double LayeredMoveStartTimeMs, uint8& InOutHighestPriority, double& InOutCurrentLayeredMoveStartTimeMs)
{
	if (LayeredMovePriority > InOutHighestPriority)
	{
		InOutHighestPriority = LayeredMovePriority;
		InOutCurrentLayeredMoveStartTimeMs = LayeredMoveStartTimeMs;
		return true;
	}
	if (LayeredMovePriority == InOutHighestPriority && LayeredMoveStartTimeMs < InOutCurrentLayeredMoveStartTimeMs)
	{
		InOutCurrentLayeredMoveStartTimeMs = LayeredMoveStartTimeMs;
		return true;
	}

	return false;
}
