// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoveLibrary/MovementRecord.h"
#include "MoverLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovementRecord)

void FMovementRecord::Reset()
{
	TotalMoveDelta = FVector::ZeroVector;
	RelevantMoveDelta = FVector::ZeroVector;
	TotalDeltaSeconds = 0.f;

	bIsRelevancyLocked = false;
	bRelevancyLockValue = false;

	Substeps.Empty(Substeps.Num());	// leaving space for the next user, as this is likely to fill up again
}

void FMovementRecord::Append(FMovementSubstep Substep)
{
	if (bIsRelevancyLocked)
	{
		Substep.bIsRelevant = bRelevancyLockValue;
	}

	if (Substep.bIsRelevant)
	{
		RelevantMoveDelta += Substep.MoveDelta;
	}

	TotalMoveDelta += Substep.MoveDelta;

	Substeps.Add(Substep);
}

FString FMovementRecord::ToString() const
{
	return FString::Printf( TEXT("TotalMove: %s over %.3f seconds. RelevantVelocity: %s. Substeps: %s"),
		*TotalMoveDelta.ToCompactString(),
		TotalDeltaSeconds,
		*GetRelevantVelocity().ToCompactString(),
		*FString::JoinBy(Substeps, TEXT(","), [](const FMovementSubstep& Substep) { return Substep.MoveName.ToString(); }));
}
