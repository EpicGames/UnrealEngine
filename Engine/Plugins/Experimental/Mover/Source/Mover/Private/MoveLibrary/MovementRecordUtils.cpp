// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoveLibrary/MovementRecordUtils.h"
#include "MoveLibrary/MovementRecord.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovementRecordUtils)

void UMovementRecordUtils::K2_SetDeltaSeconds(FMovementRecord& OutMovementRecord, float DeltaSeconds)
{
	OutMovementRecord.SetDeltaSeconds(DeltaSeconds);
}

const FVector& UMovementRecordUtils::K2_GetTotalMoveDelta(const FMovementRecord& MovementRecord)
{
	return MovementRecord.GetTotalMoveDelta();
}

const FVector& UMovementRecordUtils::K2_GetRelevantMoveDelta(const FMovementRecord& MovementRecord)
{
	return MovementRecord.GetRelevantMoveDelta();
}

FVector UMovementRecordUtils::K2_GetRelevantVelocity(const FMovementRecord& MovementRecord)
{
	return MovementRecord.GetRelevantVelocity();
}	
