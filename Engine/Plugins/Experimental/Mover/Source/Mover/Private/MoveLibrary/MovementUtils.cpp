// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoveLibrary/MovementUtils.h"
#include "MoveLibrary/MovementRecord.h"
#include "MoverLog.h"
#include "Components/BoxComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/OverlapResult.h"
#include "Engine/ScopedMovementUpdate.h"
#include "Engine/World.h"
#include "GameFramework/Pawn.h"
#include "MoverComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovementUtils)

namespace UE::MoverUtils
{
	const double SMALL_MOVE_DISTANCE = 1e-3;
	const double VERTICAL_SLOPE_NORMAL_MAX_DOT = 0.001f; // Slope is vertical if Abs(UpDirection) <= this threshold. Accounts for precision problems that sometimes angle normals slightly off horizontal for vertical surface.

	const float  VELOCITY_INPUT_NO_ACCELERATION_DIFFERENCE = 1.01f;
}

bool UMovementUtils::IsExceedingMaxSpeed(const FVector& Velocity, float InMaxSpeed)
{
	InMaxSpeed = FMath::Max(0.f, InMaxSpeed);
	const float MaxSpeedSquared = FMath::Square(InMaxSpeed);

	// Allow 1% error tolerance, to account for numeric imprecision.
	const float OverVelocityPercent = 1.01f;
	return (Velocity.SizeSquared() > MaxSpeedSquared * OverVelocityPercent);
}


FVector UMovementUtils::ComputeVelocity(const FComputeVelocityParams& InParams)
{
	FVector Acceleration = FVector::ZeroVector;
	FVector Velocity = InParams.InitialVelocity;
	float DesiredSpeed = 0.0f;

	if (InParams.MoveInputType == EMoveInputType::Velocity)
	{
		const float RequestedSpeed = FMath::Min(InParams.MaxSpeed,  InParams.MoveInput.Size());
		const FVector RequestedMoveDir = InParams.MoveInput.GetSafeNormal();
		DesiredSpeed = RequestedSpeed;
		const FVector MoveVelocity = RequestedMoveDir * DesiredSpeed;

		// Compute acceleration. Use MaxAccel to limit speed increase
		if (InParams.bUseAccelerationForVelocityMove && InParams.InitialVelocity.Size() < RequestedSpeed * UE::MoverUtils::VELOCITY_INPUT_NO_ACCELERATION_DIFFERENCE)
		{
			// Turn in the same manner as with input acceleration.
			Velocity = Velocity - (Velocity - RequestedMoveDir * RequestedSpeed) * FMath::Min(InParams.DeltaSeconds * InParams.Friction, 1.f);

			// How much do we need to accelerate to get to the new velocity?
			Acceleration = (MoveVelocity - Velocity) / InParams.DeltaSeconds;
			Acceleration = Acceleration.GetClampedToMaxSize(InParams.Acceleration);
		}
		else
		{
			// Just set velocity directly.
			// If decelerating we do so instantly, so we don't slide through the destination if we can't brake fast enough.
			Velocity = MoveVelocity;
		}
	}
	else if (InParams.MoveInputType == EMoveInputType::DirectionalIntent)
	{
		const FVector ControlAcceleration = InParams.MoveDirectionIntent.GetClampedToMaxSize(1.f);
		const float AnalogInputModifier = (ControlAcceleration.SizeSquared() > 0.f ? ControlAcceleration.Size() : 0.f);
		DesiredSpeed = InParams.MaxSpeed * AnalogInputModifier;

		if (Velocity.SizeSquared() > 0.f)
		{
			if (!InParams.MoveDirectionIntent.IsNearlyZero() && AnalogInputModifier > 0.f )
			{
				const FVector VelocityAlongInput = Velocity.ProjectOnTo(InParams.MoveDirectionIntent);
				const bool bExceedingMaxSpeedAlongInput = IsExceedingMaxSpeed(VelocityAlongInput, DesiredSpeed);
			
				if (!bExceedingMaxSpeedAlongInput)
				{
					// Apply change in velocity direction
					// Change direction faster than only using acceleration, but never increase velocity magnitude.
					const float TimeScale = FMath::Clamp(InParams.DeltaSeconds * InParams.TurningBoost, 0.f, 1.f);
					Velocity = Velocity + (ControlAcceleration * Velocity.Size() - Velocity) * FMath::Min(TimeScale * InParams.Friction, 1.f);
				}
			}
			
			const bool bExceedingMaxSpeed = IsExceedingMaxSpeed(Velocity, DesiredSpeed);
			if (bExceedingMaxSpeed)
			{
				// Dampen velocity magnitude based on deceleration.
				const FVector OldVelocity = Velocity;
				const float VelSize = FMath::Max(Velocity.Size() - FMath::Abs(InParams.Friction * Velocity.Size() + InParams.Deceleration) * InParams.DeltaSeconds, 0.f);
				Velocity = Velocity.GetSafeNormal() * VelSize;

				// Don't allow braking to lower us below max speed if we started above it.
				if (bExceedingMaxSpeed && Velocity.SizeSquared() < FMath::Square(DesiredSpeed))
				{
					Velocity = OldVelocity.GetSafeNormal() * DesiredSpeed;
				}
			}
		}
		
		Acceleration = ControlAcceleration * FMath::Abs(InParams.Acceleration);
	}
	else
	{
		UE_CLOGF((InParams.MoveInputType == EMoveInputType::Invalid), LogMover, Warning, "Mover Compute Velocity has received an invalid input type and no velocity will be generated!");
		return FVector::ZeroVector;
	}

	// Apply acceleration and clamp velocity magnitude.
	const float NewMaxSpeed = (IsExceedingMaxSpeed(Velocity, DesiredSpeed)) ? Velocity.Size() : DesiredSpeed;
	Velocity += Acceleration * InParams.DeltaSeconds;
	Velocity = Velocity.GetClampedToMaxSize(NewMaxSpeed);

	return Velocity;
}

FVector UMovementUtils::ComputeCombinedVelocity(const FComputeCombinedVelocityParams& InParams)
{
	const FVector ControlAcceleration = InParams.MoveDirectionIntent.GetClampedToMaxSize(1.f);
	FVector Velocity = InParams.InitialVelocity;

	const float AnalogInputModifier = (ControlAcceleration.SizeSquared() > 0.f ? ControlAcceleration.Size() : 0.f);
	
	const float MaxInputSpeed = InParams.MaxSpeed * AnalogInputModifier;
	const float MaxSpeed = FMath::Max(InParams.OverallMaxSpeed, MaxInputSpeed);
	
	const bool bExceedingMaxSpeed = IsExceedingMaxSpeed(Velocity, MaxSpeed);

	if ((AnalogInputModifier > KINDA_SMALL_NUMBER || InParams.ExternalAcceleration.Size() > KINDA_SMALL_NUMBER) && !bExceedingMaxSpeed)
	{
		// Apply change in velocity direction
		if (Velocity.SizeSquared() > 0.f)
		{
			// Change direction faster than only using acceleration, but never increase velocity magnitude.
			const float TimeScale = FMath::Clamp(InParams.DeltaSeconds * InParams.TurningBoost, 0.f, 1.f);
			Velocity = Velocity + (ControlAcceleration * Velocity.Size() - Velocity) * FMath::Min(TimeScale * InParams.Friction, 1.f);
		}
	}
	else
	{
		// Dampen velocity magnitude based on deceleration.
		if (Velocity.SizeSquared() > 0.f)
		{
			const FVector OldVelocity = Velocity;
			const float VelSize = FMath::Max(Velocity.Size() - FMath::Abs(InParams.Friction * Velocity.Size() + InParams.Deceleration) * InParams.DeltaSeconds, 0.f);
			Velocity = Velocity.GetSafeNormal() * VelSize;

			// Don't allow braking to lower us below max speed if we started above it.
			if (bExceedingMaxSpeed && Velocity.SizeSquared() < FMath::Square(MaxSpeed))
			{
				Velocity = OldVelocity.GetSafeNormal() * MaxSpeed;
			}
		}
	}

	// Apply input acceleration and clamp velocity magnitude.
	const float NewMaxInputSpeed = (IsExceedingMaxSpeed(Velocity, MaxInputSpeed)) ? Velocity.Size() : MaxInputSpeed;
	Velocity += ControlAcceleration * FMath::Abs(InParams.Acceleration) * InParams.DeltaSeconds;
	Velocity = Velocity.GetClampedToMaxSize(NewMaxInputSpeed);

	// Apply move requested acceleration
	const float NewMaxMoveSpeed = (IsExceedingMaxSpeed(Velocity, InParams.OverallMaxSpeed)) ? Velocity.Size() : InParams.OverallMaxSpeed;
	Velocity += InParams.ExternalAcceleration * InParams.DeltaSeconds;
	Velocity = Velocity.GetClampedToMaxSize(NewMaxMoveSpeed);

	return Velocity;
}

FVector UMovementUtils::DeduceUpDirectionFromGravity(const FVector& GravityAcceleration)
{
	const FVector DeducedUpDir = -GravityAcceleration.GetSafeNormal();

	if (DeducedUpDir.IsZero())
	{
		return MoverComponentConstants::DefaultUpDir;
	}

	return DeducedUpDir;
}

bool UMovementUtils::CanEscapeGravity(const FVector& PriorVelocity, const FVector& NewVelocity, const FVector& GravityAccel, float DeltaSeconds)
{
	if (DeltaSeconds > UE_SMALL_NUMBER)
	{
		const FVector VelocityDelta = NewVelocity - PriorVelocity;
		const FVector Acceleration = VelocityDelta / DeltaSeconds;
		const FVector AccelOntoGravity = Acceleration.ProjectOnTo(GravityAccel);

		// If acceleration opposes gravity and is stronger, then it can escape
		if (AccelOntoGravity.Dot(GravityAccel) < 0.0 &&
			AccelOntoGravity.SizeSquared() > GravityAccel.SizeSquared())
		{
			return true;
		}
	}

	return false;
}

FVector UMovementUtils::ConstrainToPlane(const FVector& Vector, const FPlane& MovementPlane, bool bMaintainMagnitude)
{
	FVector ConstrainedResult = FVector::PointPlaneProject(Vector, MovementPlane);

	if (bMaintainMagnitude)
	{
		ConstrainedResult = ConstrainedResult.GetSafeNormal() * Vector.Size();
	}

	return ConstrainedResult;
}

FRotator UMovementUtils::ApplyGravityToOrientationIntent(const FRotator& IntendedOrientation, const FQuat& WorldToGravity, bool bStayVertical)
{
	if (!bStayVertical)
	{
		return IntendedOrientation;
	}

	FRotator GravityRelativeDesiredRotation = (WorldToGravity.Inverse() * IntendedOrientation.Quaternion()).Rotator();	// world space -> gravity-relative space
	
	GravityRelativeDesiredRotation.Pitch = 0.f;
	GravityRelativeDesiredRotation.Yaw = FRotator::NormalizeAxis(GravityRelativeDesiredRotation.Yaw);
	GravityRelativeDesiredRotation.Roll = 0.f;
	
	return (WorldToGravity * GravityRelativeDesiredRotation.Quaternion()).Rotator();	// gravity-relative space -> world space
}

FVector UMovementUtils::ComputeSlideDelta(const FMovingComponentSet& MovingComps, const FVector& Delta, const float PctOfDeltaToMove, const FVector& Normal, const FHitResult& Hit)
{
	FVector ConstrainedNormal = Normal;

	if (MovingComps.MoverComponent.IsValid())
	{
		ConstrainedNormal = UPlanarConstraintUtils::ConstrainNormalToPlane(MovingComps.MoverComponent->GetPlanarConstraint(), Normal);
	}

	return FVector::VectorPlaneProject(Delta, ConstrainedNormal) * PctOfDeltaToMove;
}

FVector UMovementUtils::ComputeTwoWallAdjustedDelta(const FMovingComponentSet& MovingComps, const FVector& MoveDelta, const FHitResult& Hit, const FVector& OldHitNormal)
{
	FVector Delta = MoveDelta;
	const FVector HitNormal = Hit.Normal;

	if ((OldHitNormal | HitNormal) <= 0.f) //90 or less corner, so use cross product for direction
	{
		const FVector DesiredDir = Delta;
		FVector NewDir = (HitNormal ^ OldHitNormal);
		NewDir = NewDir.GetSafeNormal();
		Delta = (Delta | NewDir) * (1.f - Hit.Time) * NewDir;
		if ((DesiredDir | Delta) < 0.f)
		{
			Delta = -1.f * Delta;
		}
	}
	else //adjust to new wall
	{
		const FVector DesiredDir = Delta;
		Delta = UMovementUtils::ComputeSlideDelta(MovingComps, Delta, 1.f - Hit.Time, HitNormal, Hit);
		if ((Delta | DesiredDir) <= 0.f)
		{
			Delta = FVector::ZeroVector;
		}
		else if (FMath::Abs((HitNormal | OldHitNormal) - 1.f) < KINDA_SMALL_NUMBER)
		{
			// we hit the same wall again even after adjusting to move along it the first time
			// nudge away from it (this can happen due to precision issues)
			Delta += HitNormal * 0.01f;
		}
	}

	return Delta;
}

float UMovementUtils::TryMoveToSlideAlongSurface(const FMovingComponentSet& MovingComps, const FVector& Delta, float PctOfDeltaToMove, const FQuat Rotation, const FVector& Normal, FHitResult& Hit, bool bHandleImpact, FMovementRecord& MoveRecord)
{
	if (!Hit.bBlockingHit)
	{
		return 0.f;
	}

	float PctOfTimeUsed = 0.f;
	const FVector OldHitNormal = Normal;

	FVector SlideDelta = UMovementUtils::ComputeSlideDelta(MovingComps, Delta, PctOfDeltaToMove, Normal, Hit);

	if ((SlideDelta | Delta) > 0.f)
	{
		TrySafeMoveUpdatedComponent(MovingComps, SlideDelta, Rotation, true, Hit, ETeleportType::None, MoveRecord);

		PctOfTimeUsed = Hit.Time;

		if (Hit.IsValidBlockingHit())
		{
			// Notify first impact
			if (MovingComps.MoverComponent.IsValid() && bHandleImpact)
			{
				FMoverOnImpactParams ImpactParams(NAME_None, Hit, SlideDelta);
				MovingComps.MoverComponent->HandleImpact(ImpactParams);
			}

			// Compute new slide normal when hitting multiple surfaces.
			SlideDelta = UMovementUtils::ComputeTwoWallAdjustedDelta(MovingComps, SlideDelta, Hit, OldHitNormal);

			// Only proceed if the new direction is of significant length and not in reverse of original attempted move.
			if (!SlideDelta.IsNearlyZero(UE::MoverUtils::SMALL_MOVE_DISTANCE) && (SlideDelta | Delta) > 0.f)
			{
				// Perform second move
				TrySafeMoveUpdatedComponent(MovingComps, SlideDelta, Rotation, true, Hit, ETeleportType::None, MoveRecord);
				PctOfTimeUsed += (Hit.Time * (1.f - PctOfTimeUsed));

				// Notify second impact
				if (MovingComps.MoverComponent.IsValid() && bHandleImpact && Hit.bBlockingHit)
				{
					FMoverOnImpactParams ImpactParams(NAME_None, Hit, SlideDelta);
					MovingComps.MoverComponent->HandleImpact(ImpactParams);
				}
			}
		}

		return FMath::Clamp(PctOfTimeUsed, 0.f, 1.f);
	}

	return 0.f;
}

float UMovementUtils::TryMoveToSlideAlongSurfaceNoMovementRecord(const FMovingComponentSet& MovingComps, const FVector& Delta, float PctOfDeltaToMove, const FQuat Rotation, const FVector& Normal, FHitResult& Hit, bool bHandleImpact)
{
	FMovementRecord TempMoveRecord;
	return TryMoveToSlideAlongSurface(MovingComps, Delta, PctOfDeltaToMove, Rotation, Normal, Hit, bHandleImpact, TempMoveRecord);
}

float UMovementUtils::TrySafeMoveAndSlideUpdatedComponent(const FMovingComponentSet& MovingComps, const FVector& Delta, const FQuat& NewRotation, bool bSweep, FHitResult& OutHit, ETeleportType Teleport, FMovementRecord& MoveRecord, bool bSlideAlongSurface)
{
	TrySafeMoveUpdatedComponent(MovingComps, Delta, NewRotation, bSweep, OutHit, Teleport, MoveRecord);

	float PercentMovementApplied = OutHit.Time;
	
	if (OutHit.IsValidBlockingHit())
	{
		if (bSlideAlongSurface)
		{
			UMoverComponent* MoverComponent = MovingComps.MoverComponent.Get();
			FMoverOnImpactParams ImpactParams(DefaultModeNames::Flying, OutHit, Delta);
			MoverComponent->HandleImpact(ImpactParams);
			// Try to slide the remaining distance along the surface.
			TryMoveToSlideAlongSurface(FMovingComponentSet(MoverComponent), Delta, 1.f - OutHit.Time, NewRotation, OutHit.Normal, OutHit, true, MoveRecord);
			PercentMovementApplied = OutHit.Time;
		}
	}
	else
	{
		PercentMovementApplied = 1.0f;
	}

	return PercentMovementApplied;
}

float UMovementUtils::TrySafeMoveAndSlideUpdatedComponentNoMovementRecord(const FMovingComponentSet& MovingComps, const FVector& Delta, const FQuat& NewRotation, bool bSweep, FHitResult& OutHit, ETeleportType Teleport, bool bSlideAlongSurface)
{
	FMovementRecord TempRecord;
	return TrySafeMoveAndSlideUpdatedComponent(MovingComps, Delta, NewRotation, bSweep, OutHit, Teleport, TempRecord);
}


static const FName SafeMoveSubstepName = "SafeMove";

bool UMovementUtils::TrySafeMoveUpdatedComponent(const FMovingComponentSet& MovingComps, const FVector& Delta, const FQuat& NewRotation, bool bSweep, FHitResult& OutHit, ETeleportType Teleport, FMovementRecord& MoveRecord)
{
	USceneComponent* UpdatedComponent = MovingComps.UpdatedComponent.Get();

	if (!UpdatedComponent)
	{
		OutHit.Reset(1.f);
		return false;
	}

	bool bMoveResult = false;
	EMoveComponentFlags MoveComponentFlags = MOVECOMP_NoFlags;

	FVector PreviousCompPos = UpdatedComponent->GetComponentLocation();

	// Scope for move flags
	{
		// Conditionally ignore blocking overlaps (based on CVar)
		const EMoveComponentFlags IncludeBlockingOverlapsWithoutEvents = (MOVECOMP_NeverIgnoreBlockingOverlaps | MOVECOMP_DisableBlockingOverlapDispatch);
		//TGuardValue<EMoveComponentFlags> ScopedFlagRestore(MoveComponentFlags, MovementComponentCVars::MoveIgnoreFirstBlockingOverlap ? MoveComponentFlags : (MoveComponentFlags | IncludeBlockingOverlapsWithoutEvents));
		MoveComponentFlags = (MoveComponentFlags | IncludeBlockingOverlapsWithoutEvents);
		bMoveResult = TryMoveUpdatedComponent_Internal(MovingComps, Delta, NewRotation, bSweep, MoveComponentFlags, &OutHit, Teleport);

		if (UpdatedComponent)
		{
			UE_LOGF(LogMover, VeryVerbose, "TrySafeMove: %ls (role %i) Delta=%ls DidMove=%i", 
				*GetNameSafe(UpdatedComponent->GetOwner()), UpdatedComponent->GetOwnerRole(), *Delta.ToCompactString(), bMoveResult);
		}
	}

	// Handle initial penetrations
	if (OutHit.bStartPenetrating && UpdatedComponent)
	{
		const FVector RequestedAdjustment = ComputePenetrationAdjustment(OutHit);
		if (TryMoveToResolvePenetration(MovingComps, MoveComponentFlags, RequestedAdjustment, OutHit, NewRotation, MoveRecord))
		{
			PreviousCompPos = UpdatedComponent->GetComponentLocation();

			// Retry original move
			bMoveResult = TryMoveUpdatedComponent_Internal(MovingComps, Delta, NewRotation, bSweep, MoveComponentFlags, &OutHit, Teleport);

			UE_LOGF(LogMover, VeryVerbose, "TrySafeMove retry: %ls (role %i) Delta=%ls DidMove=%i",
				*GetNameSafe(UpdatedComponent->GetOwner()), UpdatedComponent->GetOwnerRole(), *Delta.ToCompactString(), bMoveResult);
		}
	}

	if (bMoveResult && UpdatedComponent)
	{
		MoveRecord.Append( FMovementSubstep(SafeMoveSubstepName, UpdatedComponent->GetComponentLocation()-PreviousCompPos, true) );
	}


	return bMoveResult;
}

bool UMovementUtils::TrySafeMoveUpdatedComponentNoMovementRecord(const FMovingComponentSet& MovingComps, const FVector& Delta, const FQuat& NewRotation, bool bSweep, FHitResult& OutHit, ETeleportType Teleport)
{
	FMovementRecord TempMovementRecord;
	return TrySafeMoveUpdatedComponent(MovingComps, Delta, NewRotation, bSweep, OutHit, Teleport, TempMovementRecord);
}

FVector UMovementUtils::ComputePenetrationAdjustment(const FHitResult& Hit)
{
	if (!Hit.bStartPenetrating)
	{
		return FVector::ZeroVector;
	}

	FVector Result;
	const float PullBackDistance = 0.125f; //FMath::Abs(BaseMovementCVars::PenetrationPullbackDistance);
	const float PenetrationDepth = (Hit.PenetrationDepth > 0.f ? Hit.PenetrationDepth : 0.125f);

	Result = Hit.Normal * (PenetrationDepth + PullBackDistance);

	return Result;
}


static const FName PenetrationResolutionSubstepName = "ResolvePenetration";

bool UMovementUtils::TryMoveToResolvePenetration(const FMovingComponentSet& MovingComps, EMoveComponentFlags MoveComponentFlags, const FVector& ProposedAdjustment, const FHitResult& Hit, const FQuat& NewRotationQuat, FMovementRecord& MoveRecord)
{
	USceneComponent* UpdatedComponent = MovingComps.UpdatedComponent.Get();
	UPrimitiveComponent* UpdatedPrimitive = Cast<UPrimitiveComponent>(UpdatedComponent);
	UMoverComponent* MoverComp = MovingComps.MoverComponent.Get();

	// SceneComponent can't be in penetration, so this function really only applies to PrimitiveComponent.
	const FVector Adjustment = UPlanarConstraintUtils::ConstrainDirectionToPlane(MovingComps.MoverComponent->GetPlanarConstraint(), ProposedAdjustment);
	if (!Adjustment.IsZero() && UpdatedPrimitive)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_BaseMovementComponent_ResolvePenetration);
		// See if we can fit at the adjusted location without overlapping anything.
		AActor* ActorOwner = MoverComp->GetOwner();
		if (!ActorOwner)
		{
			return false;
		}

		const FVector OriginalCompPos = UpdatedComponent->GetComponentLocation();

		// We really want to make sure that precision differences or differences between the overlap test and sweep tests don't put us into another overlap,
		// so make the overlap test a bit more restrictive.
		const float OverlapInflation = 0.1f; //BaseMovementCVars::PenetrationOverlapCheckInflation;
		bool bEncroached = OverlapTest(UpdatedComponent, UpdatedPrimitive, Hit.TraceStart + Adjustment, NewRotationQuat, UpdatedPrimitive->GetCollisionObjectType(), UpdatedPrimitive->GetCollisionShape(OverlapInflation), ActorOwner);
		if (!bEncroached)
		{
			// Move without sweeping.
			const bool bDidMove = TryMoveUpdatedComponent_Internal(UpdatedComponent, Adjustment, NewRotationQuat, false, MoveComponentFlags, nullptr, ETeleportType::TeleportPhysics);

			UE_LOGF(LogMover, VeryVerbose, "TryMoveToResolvePenetration unencroached: %ls (role %i) Adjustment=%ls DidMove=%i",
				*GetNameSafe(UpdatedComponent->GetOwner()), UpdatedComponent->GetOwnerRole(), *Adjustment.ToCompactString(), bDidMove);

			MoveRecord.Append( FMovementSubstep(PenetrationResolutionSubstepName, UpdatedComponent->GetComponentLocation()-OriginalCompPos, false) );
			return true;
		}
		else
		{
			// Disable MOVECOMP_NeverIgnoreBlockingOverlaps if it is enabled, otherwise we wouldn't be able to sweep out of the object to fix the penetration.
			TGuardValue<EMoveComponentFlags> ScopedFlagRestore(MoveComponentFlags, EMoveComponentFlags(MoveComponentFlags & (~MOVECOMP_NeverIgnoreBlockingOverlaps)));

			// Try sweeping as far as possible...
			FHitResult SweepOutHit(1.f);
			bool bMoved = TryMoveUpdatedComponent_Internal(MovingComps, Adjustment, NewRotationQuat, true, MoveComponentFlags, &SweepOutHit, ETeleportType::TeleportPhysics);

			UE_LOGF(LogMover, VeryVerbose, "TryMoveToResolvePenetration: %ls (role %i) Adjustment=%ls DidMove=%i",
				*GetNameSafe(UpdatedComponent->GetOwner()), UpdatedComponent->GetOwnerRole(), *Adjustment.ToCompactString(), bMoved);

			// Still stuck?
			if (!bMoved && SweepOutHit.bStartPenetrating)
			{
				// Combine two MTD results to get a new direction that gets out of multiple surfaces.
				const FVector SecondMTD = ComputePenetrationAdjustment(SweepOutHit);
				const FVector CombinedMTD = Adjustment + SecondMTD;
				if (SecondMTD != Adjustment && !CombinedMTD.IsZero())
				{
					bMoved = TryMoveUpdatedComponent_Internal(MovingComps, CombinedMTD, NewRotationQuat, true, MoveComponentFlags, nullptr, ETeleportType::TeleportPhysics);

					UE_LOGF(LogMover, VeryVerbose, "TryMoveToResolvePenetration combined: %ls (role %i) CombinedAdjustment=%ls DidMove=%i",
						*GetNameSafe(UpdatedComponent->GetOwner()), UpdatedComponent->GetOwnerRole(), *CombinedMTD.ToCompactString(), bMoved);
				}
			}

			// Still stuck?
			if (!bMoved)
			{
				// Try moving the proposed adjustment plus the attempted move direction. This can sometimes get out of penetrations with multiple objects
				const FVector MoveDelta = UPlanarConstraintUtils::ConstrainDirectionToPlane(MoverComp->GetPlanarConstraint(), (Hit.TraceEnd - Hit.TraceStart));
				if (!MoveDelta.IsZero())
				{
					const FVector AdjustAndMoveDelta = Adjustment + MoveDelta;
					bMoved = TryMoveUpdatedComponent_Internal(MovingComps, AdjustAndMoveDelta, NewRotationQuat, true, MoveComponentFlags, nullptr, ETeleportType::TeleportPhysics);

					UE_LOGF(LogMover, VeryVerbose, "TryMoveToResolvePenetration multiple: %ls (role %i) AdjustAndMoveDelta=%ls DidMove=%i",
						*GetNameSafe(UpdatedComponent->GetOwner()), UpdatedComponent->GetOwnerRole(), *AdjustAndMoveDelta.ToCompactString(), bMoved);
				}
			}

			if (bMoved)
			{
				MoveRecord.Append( FMovementSubstep(PenetrationResolutionSubstepName, UpdatedComponent->GetComponentLocation()-OriginalCompPos, false) );
			}
			
			return bMoved;
		}
	}

	return false;
}


void UMovementUtils::InitCollisionParams(const UPrimitiveComponent* UpdatedPrimitive, FCollisionQueryParams& OutParams, FCollisionResponseParams& OutResponseParam)
{
	if (UpdatedPrimitive)
	{
		UpdatedPrimitive->InitSweepCollisionParams(OutParams, OutResponseParam);
	}
}


bool UMovementUtils::OverlapTest(const USceneComponent* UpdatedComponent, const UPrimitiveComponent* UpdatedPrimitive, const FVector& Location, const FQuat& RotationQuat, const ECollisionChannel CollisionChannel, const FCollisionShape& CollisionShape, const AActor* IgnoreActor)
{
	FCollisionQueryParams QueryParams(SCENE_QUERY_STAT(MovementOverlapTest), false, IgnoreActor);
	FCollisionResponseParams ResponseParam;
	InitCollisionParams(UpdatedPrimitive, QueryParams, ResponseParam);
	return UpdatedComponent->GetWorld()->OverlapBlockingTestByChannel(Location, RotationQuat, CollisionChannel, CollisionShape, QueryParams, ResponseParam);
}

FVector UMovementUtils::ComputeVelocityFromPositions(const FVector& FromPos, const FVector& ToPos, float DeltaSeconds)
{
	if (DeltaSeconds > 0.f)
	{ 
		return (ToPos - FromPos) / DeltaSeconds;
	}
	
	return FVector::ZeroVector;
}

FRotator UMovementUtils::ComputeAngularVelocity(const FRotator& FromOrientation, const FRotator& ToOrientation, const FQuat& WorldToGravity, float DeltaSeconds, float TurningRateLimit)
{
	FRotator AngularVelocityDpS(FRotator::ZeroRotator);

	const float AngleTolerance = 1e-3f;
	if (!FromOrientation.Equals(ToOrientation, AngleTolerance))
	{
		FRotator GravityRelativeCurrentRotation = (WorldToGravity.Inverse() * FromOrientation.Quaternion()).Rotator();
		FRotator GravityRelativeDesiredRotation = (WorldToGravity.Inverse() * ToOrientation.Quaternion()).Rotator();

		const FRotator RotationDelta = GravityRelativeDesiredRotation - GravityRelativeCurrentRotation;
		const float YawTurnRateLimit   = FMath::Clamp(RotationDelta.Yaw,   -TurningRateLimit, TurningRateLimit);
		const float PitchTurnRateLimit = FMath::Clamp(RotationDelta.Pitch, -TurningRateLimit, TurningRateLimit);
		const float RollTurnRateLimit  = FMath::Clamp(RotationDelta.Roll,  -TurningRateLimit, TurningRateLimit);
		
		// PITCH
		if (!FMath::IsNearlyEqual(GravityRelativeCurrentRotation.Pitch, GravityRelativeDesiredRotation.Pitch, AngleTolerance))
		{
			GravityRelativeDesiredRotation.Pitch = FMath::FixedTurn(GravityRelativeCurrentRotation.Pitch, GravityRelativeDesiredRotation.Pitch, TurningRateLimit * DeltaSeconds);
		}

		// YAW
		if (!FMath::IsNearlyEqual(GravityRelativeCurrentRotation.Yaw, GravityRelativeDesiredRotation.Yaw, AngleTolerance))
		{
			GravityRelativeDesiredRotation.Yaw = FMath::FixedTurn(GravityRelativeCurrentRotation.Yaw, GravityRelativeDesiredRotation.Yaw, TurningRateLimit * DeltaSeconds);
		}

		// ROLL
		if (!FMath::IsNearlyEqual(GravityRelativeCurrentRotation.Roll, GravityRelativeDesiredRotation.Roll, AngleTolerance))
		{
			GravityRelativeDesiredRotation.Roll = FMath::FixedTurn(GravityRelativeCurrentRotation.Roll, GravityRelativeDesiredRotation.Roll, TurningRateLimit * DeltaSeconds);
		}

		const FRotator DesiredRotation = (WorldToGravity * GravityRelativeDesiredRotation.Quaternion()).Rotator();
		const FRotator AngularVelocity = DesiredRotation - FromOrientation;
		AngularVelocityDpS = AngularVelocity * (1.f / DeltaSeconds);
	}

	return AngularVelocityDpS;
}

FVector UMovementUtils::ComputeAngularVelocityDegrees(const FRotator& From, const FRotator& To, float DeltaSeconds, float TurningRateLimit)
{
	if (DeltaSeconds <= UE_SMALL_NUMBER)
	{
		return FVector::ZeroVector;
	}

	FQuat Diff = From.Quaternion().Inverse() * To.Quaternion();
	Diff.EnforceShortestArcWith(FQuat::Identity);
	FVector AngularVelocity = FMath::RadiansToDegrees(Diff.ToRotationVector() / DeltaSeconds);

	if (TurningRateLimit >= 0.0f)
	{
		AngularVelocity = AngularVelocity.GetClampedToMaxSize(TurningRateLimit);
	}

	return AngularVelocity;
}

FVector UMovementUtils::ComputeDirectionIntent(const FVector& MoveInput, EMoveInputType MoveInputType, float MaxSpeed)
{
	FVector ResultDirIntent(FVector::ZeroVector);

	switch (MoveInputType)
	{
		default: break;

		case EMoveInputType::DirectionalIntent:
			ResultDirIntent = MoveInput;
			break;

		case EMoveInputType::Velocity:
			const float IntentScalar = FMath::Clamp(MoveInput.Size() / MaxSpeed, 0.0f, 1.0f);
			ResultDirIntent = MoveInput.GetSafeNormal() * IntentScalar;
			break;
	}

	return ResultDirIntent;
}

bool UMovementUtils::IsAngularVelocityZero(const FRotator& AngularVelocity)
{
	return (AngularVelocity.Yaw == 0.0 && AngularVelocity.Pitch == 0.0 && AngularVelocity.Roll == 0.0);
}


FRotator UMovementUtils::ApplyAngularVelocity(const FRotator& StartingOrient, const FRotator& AngularVelocity, float DeltaSeconds)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!UMovementUtils::IsAngularVelocityZero(AngularVelocity))
	{
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		const FQuat ProposedMoveQuat(AngularVelocity * DeltaSeconds);
		const FQuat TargetOrientQuat = FQuat(StartingOrient) * ProposedMoveQuat;
		return TargetOrientQuat.Rotator();
	}

	return StartingOrient;
}

FRotator UMovementUtils::ApplyAngularVelocityToRotator(const FRotator& StartingOrient, const FVector& AngularVelocityDegrees, float DeltaSeconds)
{
	return ApplyAngularVelocityToQuat(StartingOrient.Quaternion(), AngularVelocityDegrees, DeltaSeconds).Rotator();
}

FQuat UMovementUtils::ApplyAngularVelocityToQuat(const FQuat& StartingOrient, const FVector& AngularVelocityDegrees, float DeltaSeconds)
{
	if (!AngularVelocityDegrees.IsZero())
	{
		const FQuat TargetOrientQuat = StartingOrient * FQuat::MakeFromRotationVector(FMath::DegreesToRadians(AngularVelocityDegrees) * DeltaSeconds);
		return TargetOrientQuat;
	}

	return StartingOrient;
}

bool UMovementUtils::FindTeleportSpot(const UMoverComponent* MoverComp, FVector& TestLocation, FRotator TestRotation)
{
	if (!MoverComp || !MoverComp->GetUpdatedComponent())
	{
		return true;
	}
	FVector ProposedTeleportAdjustment(0.f);

	const FVector OriginalTestLocation = TestLocation;

	// check if it fits at desired location
	if (!TestEncroachmentAndAdjust(MoverComp, TestLocation, TestRotation, OUT ProposedTeleportAdjustment))
	{
		return true;	// it fits, so we're done
	}

	if (ProposedTeleportAdjustment.IsNearlyZero())
	{
		// Doesn't fit and we didn't find an adjustment
		// Reset in case Adjust is not actually zero
		TestLocation = OriginalTestLocation;
		return false;
	}

	// Feel around for a non-encroaching location
	// 
	// first do only Z
	const FVector::FReal ZeroThreshold = UE_KINDA_SMALL_NUMBER;
	const bool bZeroZ = FMath::IsNearlyZero(ProposedTeleportAdjustment.Z, ZeroThreshold);
	if (!bZeroZ)
	{
		TestLocation.Z += ProposedTeleportAdjustment.Z;
		if (!TestEncroachment(MoverComp, TestLocation, TestRotation))
		{
			return true;
		}

		TestLocation = OriginalTestLocation;
	}

	// now try just XY
	const bool bZeroX = FMath::IsNearlyZero(ProposedTeleportAdjustment.X, ZeroThreshold);
	const bool bZeroY = FMath::IsNearlyZero(ProposedTeleportAdjustment.Y, ZeroThreshold);
	if (!bZeroX || !bZeroY)
	{
		const float X = bZeroX ? 0.f : ProposedTeleportAdjustment.X;
		const float Y = bZeroY ? 0.f : ProposedTeleportAdjustment.Y;
		FVector Adjustments[8];
		Adjustments[0] = FVector(X, Y, 0);

		// If initially spawning allow testing a few permutations (though this needs improvement).
		// During play only test the first adjustment, permuting axes could put the location on other sides of geometry.
		const int32 Iterations = (MoverComp->HasBegunPlay() ? 1 : 8);

		if (Iterations > 1)
		{
			if (!bZeroX && !bZeroY)
			{
				Adjustments[1] = FVector(-X, Y, 0);
				Adjustments[2] = FVector(X, -Y, 0);
				Adjustments[3] = FVector(-X, -Y, 0);
				Adjustments[4] = FVector(Y, X, 0);
				Adjustments[5] = FVector(-Y, X, 0);
				Adjustments[6] = FVector(Y, -X, 0);
				Adjustments[7] = FVector(-Y, -X, 0);
			}
			else
			{
				// If either X or Y was zero, the permutations above would result in only 4 unique attempts.
				Adjustments[1] = FVector(-X, -Y, 0);
				Adjustments[2] = FVector(Y, X, 0);
				Adjustments[3] = FVector(-Y, -X, 0);
				// Mirror the dominant non-zero value
				const float D = bZeroY ? X : Y;
				Adjustments[4] = FVector(D, D, 0);
				Adjustments[5] = FVector(D, -D, 0);
				Adjustments[6] = FVector(-D, D, 0);
				Adjustments[7] = FVector(-D, -D, 0);
			}
		}

		for (int i = 0; i < Iterations; ++i)
		{
			TestLocation = OriginalTestLocation + Adjustments[i];
			if (!TestEncroachment(MoverComp, TestLocation, TestRotation))
			{
				return true;
			}
		}

		// Try XY adjustment including Z. Note that even with only 1 iteration, this will still try the full proposed (X,Y,Z) adjustment.
		if (!bZeroZ)
		{
			for (int i = 0; i < Iterations; ++i)
			{
				TestLocation = OriginalTestLocation + Adjustments[i];
				TestLocation.Z += ProposedTeleportAdjustment.Z;
				if (!TestEncroachment(MoverComp, TestLocation, TestRotation))
				{
					return true;
				}
			}
		}
	}

	// Don't write out the last failed test location, we promised to only if we find a good spot, in case the caller re-uses the original input.
	TestLocation = OriginalTestLocation;
	return false;
}

bool UMovementUtils::TestEncroachment(const UMoverComponent* MoverComp, FVector TestLocation, FRotator TestRotation)
{
	const AActor* OwnerActor = MoverComp->GetOwner();
	const UWorld* OwnerWorld = OwnerActor->GetWorld();
	const UPrimitiveComponent* MovingPrimitiveRoot = Cast<UPrimitiveComponent>(MoverComp->GetUpdatedComponent());
	if (MovingPrimitiveRoot && MovingPrimitiveRoot->IsQueryCollisionEnabled())
	{ 
		FTransform const TestRootToWorld = FTransform(TestRotation, TestLocation);

		TArray<AActor*> ChildActors;
		OwnerActor->GetAllChildActors(ChildActors);

		return TestEncroachment_Internal(OwnerWorld, OwnerActor, MovingPrimitiveRoot, TestRootToWorld, ChildActors);
	}

	return false;
}

bool UMovementUtils::TestEncroachmentAndAdjust(const UMoverComponent* MoverComp, FVector TestLocation, FRotator TestRotation, OUT FVector& OutProposedAdjustment)
{
	if (MoverComp == nullptr)
	{
		return false;
	}

	USceneComponent* const RootComponent = MoverComp->GetUpdatedComponent();
	if (RootComponent == nullptr)
	{
		return false;
	}

	bool bFoundEncroacher = false;

	FTransform const TestRootToWorld = FTransform(TestRotation, TestLocation);
	FTransform const WorldToOldRoot = RootComponent->GetComponentToWorld().Inverse();

	const UPrimitiveComponent* MovingPrimitiveRoot = Cast<UPrimitiveComponent>(MoverComp->GetUpdatedComponent());
	if (MovingPrimitiveRoot)
	{
		// This actor has a movement component, which we interpret to mean that this actor has a primary component being swept around
		// the world, and that component is the only one we care about encroaching (since the movement code will happily embedding
		// other components in the world during movement updates)

		if (MovingPrimitiveRoot->IsQueryCollisionEnabled())
		{
			// might not be the root, so we need to compute the transform
			FTransform const CompToRoot = MovingPrimitiveRoot->GetComponentToWorld() * WorldToOldRoot;
			FTransform const CompToNewWorld = CompToRoot * TestRootToWorld;

			TArray<AActor*> ChildActors;
			MoverComp->GetOwner()->GetAllChildActors(ChildActors);

			bFoundEncroacher = TestEncroachmentWithAdjustment_Internal(MoverComp->GetOwner()->GetWorld(), MoverComp->GetOwner(), MovingPrimitiveRoot, CompToNewWorld, ChildActors, OutProposedAdjustment);
		}
	}

	return bFoundEncroacher;
}


bool UMovementUtils::TryMoveUpdatedComponent_Internal(const FMovingComponentSet& MovingComps, const FVector& Delta, const FQuat& NewRotation, bool bSweep, EMoveComponentFlags MoveComponentFlags, FHitResult* OutHit, ETeleportType Teleport)
{
	if (MovingComps.UpdatedComponent.IsValid())
	{
		FVector ConstrainedDelta = Delta;

		if (MovingComps.MoverComponent.IsValid())
		{
			ConstrainedDelta = UPlanarConstraintUtils::ConstrainDirectionToPlane(MovingComps.MoverComponent->GetPlanarConstraint(), Delta);	
		}

		return MovingComps.UpdatedComponent->MoveComponent(ConstrainedDelta, NewRotation, bSweep, OutHit, MoveComponentFlags, Teleport);
	}

	return false;
}



// How much to adjust out collision shape box during encroachment testing, for a little leeway. This is applied to each axis, in cm.
static const float ENCROACH_SHRINK_EPSILON = 0.15f;

bool UMovementUtils::TestEncroachment_Internal(const UWorld* World, const AActor* TestActor, const UPrimitiveComponent* PrimComp, const FTransform& TestWorldTransform, const TArray<AActor*>& IgnoreActors)
{
	const float Epsilon = ENCROACH_SHRINK_EPSILON;

	if (World && PrimComp)
	{
		bool bFoundBlockingHit = false;

		ECollisionChannel const BlockingChannel = PrimComp->GetCollisionObjectType();
		FCollisionShape const CollisionShape = PrimComp->GetCollisionShape(-Epsilon);

		if (CollisionShape.IsBox() && (Cast<UBoxComponent>(PrimComp) == nullptr))
		{
			// we have a bounding box not for a box component, which means this was the fallback aabb
			// since we don't need the penetration info, go ahead and test the component itself for overlaps, which is more accurate
			if (PrimComp->IsRegistered())
			{
				// must be registered
				TArray<FOverlapResult> Overlaps;
				FComponentQueryParams Params(SCENE_QUERY_STAT(ComponentEncroachesBlockingGeometry_NoAdjustment), TestActor);
				FCollisionResponseParams ResponseParams;
				PrimComp->InitSweepCollisionParams(Params, ResponseParams);
				Params.AddIgnoredActors(IgnoreActors);
				return World->ComponentOverlapMultiByChannel(Overlaps, PrimComp, TestWorldTransform.GetLocation(), TestWorldTransform.GetRotation(), BlockingChannel, Params);
			}
			else
			{
				UE_LOGF(LogMover, Log, "Components must be registered in order to be used in a ComponentOverlapMulti call. PriComp: %ls TestActor: %ls", *PrimComp->GetName(), *TestActor->GetName());
				return false;
			}
		}
		else
		{
			FCollisionQueryParams Params(SCENE_QUERY_STAT(ComponentEncroachesBlockingGeometry_NoAdjustment), false, TestActor);
			FCollisionResponseParams ResponseParams;
			PrimComp->InitSweepCollisionParams(Params, ResponseParams);
			Params.AddIgnoredActors(IgnoreActors);
			return World->OverlapBlockingTestByChannel(TestWorldTransform.GetLocation(), TestWorldTransform.GetRotation(), BlockingChannel, CollisionShape, Params, ResponseParams);
		}
	}

	return false;

}

bool UMovementUtils::TestEncroachmentWithAdjustment_Internal(const UWorld* World, const AActor* TestActor, const UPrimitiveComponent* PrimComp, const FTransform& TestWorldTransform, const TArray<AActor*>& IgnoreActors, OUT FVector& OutProposedAdjustment)
{
	// this function is based on UWorld's ComponentEncroachesBlockingGeometry

	// init our output
	OutProposedAdjustment = FVector::ZeroVector;

	float const Epsilon = ENCROACH_SHRINK_EPSILON;

	if (World && PrimComp)
	{
		bool bFoundBlockingHit = false;
		bool bComputePenetrationAdjustment = true;

		TArray<FOverlapResult> Overlaps;
		ECollisionChannel const BlockingChannel = PrimComp->GetCollisionObjectType();
		FCollisionShape const CollisionShape = PrimComp->GetCollisionShape(-Epsilon);

		if (CollisionShape.IsBox() && (Cast<UBoxComponent>(PrimComp) == nullptr))
		{
			// we have a bounding box not for a box component, which means this was the fallback aabb
			// so lets test the actual component instead of it's aabb
			// note we won't get penetration adjustment but that's ok
			if (PrimComp->IsRegistered())
			{
				// must be registered
				FComponentQueryParams Params(SCENE_QUERY_STAT(TestEncroachmentWithAdjustment), TestActor);
				FCollisionResponseParams ResponseParams;
				PrimComp->InitSweepCollisionParams(Params, ResponseParams);
				Params.AddIgnoredActors(IgnoreActors);
				bFoundBlockingHit = World->ComponentOverlapMultiByChannel(Overlaps, PrimComp, TestWorldTransform.GetLocation(), TestWorldTransform.GetRotation(), BlockingChannel, Params);
				bComputePenetrationAdjustment = false;
			}
			else
			{
				UE_LOGF(LogMover, Log, "Components must be registered in order to be used in a ComponentOverlapMulti call. PriComp: %ls TestActor: %ls", *PrimComp->GetName(), *TestActor->GetName());
			}
		}
		else
		{
			// overlap our shape
			FCollisionQueryParams Params(SCENE_QUERY_STAT(TestEncroachmentWithAdjustment), false, TestActor);
			FCollisionResponseParams ResponseParams;
			PrimComp->InitSweepCollisionParams(Params, ResponseParams);
			Params.AddIgnoredActors(IgnoreActors);
			bFoundBlockingHit = World->OverlapMultiByChannel(Overlaps, TestWorldTransform.GetLocation(), TestWorldTransform.GetRotation(), BlockingChannel, CollisionShape, Params, ResponseParams);
		}

		// compute adjustment
		if (bFoundBlockingHit && bComputePenetrationAdjustment)
		{
			// if encroaching, add up all the MTDs of overlapping shapes
			FMTDResult MTDResult;
			uint32 NumBlockingHits = 0;
			for (int32 HitIdx = 0; HitIdx < Overlaps.Num(); HitIdx++)
			{
				UPrimitiveComponent* const OverlapComponent = Overlaps[HitIdx].Component.Get();
				// first determine closest impact point along each axis
				if (OverlapComponent && OverlapComponent->GetCollisionResponseToChannel(BlockingChannel) == ECR_Block)
				{
					NumBlockingHits++;
					FCollisionShape const NonShrunkenCollisionShape = PrimComp->GetCollisionShape();
					const FBodyInstance* OverlapBodyInstance = OverlapComponent->GetBodyInstance(NAME_None, true, Overlaps[HitIdx].GetItemIndex());
					bool bSuccess = OverlapBodyInstance && OverlapBodyInstance->OverlapTest(TestWorldTransform.GetLocation(), TestWorldTransform.GetRotation(), NonShrunkenCollisionShape, &MTDResult);
					if (bSuccess)
					{
						OutProposedAdjustment += MTDResult.Direction * MTDResult.Distance;
					}
					else
					{
						UE_LOGF(LogMover, Log, "OverlapTest says we are overlapping, yet MTD says we're not. Something is wrong");
						// It's not safe to use a partial result, that could push us out to an invalid location (like the other side of a wall).
						OutProposedAdjustment = FVector::ZeroVector;
						return true;
					}

					// #hack: sometimes for boxes, physx returns a 0 MTD even though it reports a contact (returns true)
					// to get around this, let's go ahead and test again with the epsilon-shrunken collision shape to see if we're really in 
					// the clear.
					if (bSuccess && FMath::IsNearlyZero(MTDResult.Distance))
					{
						FCollisionShape const ShrunkenCollisionShape = PrimComp->GetCollisionShape(-Epsilon);
						bSuccess = OverlapBodyInstance && OverlapBodyInstance->OverlapTest(TestWorldTransform.GetLocation(), TestWorldTransform.GetRotation(), ShrunkenCollisionShape, &MTDResult);
						if (bSuccess)
						{
							OutProposedAdjustment += MTDResult.Direction * MTDResult.Distance;
						}
						else
						{
							// Ignore this overlap.
							UE_LOGF(LogMover, Log, "OverlapTest says we are overlapping, yet MTD says we're not (with smaller shape). Ignoring this overlap.");
							NumBlockingHits--;
							continue;
						}
					}
				}
			}

			// See if we chose to invalidate all of our supposed "blocking hits".
			if (NumBlockingHits == 0)
			{
				OutProposedAdjustment = FVector::ZeroVector;
				bFoundBlockingHit = false;
			}
		}

		return bFoundBlockingHit;
	}

	return false;
}
