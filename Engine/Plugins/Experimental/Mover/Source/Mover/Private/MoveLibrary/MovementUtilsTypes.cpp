// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoveLibrary/MovementUtilsTypes.h"
#include "Components/PrimitiveComponent.h"
#include "MoverComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovementUtilsTypes)



void FMovingComponentSet::SetFrom(USceneComponent* InUpdatedComponent)
{
	UpdatedComponent = InUpdatedComponent;

	if (UpdatedComponent.IsValid())
	{
		UpdatedPrimitive = Cast<UPrimitiveComponent>(UpdatedComponent);
		MoverComponent = UpdatedComponent->GetOwner()->FindComponentByClass<UMoverComponent>();

		checkf(!MoverComponent.IsValid() || UpdatedComponent == MoverComponent->GetUpdatedComponent(), TEXT("Expected MoverComponent to have the same UpdatedComponent"));
	}
}

void FMovingComponentSet::SetFrom(UMoverComponent* InMoverComponent)
{
	MoverComponent = InMoverComponent;

	if (MoverComponent.IsValid())
	{
		UpdatedComponent = MoverComponent->GetUpdatedComponent();
		UpdatedPrimitive = Cast<UPrimitiveComponent>(UpdatedComponent);
	}
}


static const FName DefaultCollisionTraceTag = "SweepTestMoverComponent";

FMoverCollisionParams::FMoverCollisionParams(const USceneComponent* SceneComp)
{
	if (const UPrimitiveComponent* AsPrimitive = Cast<const UPrimitiveComponent>(SceneComp))
	{
		SetFromPrimitiveComponent(AsPrimitive);
	}
	else
	{
		// TODO: set up a line trace if SceneComp is not a primitive component
		ensureMsgf(0, TEXT("Support for non-primitive components is not yet implemented"));
	}
}

void FMoverCollisionParams::SetFromPrimitiveComponent(const UPrimitiveComponent* PrimitiveComp)
{
	Channel = PrimitiveComp->GetCollisionObjectType();

	Shape = PrimitiveComp->GetCollisionShape();

	PrimitiveComp->InitSweepCollisionParams(QueryParams, ResponseParams);

	const AActor* OwningActor = PrimitiveComp->GetOwner();
	
	QueryParams.TraceTag = DefaultCollisionTraceTag;
	QueryParams.OwnerTag = OwningActor->GetFName();
	QueryParams.AddIgnoredActor(OwningActor);
}
