// Copyright Epic Games, Inc. All Rights Reserved.


#include "MoveLibrary/MoverBlackboard.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoverBlackboard)



void UMoverBlackboard::Invalidate(FName ObjName)
{
	UE::TWriteScopeLock Lock(ObjectsMapLock);
	ObjectsByName.Remove(ObjName);
}

void UMoverBlackboard::Invalidate(EInvalidationReason Reason)
{
	switch (Reason)
	{
		default:
		case EInvalidationReason::FullReset:
		{
			UE::TWriteScopeLock Lock(ObjectsMapLock);
			ObjectsByName.Empty();
		}
		break;

		// TODO: Support other reasons
	}
}

void UMoverBlackboard::BeginDestroy()
{
	InvalidateAll();
	Super::BeginDestroy();
}
