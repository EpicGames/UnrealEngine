// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavMovementUtils.h"
#include "CoreTypes.h"
#include "NavMesh/RecastNavMesh.h"
#include "VisualLogger/VisualLogger.h"

bool NavMovementUtils::CalculateNavMeshNormal(const FNavLocation& Location, FVector& OutNormal, const INavigationDataInterface* NavData,
                                              UObject* LogOwner)
{
	const ARecastNavMesh* RecastNavMesh = Cast<ARecastNavMesh>(NavData);
	if (RecastNavMesh == nullptr)
	{
		return false;
	}

	if (Location.HasNodeRef() == false)
	{
		return false;
	}

	TArray<FVector> OutVerts;
	if (RecastNavMesh->GetPolyVerts(Location.NodeRef, OutVerts) == false)
	{
		return false;
	}

	check(OutVerts.Num() >= 3);
	OutNormal = FVector::ZeroVector;
	float DebugThickness = 2.0f;

	// Calculate normal from navmesh verts (arbitrary number)
	for (int i = 0; i < OutVerts.Num() - 1; ++i)
	{
		OutNormal += OutVerts[i+1].Cross(OutVerts[i]);
		UE_VLOG_SEGMENT_THICK(LogOwner, "AsyncNavWalkingMode", Display, OutVerts[i], OutVerts[i+1], FColor::Magenta, DebugThickness, TEXT(""));
	}
	OutNormal += OutVerts[0].Cross(OutVerts.Last());
	UE_VLOG_SEGMENT_THICK(LogOwner, "AsyncNavWalkingMode", Display, OutVerts.Last(), OutVerts[0], FColor::Magenta, DebugThickness, TEXT(""));

	// Normalize
	float SizeSquared = OutNormal.SizeSquared();
	if (FMath::IsNearlyZero(SizeSquared))
	{
		return false;
	}
	OutNormal /= FMath::Sqrt(SizeSquared);
	return true;
}
