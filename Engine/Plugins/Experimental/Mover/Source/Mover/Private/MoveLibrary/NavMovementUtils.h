// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector.h"

struct FNavLocation;
class INavigationDataInterface;
class UObject;

class NavMovementUtils
{
	public:
	
	static bool CalculateNavMeshNormal(const FNavLocation& Location, FVector& OutNormal, const INavigationDataInterface* NavData, UObject* LogOwner = nullptr);
};