// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoveLibrary/PlayMoverMontageCallbackProxy.h"
#include "Animation/AnimMontage.h"
#include "Components/SkeletalMeshComponent.h"
#include "MoverComponent.h"
#include "DefaultMovementSet/LayeredMoves/AnimRootMotionLayeredMove.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlayMoverMontageCallbackProxy)

UPlayMoverMontageCallbackProxy::UPlayMoverMontageCallbackProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UPlayMoverMontageCallbackProxy* UPlayMoverMontageCallbackProxy::CreateProxyObjectForPlayMoverMontage(
	class UMoverComponent* InMoverComponent,
	class UAnimMontage* MontageToPlay,
	float PlayRate,
	float StartingPosition,
	FName StartingSection,
	bool bShouldStopAllMontages,
	float BlendOutOverride)
{
	USkeletalMeshComponent* SkelMeshComp = nullptr; 
	
	if (InMoverComponent)
	{
		SkelMeshComp = Cast<USkeletalMeshComponent>(InMoverComponent->GetPrimaryVisualComponent());

		if (!SkelMeshComp)
		{
			SkelMeshComp = InMoverComponent->GetOwner()->GetComponentByClass<USkeletalMeshComponent>();
		}
	}

	UPlayMoverMontageCallbackProxy* Proxy = NewObject<UPlayMoverMontageCallbackProxy>();
	Proxy->SetFlags(RF_StrongRefOnFrame);
	Proxy->PlayMoverMontage(InMoverComponent, SkelMeshComp, MontageToPlay, PlayRate, StartingPosition, StartingSection, bShouldStopAllMontages, BlendOutOverride, EMoveMixMode::OverrideAll);

	return Proxy;
}


bool UPlayMoverMontageCallbackProxy::PlayMoverMontage(
	UMoverComponent* InMoverComponent,
	USkeletalMeshComponent* InSkeletalMeshComponent,
	UAnimMontage* MontageToPlay,
	float PlayRate,
	float StartingPosition,
	FName StartingSection,
	bool bShouldStopAllMontages,
	float BlendOutOverride,
	EMoveMixMode MixMode)
{
	UE_CLOGF(!MontageToPlay, LogMover, Warning, "PlayMoverMontage: MontageToPlay is null");

	bool bDidPlay = PlayMontage(InSkeletalMeshComponent, MontageToPlay, PlayRate, StartingPosition, StartingSection, bShouldStopAllMontages);

	if (bDidPlay && PlayRate != 0.f && MontageToPlay && MontageToPlay->HasRootMotion() && !FMath::IsNearlyZero(MontageToPlay->RateScale))
	{
		// Only one montage layered move can be active at a time, so cancel any others.  This will NOT cancel the skeletal mesh counterparts.
		InMoverComponent->CancelFeaturesWithTag(Mover_AnimRootMotion_Montage, /* bExactMatch? */false);

		if (UAnimInstance* AnimInstance = InSkeletalMeshComponent->GetAnimInstance())
		{
			if (FAnimMontageInstance* MontageInstance = AnimInstance->GetActiveInstanceForMontage(MontageToPlay))
			{
				// Listen for possible ways the montage could end
				OnCompleted.AddUniqueDynamic(this, &UPlayMoverMontageCallbackProxy::OnMoverMontageEnded);
				OnInterrupted.AddUniqueDynamic(this, &UPlayMoverMontageCallbackProxy::OnMoverMontageEnded);

				// Disable the actual animation-driven root motion, in favor of our own layered move
				MontageInstance->PushDisableRootMotion();

				const float StartingMontagePosition = MontageInstance->GetPosition();	// position in seconds, disregarding PlayRate

				// Queue a layered move to perform the same anim root motion over the same time span.
				// Use the async variant when the backend runs on a worker thread so that GenerateMove_Async
				// is called instead of the game-thread-only GenerateMove.
				TSharedPtr<FLayeredMove_AnimRootMotion> AnimRootMotionMove = InMoverComponent->IsBackendAsync()
					? MakeShared<FLayeredMove_AnimRootMotion_SimDriven>()
					: MakeShared<FLayeredMove_AnimRootMotion>();
				AnimRootMotionMove->MontageState.Montage = MontageToPlay;
				AnimRootMotionMove->MontageState.PlayRate = PlayRate;
				AnimRootMotionMove->MontageState.StartingMontagePosition = StartingMontagePosition;
				AnimRootMotionMove->MontageState.CurrentPosition = StartingMontagePosition;
				AnimRootMotionMove->MontageState.BlendOutTimeSeconds = (BlendOutOverride >= 0.f) ? BlendOutOverride : MontageToPlay->GetDefaultBlendOutTime();
				// Heuristic: auto blend-out is enabled when the last declared section has no outgoing link.
				// Montages with non-linear section graphs where the terminal section is not the last declared
				// one will be not blend out. If more control is needed, PlayMontage will have to expose more options.
				AnimRootMotionMove->MontageState.bEnableAutoBlendOut = MontageToPlay->CompositeSections.IsEmpty() || MontageToPlay->CompositeSections.Last().NextSectionName == NAME_None;
#if !UE_BUILD_SHIPPING
				AnimRootMotionMove->MontageState.Debug_MontageName = MontageToPlay->GetFName();
#endif
				
				float RemainingUnscaledMontageSeconds(0.f);

				if (PlayRate > 0.f)
				{
					// playing forwards, so working towards the end of the montage
					RemainingUnscaledMontageSeconds = MontageToPlay->GetPlayLength() - StartingMontagePosition;
				}
				else
				{
					// playing backwards, so working towards the start of the montage
					RemainingUnscaledMontageSeconds = StartingMontagePosition;	
				}

				AnimRootMotionMove->DurationMs = (RemainingUnscaledMontageSeconds / FMath::Abs(PlayRate)) * (1.0 / MontageToPlay->RateScale) * 1000.f;
				AnimRootMotionMove->MixMode = MixMode;

				InMoverComponent->QueueLayeredMove(AnimRootMotionMove);
			}
		}
	}

	return bDidPlay;
}

void UPlayMoverMontageCallbackProxy::OnMoverMontageEnded(FName IgnoredNotifyName)
{
	// TODO: this is where we'd want to schedule the ending of the associated move, whether the montage instance was interrupted or ended

	UnbindMontageDelegates();
}

void UPlayMoverMontageCallbackProxy::UnbindMontageDelegates()
{
	OnCompleted.RemoveDynamic(this, &UPlayMoverMontageCallbackProxy::OnMoverMontageEnded);
	OnInterrupted.RemoveDynamic(this, &UPlayMoverMontageCallbackProxy::OnMoverMontageEnded);
}


void UPlayMoverMontageCallbackProxy::BeginDestroy()
{
	UnbindMontageDelegates();

	Super::BeginDestroy();
}
