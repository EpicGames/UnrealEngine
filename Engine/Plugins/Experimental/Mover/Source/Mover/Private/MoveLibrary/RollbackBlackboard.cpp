// Copyright Epic Games, Inc. All Rights Reserved.


#include "MoveLibrary/RollbackBlackboard.h"
#include "MoverTypes.h"
#include "GenericPlatform/GenericPlatformMath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RollbackBlackboard)


uint32 URollbackBlackboard::BlackboardEntryBase::ComputeBufferSize(EBlackboardSizingPolicy SizingPolicy, uint32 FixedBufferSize)
{
	uint32 BufferSize = 2;

	// TODO: handle these special cases:
	//     if we are in a non-networked situation w/ non-async simulation, we only need 1 entry slot
	//     if we are in a non-networked situation w/ async simulation, we only need 2 entry slots (one for simulating forwarD)

	switch (SizingPolicy)
	{
		case EBlackboardSizingPolicy::FixedDeclaredSize:
			// TODO: Warn if FixedBufferSize is invalid
			BufferSize = FMath::Max(BufferSize, FixedBufferSize);
			break;
		
		case EBlackboardSizingPolicy::SingleEntry:
			// TODO: only allocate 1 element if our simulation is not async
			BufferSize = 2;		// Even for a single value entry, we need 2 slots in async simulations to avoid potential thread contention
			break;

		default:	//TODO: warn about an unhandled policy type
			break;
	}

	return BufferSize;
}

URollbackBlackboard::BlackboardEntryBase::BlackboardEntryBase(const EntrySettings& InSettings, uint32 BufferSize, ERollbackBufferFlags BufferFlags)
	: Settings(InSettings)
	, Timestamps(BufferSize, BufferFlags, EntryTimeStamp())
	, ExternalIdx(0)
	, InternalIdx(0)
{
}

void URollbackBlackboard::BlackboardEntryBase::RollBack(uint32 NewPendingFrame)
{
	if (Settings.RollbackPolicy == EBlackboardRollbackPolicy::IgnoreRollback)
	{
		return;
	}

	// Goal: adjust entry indices to point at the value from the frame prior to NewPendingFrame. May make the entire entry invalidated, if there were no value slots that old.
	check(ExternalIdx == InternalIdx);

	const uint32 LowestPossibleIdx = (ExternalIdx > Timestamps.Capacity()) ? (ExternalIdx - Timestamps.Capacity()) : 0u;

	// Walk downwards to find the highest index with a frame < NewPendingFrame. 
	// If we hit uint32::max, that indicates we wrapped around from 0 and we're out of slots to check.
	for (uint32 IdxToCheck = ExternalIdx; IdxToCheck >= LowestPossibleIdx && IdxToCheck != TNumericLimits<uint32>::Max(); --IdxToCheck)
	{
		if (Timestamps[IdxToCheck].IsValid() && Timestamps[IdxToCheck].Frame < NewPendingFrame)
		{
			ExternalIdx = InternalIdx = IdxToCheck;
			return;
		}
	}

	// If we made it here, then there are no value slots that weren't rolled back, so let's make it clear
	Timestamps[ExternalIdx].Invalidate();

}

void URollbackBlackboard::BlackboardEntryBase::InvalidatePredictiveState()
{
	Timestamps.GetPredictiveElement().Invalidate();

	ResetPredictiveElement();
}



bool URollbackBlackboard::BlackboardEntryBase::CanReadEntryAtTime(const EntryTimeStamp& ReaderTimeStamp, const EntryTimeStamp& EntryTimeStamp) const
{

	if (!EntryTimeStamp.IsValid() || !ReaderTimeStamp.IsValid())
	{
		return false;	// entry isn't initialized yet or never set
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS;	// supporting deprecation of EBlackboardPersistencePolicy::NextFrameOnly

	switch (Settings.PersistencePolicy)
	{
		case EBlackboardPersistencePolicy::NextFrameOnly:	// deprecated
		case EBlackboardPersistencePolicy::ThroughNextFrame:
		{
			// if value wasn't last set during the current or immediate prior sim frame, then it can't be read
			if (!(ReaderTimeStamp.Frame == EntryTimeStamp.Frame ||
				  ReaderTimeStamp.Frame == EntryTimeStamp.Frame+1))
			{
				return false;
			}
		}
		break;

		case EBlackboardPersistencePolicy::CurrentFrameOnly:
		{
			// if value wasn't last set during the current sim frame, then it can't be read
			if (ReaderTimeStamp.Frame != EntryTimeStamp.Frame)
			{
				return false;
			}
		}
		break;

		case EBlackboardPersistencePolicy::Forever:
			// If the reader is in the past compared to the time of the entry, then it can't be read
			if (ReaderTimeStamp.Frame < EntryTimeStamp.Frame)
			{
				return false;
			}
			break;

		default:		// Allow reading of any not-yet-implemented policies
			break;
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS;


	return true;
}

bool URollbackBlackboard::BlackboardEntryBase::CanReadEntryAtTime(const EntryTimeStamp& ReaderTimeStamp, EEntryIndexType IndexType) const
{
	const EntryTimeStamp* TargetEntryTimeStamp = nullptr;

	if (IndexType == EEntryIndexType::Predictive)
	{
		TargetEntryTimeStamp = &Timestamps.GetPredictiveElement();
	}
	else
	{
		const uint32 TimestampIdx = ((IndexType == EEntryIndexType::External) ? ExternalIdx : InternalIdx);
		TargetEntryTimeStamp = &Timestamps[TimestampIdx];
	}
	
	return CanReadEntryAtTime(ReaderTimeStamp, *TargetEntryTimeStamp);
}


void URollbackBlackboard::BeginSimulationFrame(const FMoverTimeStep& PendingTimeStep)
{
	check(!bIsSimulationInProgress && !bIsRollbackInProgress);
	InProgressSimFrameThreadId = FPlatformTLS::GetCurrentThreadId();
	bIsSimulationInProgress = true;
	bIsResimulating = PendingTimeStep.bIsResimulating;

	InProgressSimTimeStamp.TimeMs = PendingTimeStep.BaseSimTimeMs;
	InProgressSimTimeStamp.Frame = PendingTimeStep.ServerFrame;
}

void URollbackBlackboard::EndSimulationFrame()
{
	check(bIsSimulationInProgress && (InProgressSimFrameThreadId == FPlatformTLS::GetCurrentThreadId()));
	bIsSimulationInProgress = bIsResimulating = false;
	InProgressSimFrameThreadId = 0;

	// TODO: need some kind of lock so we wait on any in-progress operations before advancing the CurrentSimTimeStamp
	CurrentSimTimeStamp = InProgressSimTimeStamp;


	// TODO: could we skip this if no "set" operations occurred? Consider different policies, like change-every-frame entries

	for (const TPair<FName, TUniquePtr<BlackboardEntryBase>>& KVP : EntryMap)
	{
		KVP.Value->OnSimulationFrameEnd();
	}
}


void URollbackBlackboard::BeginRollback(const FMoverTimeStep& NewBaseTimeStep)
{
	check(!bIsSimulationInProgress && !bIsRollbackInProgress);
	InRollbackThreadId = FPlatformTLS::GetCurrentThreadId();
	bIsRollbackInProgress = true;

	UE_LOGF(LogMover, Verbose, "Blackboard begin rollback. From Sim F %i / T %.3f -> F %i / T %.3f", 
		CurrentSimTimeStamp.Frame, CurrentSimTimeStamp.TimeMs, NewBaseTimeStep.ServerFrame, NewBaseTimeStep.BaseSimTimeMs);

	// TODO: Need locking mechanism

	const EntryTimeStamp NewBaseTimeStamp((double)NewBaseTimeStep.BaseSimTimeMs, NewBaseTimeStep.ServerFrame);

	for (const TPair<FName, TUniquePtr<BlackboardEntryBase>>& KVP : EntryMap)
	{
		KVP.Value->RollBack(NewBaseTimeStamp.Frame);
	}

	// As the rollback occurs, we need to pull back the timestamps to match
	CurrentSimTimeStamp = InProgressSimTimeStamp = NewBaseTimeStamp;
}

void URollbackBlackboard::EndRollback()
{
	check (bIsRollbackInProgress && (InRollbackThreadId == FPlatformTLS::GetCurrentThreadId()));
	bIsRollbackInProgress = false;
	InRollbackThreadId = 0;
}

void URollbackBlackboard::BeginPredictionFrame(const FMoverTimeStep& PendingTimeStep)
{
	check(BufferFlags & ERollbackBufferFlags::Prediction);
	check(!bIsPredictionInProgress || InPredictionThreadId == FPlatformTLS::GetCurrentThreadId());
	
	if (!bIsPredictionInProgress)
	{
		bIsPredictionInProgress = true;
		InPredictionThreadId = FPlatformTLS::GetCurrentThreadId();
	}
}

void URollbackBlackboard::EndPrediction()
{
	check(((BufferFlags & ERollbackBufferFlags::Prediction) == ERollbackBufferFlags::Prediction) && bIsPredictionInProgress && IsInPredictionThread());

	bIsPredictionInProgress = false;
	InPredictionThreadId = 0;

	for (const TPair<FName, TUniquePtr<BlackboardEntryBase>>& KVP : EntryMap)
	{
		KVP.Value->InvalidatePredictiveState();
	}
}

bool URollbackBlackboard::IsInSimulationThread() const
{
	return (InProgressSimFrameThreadId == FPlatformTLS::GetCurrentThreadId());
}

bool URollbackBlackboard::IsInPredictionThread() const
{
	return (InPredictionThreadId == FPlatformTLS::GetCurrentThreadId());
}

