// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoveLibrary/RollbackBlackboardLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RollbackBlackboardLibrary)


URollbackBlackboard::EntrySettings URollbackBlackboardLibrary::MakeEventEntrySettings(int32 MaxHistoryCount)
{
	URollbackBlackboard::EntrySettings EventEntrySettings;
	EventEntrySettings.SizingPolicy = EBlackboardSizingPolicy::FixedDeclaredSize;
	EventEntrySettings.FixedSize = MaxHistoryCount;
	EventEntrySettings.PersistencePolicy = EBlackboardPersistencePolicy::Forever;
	EventEntrySettings.RollbackPolicy = EBlackboardRollbackPolicy::InvalidatedOnRollback;
	return EventEntrySettings;
}


URollbackBlackboard::EntrySettings URollbackBlackboardLibrary::MakeSingleFrameEntrySettings()
{
	URollbackBlackboard::EntrySettings SingleFrameEntrySettings;
	SingleFrameEntrySettings.SizingPolicy = EBlackboardSizingPolicy::SingleEntry;
	SingleFrameEntrySettings.PersistencePolicy = EBlackboardPersistencePolicy::CurrentFrameOnly;
	SingleFrameEntrySettings.RollbackPolicy = EBlackboardRollbackPolicy::InvalidatedOnRollback;
	return SingleFrameEntrySettings;
}


URollbackBlackboard::EntrySettings URollbackBlackboardLibrary::MakeRollingEntrySettings()
{
	URollbackBlackboard::EntrySettings RollingEntrySettings;
	RollingEntrySettings.SizingPolicy = EBlackboardSizingPolicy::FixedDeclaredSize;	// TODO: use FixedBackendBufferSize policy once it's supported
	RollingEntrySettings.FixedSize = 30;
	RollingEntrySettings.PersistencePolicy = EBlackboardPersistencePolicy::Forever;
	RollingEntrySettings.RollbackPolicy = EBlackboardRollbackPolicy::InvalidatedOnRollback;
	return RollingEntrySettings;
}
