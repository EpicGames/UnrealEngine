// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoveLibrary/RollbackBlackboardWrappers.h"

FRollbackBlackboardExternalWrapper::FRollbackBlackboardExternalWrapper(URollbackBlackboard& InBlackboard)
	: Blackboard(&InBlackboard) 
{
}

FRollbackBlackboardExternalWrapper::FRollbackBlackboardExternalWrapper(const FRollbackBlackboardExternalWrapper& Other)
	: Blackboard(Other.Blackboard) 
{
}

FRollbackBlackboardExternalWrapper::FRollbackBlackboardExternalWrapper()
	: Blackboard(nullptr)
{
}

FRollbackBlackboardSimWrapper::FRollbackBlackboardSimWrapper(URollbackBlackboard& InBlackboard, bool bUsePredictiveMode)
	: Blackboard(&InBlackboard), bInPredictiveMode(bUsePredictiveMode) 
{
}

FRollbackBlackboardSimWrapper::FRollbackBlackboardSimWrapper(const FRollbackBlackboardSimWrapper& Other)
	: Blackboard(Other.Blackboard), bInPredictiveMode(Other.bInPredictiveMode) 
{
}


FRollbackBlackboardSimWrapper::FRollbackBlackboardSimWrapper()
	: Blackboard(nullptr), bInPredictiveMode(false)
{
}

