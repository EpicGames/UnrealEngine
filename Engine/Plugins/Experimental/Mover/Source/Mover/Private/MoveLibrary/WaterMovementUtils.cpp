// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoveLibrary/WaterMovementUtils.h"
#include "MoveLibrary/MovementUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaterMovementUtils)

FProposedMove UWaterMovementUtils::ComputeControlledWaterMove(const FWaterMoveParams& InParams)
{
	FProposedMove OutMove;

	OutMove.DirectionIntent = UMovementUtils::ComputeDirectionIntent(InParams.MoveInput, InParams.MoveInputType, InParams.MaxSpeed);
	OutMove.bHasDirIntent = !OutMove.DirectionIntent.IsNearlyZero();

	FComputeCombinedVelocityParams ComputeCombinedVelocityParams;
	ComputeCombinedVelocityParams.DeltaSeconds = InParams.DeltaSeconds;
	ComputeCombinedVelocityParams.InitialVelocity = InParams.PriorVelocity;
	ComputeCombinedVelocityParams.MoveDirectionIntent = InParams.MoveInput;
	ComputeCombinedVelocityParams.MaxSpeed = InParams.MaxSpeed;
	ComputeCombinedVelocityParams.TurningBoost = InParams.TurningBoost;
	ComputeCombinedVelocityParams.Deceleration = InParams.Deceleration;
	ComputeCombinedVelocityParams.Acceleration = InParams.Acceleration;
	ComputeCombinedVelocityParams.ExternalAcceleration = InParams.MoveAcceleration;
	ComputeCombinedVelocityParams.OverallMaxSpeed = InParams.MoveSpeed;
	
	OutMove.LinearVelocity = UMovementUtils::ComputeCombinedVelocity(ComputeCombinedVelocityParams);

	// JAH TODO: this is where we can perform turning, based on aux settings. For now, just snap to the intended final orientation.
	FVector IntendedFacingDir = InParams.OrientationIntent.RotateVector(FVector::ForwardVector).GetSafeNormal();
	OutMove.AngularVelocityDegrees = UMovementUtils::ComputeAngularVelocityDegrees(InParams.PriorOrientation, IntendedFacingDir.ToOrientationRotator(), InParams.DeltaSeconds, InParams.TurningRate);

	return OutMove;
}

void UWaterMovementUtils::UpdateWaterSplineData(const FUpdateWaterSplineDataParams& UpdateWaterSplineDataParams, FWaterCheckResult& WaterCheckResult)
{
	if(WaterCheckResult.HitResult.Component.IsValid())
	{
		if (AWaterBody* WaterBody = Cast<AWaterBody>(WaterCheckResult.HitResult.Component->GetOwner()))
		{
			if (WaterBody->GetWaterBodyComponent())
			{
				const TValueOrError<FWaterBodyQueryResult, EWaterBodyQueryError> QueryResult = WaterBody->GetWaterBodyComponent()->TryQueryWaterInfoClosestToWorldLocation(UpdateWaterSplineDataParams.PlayerLocation,
						   EWaterBodyQueryFlags::ComputeLocation
						   | EWaterBodyQueryFlags::ComputeDepth
						   | EWaterBodyQueryFlags::ComputeNormal
						   | EWaterBodyQueryFlags::ComputeVelocity
						   | EWaterBodyQueryFlags::ComputeImmersionDepth);

				if (!QueryResult.HasValue())
				{
					return;
				}

				const FWaterBodyQueryResult& Query = QueryResult.GetValue();
		
				FWaterFlowSplineData& WaterSplineData = WaterCheckResult.WaterSplineData;
		
				// Immersion depth
				WaterSplineData.ImmersionDepth = Query.GetImmersionDepth();
		
				WaterSplineData.WaterPlaneLocation = Query.GetWaterPlaneLocation();
				WaterSplineData.WaterPlaneNormal = Query.GetWaterPlaneNormal();
		
				// Water Depth
				WaterSplineData.WaterDepth = Query.GetWaterSurfaceDepth();
		
				// Water velocity
				WaterSplineData.RawWaterVelocity = Query.GetVelocity();
		
				// Water velocity modified by depth.
				const FVector2D DepthRange = FVector2D(UpdateWaterSplineDataParams.TargetImmersionDepth, UpdateWaterSplineDataParams.WaterVelocityDepthForMax);
				const FVector2D VelocityMultiplierRange = FVector2D(UpdateWaterSplineDataParams.WaterVelocityMinMultiplier, 1.0f);
				WaterSplineData.WaterVelocityDepthMultiplier = FMath::GetMappedRangeValueClamped(DepthRange, VelocityMultiplierRange, WaterSplineData.WaterDepth);
				WaterSplineData.WaterVelocity = WaterSplineData.RawWaterVelocity * WaterSplineData.WaterVelocityDepthMultiplier;
		
				// Player velocity relative to water velocity.
				const FVector PlayerVelocity2D = FVector(UpdateWaterSplineDataParams.PlayerVelocity.X, UpdateWaterSplineDataParams.PlayerVelocity.Y, 0.f);
				const FVector WaterVelocity2D = FVector(WaterSplineData.WaterVelocity.X, WaterSplineData.WaterVelocity.Y, 0.f);
				const FVector WaterVelocityProjection = PlayerVelocity2D.IsNearlyZero(0.1f) ? WaterVelocity2D : WaterVelocity2D.ProjectOnToNormal(PlayerVelocity2D.GetSafeNormal2D());
				WaterSplineData.PlayerRelativeVelocityToWater = PlayerVelocity2D - WaterVelocityProjection;
		
				// Surface location
				const float CapsuleBottom = UpdateWaterSplineDataParams.PlayerLocation.Z - UpdateWaterSplineDataParams.CapsuleHalfHeight;
				const float CapsuleTop = UpdateWaterSplineDataParams.PlayerLocation.Z + UpdateWaterSplineDataParams.CapsuleHalfHeight;
				WaterSplineData.WaterSurfaceLocation = Query.GetWaterSurfaceLocation();
		
				// Surface offset
				WaterSplineData.WaterSurfaceOffset = WaterSplineData.WaterSurfaceLocation - UpdateWaterSplineDataParams.PlayerLocation;
		
				// Immersion Percent
				WaterSplineData.ImmersionPercent = FMath::Clamp((WaterSplineData.WaterSurfaceLocation.Z - CapsuleBottom) / (CapsuleTop - CapsuleBottom), 0.f, 1.f);
		
				// Normal
				WaterSplineData.WaterSurfaceNormal = Query.GetWaterSurfaceNormal();
			}
		}
	}
}
