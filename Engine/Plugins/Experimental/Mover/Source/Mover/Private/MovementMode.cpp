// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovementMode.h"
#include "MovementModeTransition.h"
#include "MoverComponent.h"
#include "MoverLog.h"
#include "Engine/BlueprintGeneratedClass.h"

#if WITH_EDITOR
#include "Misc/DataValidation.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovementMode)

#define LOCTEXT_NAMESPACE "Mover"

UWorld* UBaseMovementMode::GetWorld() const
{
#if WITH_EDITOR
	// In the editor, GetWorld() is called on the CDO as part of checking ImplementsGetWorld().
	if (IsTemplate())
	{
		return nullptr;
	}
#endif
	return GetOuter()->GetWorld();
}


void UBaseMovementMode::OnRegistered(const FName ModeName, const FMoverSimContext& SimContext)
{
	for (TObjectPtr<UBaseMovementModeTransition>& Transition : Transitions)
	{
		if (Transition)
		{
			Transition->OnRegistered();
		}
		else
		{
			UE_LOGF(LogMover, Error, "Invalid or missing transition object on mode of type %ls of component %ls for actor %ls", *GetPathNameSafe(this), *GetNameSafe(GetOuter()), *GetNameSafe(GetOutermost()));
		}
	}
	
	K2_OnRegistered(ModeName);
}

void UBaseMovementMode::OnUnregistered(const FMoverSimContext& SimContext)
{
	for (TObjectPtr<UBaseMovementModeTransition>& Transition : Transitions)
	{
		if (Transition)
		{
			Transition->OnUnregistered();
		}
		else
		{
			UE_LOGF(LogMover, Error, "Invalid or missing transition object on mode of type %ls of component %ls for actor %ls", *GetPathNameSafe(this), *GetNameSafe(GetOuter()), *GetNameSafe(GetOutermost()));
		}
	}

	K2_OnUnregistered();
}

void UBaseMovementMode::Activate(const FMoverEventContext& Context, FName PrevModeName, const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, FMoverSyncState* OutSyncState, FMoverAuxStateContext* OutAuxState)
{
	if (!bSupportsAsync)
	{
		K2_OnActivated();
	}
}

void UBaseMovementMode::Deactivate(const FMoverEventContext& Context, FName NextModeName, const FMoverSimContext& SimContext)
{
	if (!bSupportsAsync)
	{
		K2_OnDeactivated();
	}
}

void UBaseMovementMode::Activate_External(const FMoverEventContext& Context)
{
	if (bSupportsAsync)
	{
		K2_OnActivated();
	}
}

void UBaseMovementMode::Deactivate_External(const FMoverEventContext& Context)
{
	if (bSupportsAsync)
	{
		K2_OnDeactivated();
	}
}

void UBaseMovementMode::GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const
{
}

void UBaseMovementMode::SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState)
{
}

UMoverComponent* UBaseMovementMode::K2_GetMoverComponent() const
{
	return Cast<UMoverComponent>(GetOuter());
}

#if WITH_EDITOR
EDataValidationResult UBaseMovementMode::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = EDataValidationResult::Valid;
	for (UBaseMovementModeTransition* Transition : Transitions)
	{
		if (!IsValid(Transition))
		{
			Context.AddError(FText::Format(LOCTEXT("InvalidTransitionOnModeError", "Invalid or missing transition object on mode of type {0}. Clean up the Transitions array."),
				FText::FromString(GetClass()->GetName())));

			Result = EDataValidationResult::Invalid;
		}
		else if (Transition->IsDataValid(Context) == EDataValidationResult::Invalid)
		{
			Result = EDataValidationResult::Invalid;
		}
	}

	return Result;
}
#endif // WITH_EDITOR


bool UBaseMovementMode::HasGameplayTag(FGameplayTag TagToFind, bool bExactMatch) const
{
	if (bExactMatch)
	{
		return GameplayTags.HasTagExact(TagToFind);
	}

	return GameplayTags.HasTag(TagToFind);
}

void UBaseMovementMode::GetGameplayTags(FGameplayTagContainer& InOutTags) const
{
	InOutTags.AppendTags(GameplayTags);
}

const FName UNullMovementMode::NullModeName(TEXT("Null"));

UNullMovementMode::UNullMovementMode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UNullMovementMode::SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState)
{
}

#undef LOCTEXT_NAMESPACE
