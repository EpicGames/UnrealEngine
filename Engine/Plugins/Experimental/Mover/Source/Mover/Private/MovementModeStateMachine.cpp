// Copyright Epic Games, Inc. All Rights Reserved.


#include "MovementModeStateMachine.h"
#include "MovementModeTransition.h"
#include "MoverDeveloperSettings.h"
#include "MoverLog.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoverComponent.h"
#include "MoverSimulation.h"
#include "MoveLibrary/MovementMixer.h"
#include "Templates/SubclassOf.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(MovementModeStateMachine)

namespace MoverComponentCVars
{
	static int32 SkipGenerateMoveIfOverridden = 1;
	FAutoConsoleVariableRef CVarSkipGenerateMoveIfOverridden(
		TEXT("mover.perf.SkipGenerateMoveIfOverridden"),
		SkipGenerateMoveIfOverridden,
		TEXT("If true and we have a layered move fully overriding movement, then we will skip calling OnGenerateMove on the active movement mode for better performance\n")
	);

} // end MoverComponentCVars

UMovementModeStateMachine::UMovementModeStateMachine(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UMovementModeStateMachine::BeginDestroy()
{
	QueuedModeTransition = nullptr;
	Simulation = nullptr;

	Super::BeginDestroy();
}

void UMovementModeStateMachine::RegisterMovementMode(FName ModeName, TObjectPtr<UBaseMovementMode> Mode, bool bIsDefaultMode)
{
	// JAH TODO: add validation and warnings for overwriting modes
	// JAH TODO: add validation of Mode

	Modes.Add(ModeName, Mode);

	if (bIsDefaultMode)
	{
		//JAH TODO: add validation that we are only overriding the default null mode
		DefaultModeName = ModeName;
	}

	Mode->OnRegistered(ModeName, GetSimContext());
}


void UMovementModeStateMachine::RegisterMovementMode(FName ModeName, TSubclassOf<UBaseMovementMode> ModeType, bool bIsDefaultMode)
{
	
	RegisterMovementMode(ModeName, NewObject<UBaseMovementMode>(GetOuter(), ModeType), bIsDefaultMode);

}



void UMovementModeStateMachine::UnregisterMovementMode(FName ModeName)
{
	TObjectPtr<UBaseMovementMode> ModeToUnregister;

	if (Modes.RemoveAndCopyValue(ModeName, ModeToUnregister) && ModeToUnregister)
	{
		ModeToUnregister->OnUnregistered(GetSimContext());
	}
}

void UMovementModeStateMachine::ClearAllMovementModes()
{
	if (Modes.Contains(CurrentModeName))
	{
		if (TObjectPtr<UBaseMovementMode> CurrentMode = Modes[CurrentModeName])
		{
			if (IsValid(CurrentMode))
			{
				FMoverEventContext MoverEventContext;
				MoverEventContext.EventTimeMs = CurrentBaseTimeStep.BaseSimTimeMs;
				MoverEventContext.ServerFrame = CurrentBaseTimeStep.ServerFrame;
				MoverEventContext.bIsDuringResimulation = false;
				MoverEventContext.bIsCausedByRollback = false;

				CurrentMode->Deactivate(MoverEventContext, NAME_None, GetSimContext());
			}
		}
	}

	for (TPair<FName, TObjectPtr<UBaseMovementMode>>& Element : Modes)
	{
		if (IsValid(Element.Value))
		{
			Element.Value->OnUnregistered(GetSimContext());
		}
	}

	Modes.Empty();
	
	ConstructDefaultModes();	// Note that we're resetting to our defaults so we keep the null movement mode
}

void UMovementModeStateMachine::SetDefaultMode(FName NewDefaultModeName)
{
	check(Modes.Contains(NewDefaultModeName));

	DefaultModeName = NewDefaultModeName;
}

void UMovementModeStateMachine::RegisterGlobalTransition(TObjectPtr<UBaseMovementModeTransition> Transition)
{
	GlobalTransitions.Add(Transition);

	Transition->OnRegistered();
}

void UMovementModeStateMachine::UnregisterGlobalTransition(TObjectPtr<UBaseMovementModeTransition> Transition)
{
	Transition->OnUnregistered();

	GlobalTransitions.Remove(Transition);
}

void UMovementModeStateMachine::ClearAllGlobalTransitions()
{
	for (TObjectPtr<UBaseMovementModeTransition>& Transition : GlobalTransitions)
	{
		if (IsValid(Transition))
		{
			Transition->OnUnregistered();
		}
	}

	GlobalTransitions.Empty();
}

void UMovementModeStateMachine::SetSimulation(UMoverSimulation& InSimulation)
{
	Simulation = &InSimulation;
}

void UMovementModeStateMachine::QueueNextMode(FName DesiredNextModeName, bool bShouldReenter)
{
	QueueNextModeInternal(DesiredNextModeName, bShouldReenter, /*bIsFromRollback*/ false);
}

void UMovementModeStateMachine::QueueNextModeInternal(FName DesiredNextModeName, bool bShouldReenter, bool bIsFromRollback)
{
	if (DesiredNextModeName != NAME_None)
	{ 
		const FName NextModeName          = QueuedModeTransition->GetNextModeName();
		const bool bShouldNextModeReenter = QueuedModeTransition->ShouldReenter();

		if ((NextModeName != NAME_None) &&
		    (NextModeName != DesiredNextModeName || bShouldReenter != bShouldNextModeReenter))
		{
			UE_LOGF(LogMover, Log, "%ls Overwriting of queued mode change (%ls, reenter: %i) with (%ls, reenter: %i)", *GetNameSafe(GetTypedOuter<AActor>()), *NextModeName.ToString(), bShouldNextModeReenter, *DesiredNextModeName.ToString(), bShouldReenter);
		}

		if (Modes.Contains(DesiredNextModeName))
		{
			QueuedModeTransition->SetNextMode(DesiredNextModeName, bShouldReenter, bIsFromRollback);
		}
		else
		{
			UE_LOGF(LogMover, Warning, "Attempted to queue an unregistered movement mode: %ls on owner %ls", *DesiredNextModeName.ToString(), *GetNameSafe(GetTypedOuter<AActor>()));
		}
	}
	else
	{
		ClearQueuedMode();
	}
}


void UMovementModeStateMachine::SetModeImmediately(FName DesiredModeName, bool bShouldReenter)
{
	SetModeImmediatelyInternal(DesiredModeName, bShouldReenter, /*bIsFromRollback*/ false);
}


void UMovementModeStateMachine::SetModeImmediatelyInternal(FName DesiredModeName, bool bShouldReenter, bool bIsFromRollback)
{
	QueueNextModeInternal(DesiredModeName, bShouldReenter, bIsFromRollback);
	AdvanceToNextMode();
}


void UMovementModeStateMachine::ClearQueuedMode()
{
	QueuedModeTransition->Clear();
}


void UMovementModeStateMachine::OnSimulationTick(USceneComponent* UpdatedComponent, UPrimitiveComponent* UpdatedPrimitive, UMoverBlackboard* SimBlackboard, const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FMoverTickEndData& OutputState)
{
	//GEngine->AddOnScreenDebugMessage(-1, -0.1f, FColor::White, FString::Printf(TEXT("Mode Tick: %s  Queued: %s"), *CurrentModeName.ToString(), *NextModeName.ToString()));
	FMoverTimeStep SubTimeStep = TimeStep;
	CurrentBaseTimeStep = TimeStep;

	WorkingSimTickEndData = &OutputState;
	ON_SCOPE_EXIT { WorkingSimTickEndData = nullptr; };

	WorkingSubstepStartData = StartState;
	bool bIsWorkingStartStateReady = true;	// this flag is used to avoid unneeded data copying after substeps

	UMoverComponent* MoverComp = CastChecked<UMoverComponent>(SimContext.SimulationOwner);
	check(MoverComp->MovementMixer);

	// Drain GT-staged queues from the simulation into the state machine.
	// After this, the state machine works on its own internal queues.
	if (SimContext.Simulation)
	{
		QueueInstantMovementEffects(SimContext.Simulation->ConsumeQueuedInstantMovementEffects());

		QueueLayeredMoves(SimContext.Simulation->ConsumeQueuedLayeredMoves());

		for (TSharedPtr<FLayeredMoveInstance>& MoveInstance : SimContext.Simulation->ConsumeQueuedLayeredMoveInstances())
		{
			QueueActiveLayeredMove(MoveInstance);
		}
	}

	if (!QueuedModeTransition->IsSet())
	{
		QueueNextMode(WorkingSubstepStartData.SyncState.MovementMode);
	}

	AdvanceToNextMode();

	int SubStepCount = 0;
	const int32 MaxConsecutiveFullRefundedSubsteps = GetDefault<UMoverDeveloperSettings>()->MaxTimesToRefundSubstep;
	int32 NumConsecutiveFullRefundedSubsteps = 0;

	float TotalUsedMs = 0.0f;
	while (TotalUsedMs < TimeStep.StepMs)
	{
		if (!bIsWorkingStartStateReady)
		{
			WorkingSubstepStartData.SyncState = OutputState.SyncState;
			WorkingSubstepStartData.AuxState = OutputState.AuxState;
			bIsWorkingStartStateReady = true;
		}

		WorkingSubstepStartData.SyncState.MovementMode = CurrentModeName;

		FMoverDefaultSyncState* OutputSyncState = &OutputState.SyncState.SyncStateCollection.FindOrAddMutableDataByType<FMoverDefaultSyncState>();
		OutputState.SyncState.MovementMode = CurrentModeName;

		OutputState.MovementEndState.ResetToDefaults();

		SubTimeStep.StepMs = TimeStep.StepMs - TotalUsedMs;		// TODO: convert this to an overridable function that can support MaxStepTime, MaxIterations, etc.

		// Process any cancellation requests first, so we can catch any queued features before they're activated
		FlushTagCancellationsToSyncState(WorkingSubstepStartData.SyncState);

		// Transfer any queued moves into the starting state. They'll be started during the move generation.
		FlushQueuedMovesToGroup(TimeStep, WorkingSubstepStartData.SyncState.LayeredMoves);
		OutputState.SyncState.LayeredMoves = WorkingSubstepStartData.SyncState.LayeredMoves;
		
		ActivateQueuedMoves(WorkingSubstepStartData.SyncState.LayeredMoveInstances);
		WorkingSubstepStartData.SyncState.LayeredMoveInstances.PopulateMissingActiveMoveLogic(*MoverComp->GetRegisteredMoves());
		OutputState.SyncState.LayeredMoveInstances = WorkingSubstepStartData.SyncState.LayeredMoveInstances;

		FlushQueuedModifiersToGroup(WorkingSubstepStartData.SyncState.MovementModifiers);
		OutputState.SyncState.MovementModifiers = WorkingSubstepStartData.SyncState.MovementModifiers;
		
		FApplyMovementEffectParams EffectParams;
		EffectParams.MoverComp = MoverComp;
		EffectParams.StartState = &WorkingSubstepStartData;
		EffectParams.TimeStep = &SubTimeStep;
		EffectParams.UpdatedComponent = UpdatedComponent;
		EffectParams.UpdatedPrimitive = UpdatedPrimitive;
		
		bool bModeSetFromInstantEffect = false;
		// Apply any instant effects that were queued up between ticks
		if (ApplyInstantEffects(EffectParams, OutputState.SyncState))
		{
			bModeSetFromInstantEffect = UpdateWorkingStateFromSyncState(OutputState.SyncState);
		}

		FMovementModifierGroup& CurrentModifiers = OutputState.SyncState.MovementModifiers;
		FlushModifierCancellationsToGroup(CurrentModifiers);
		TArray<TSharedPtr<FMovementModifierBase>> ActiveModifiers = CurrentModifiers.GenerateActiveModifiers(MoverComp, SubTimeStep, WorkingSubstepStartData.SyncState, WorkingSubstepStartData.AuxState);

		for (TSharedPtr<FMovementModifierBase> Modifier : ActiveModifiers)
		{
			Modifier->OnPreMovement(MoverComp, SubTimeStep);
		}
		
		FLayeredMoveGroup& CurrentLayeredMoves = OutputState.SyncState.LayeredMoves;
		FLayeredMoveInstanceGroup& CurrentActiveLayeredMoves = OutputState.SyncState.LayeredMoveInstances;

		// Gather any layered move contributions
		FProposedMove CombinedLayeredMove;
		CombinedLayeredMove.MixMode = EMoveMixMode::AdditiveVelocity;
		bool bHasLayeredMoveContributions = false;
		MoverComp->MovementMixer->ResetMixerState();
		
		TArray<TSharedPtr<FLayeredMoveBase>> ActiveMoves = CurrentLayeredMoves.GenerateActiveMoves(SubTimeStep, MoverComp, SimBlackboard);
		CurrentActiveLayeredMoves.FlushMoveArrays(SubTimeStep, SimBlackboard);
		bHasLayeredMoveContributions = CurrentActiveLayeredMoves.GenerateMixedMove(WorkingSubstepStartData, SubTimeStep, *MoverComp->MovementMixer, SimBlackboard, CombinedLayeredMove);

		// Tick and accumulate all active moves
		// Gather all proposed moves and distill this into a cumulative movement report. May include separate additive vs override moves.
		// TODO: may want to sort by priority or other factors
		for (TSharedPtr<FLayeredMoveBase>& ActiveMove : ActiveMoves)
		{
			FProposedMove MoveStep;
			MoveStep.MixMode = ActiveMove->MixMode;	// Initialize using the move's mixmode, but allow it to be changed in GenerateMove

			if (ActiveMove->GenerateMove(WorkingSubstepStartData, SubTimeStep, MoverComp, SimBlackboard, MoveStep))
			{
				// If this active move is already past it's first tick we don't need to set the preferred mode again
				if (ActiveMove->StartSimTimeMs < SubTimeStep.BaseSimTimeMs)
				{
					MoveStep.PreferredMode = NAME_None;
				}

				// Some modes may handle vertical movement themselves (e.g. falling) during anim root motion
				if (MoveStep.MixMode == EMoveMixMode::OverrideAll
					&& ActiveMove->HasGameplayTag(Mover_AnimRootMotion, /*exact?*/false)
					&& MoverComp->HasGameplayTagInState(WorkingSubstepStartData.SyncState, Mover_SkipVerticalAnimRootMotion, /*exact?*/false))
				{
					MoveStep.MixMode = EMoveMixMode::OverrideAllExceptVerticalVelocity;
				}
				
				bHasLayeredMoveContributions = true;
				MoverComp->MovementMixer->MixLayeredMove(*ActiveMove, MoveStep, CombinedLayeredMove);
			}
		}

		// Flush instant effects queued during this substep from the simulation into the state machine.
		if (SimContext.Simulation)
		{
			QueueInstantMovementEffects(SimContext.Simulation->ConsumeQueuedInstantMovementEffects());
		}
		
		// Apply any instant effects that were queued from various movement effects - Modifiers, layered moves, etc.
		if (ApplyInstantEffects(EffectParams, OutputState.SyncState))
		{
			bModeSetFromInstantEffect = UpdateWorkingStateFromSyncState(OutputState.SyncState);
		}
		
		if (bHasLayeredMoveContributions && !CombinedLayeredMove.PreferredMode.IsNone() && !bModeSetFromInstantEffect)
		{
			SetModeImmediately(CombinedLayeredMove.PreferredMode);
			OutputState.SyncState.MovementMode = CurrentModeName;
		}

		// Merge proposed movement from the current mode with movement from layered moves
		if (!CurrentModeName.IsNone() && Modes.Contains(CurrentModeName))
		{
			UBaseMovementMode* CurrentMode = Modes[CurrentModeName];
			FProposedMove CombinedMove;
			bool bHasModeMoveContribution = false;

			if (!MoverComponentCVars::SkipGenerateMoveIfOverridden ||
				!(bHasLayeredMoveContributions && CombinedLayeredMove.MixMode == EMoveMixMode::OverrideAll))
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_GenerateMoveFromMode);
				CurrentMode->GenerateMove(SimContext, WorkingSubstepStartData, SubTimeStep, OUT CombinedMove);

				bHasModeMoveContribution = true;
			}

			if (bHasModeMoveContribution && bHasLayeredMoveContributions)
			{
				MoverComp->MovementMixer->MixProposedMoves(CombinedLayeredMove, MoverComp->GetUpDirection(), CombinedMove);
			}
			else if (bHasLayeredMoveContributions && !bHasModeMoveContribution)
			{
				CombinedMove = CombinedLayeredMove;
			}

			// Apply any layered move finish velocity settings
			if (CurrentLayeredMoves.bApplyResidualVelocity)
			{
				CombinedMove.LinearVelocity = CurrentLayeredMoves.ResidualVelocity;
			}
			if (CurrentLayeredMoves.ResidualClamping >= 0.0f)
			{
				CombinedMove.LinearVelocity = CombinedMove.LinearVelocity.GetClampedToMaxSize(CurrentLayeredMoves.ResidualClamping);
			}
			CurrentLayeredMoves.ResetResidualVelocity();

			MoverComp->ProcessGeneratedMovement.ExecuteIfBound(WorkingSubstepStartData, SubTimeStep, OUT CombinedMove);
			
			// Execute the combined proposed move
			{
				WorkingSimTickParams.SimContext = SimContext;
				WorkingSimTickParams.StartState = WorkingSubstepStartData;
				WorkingSimTickParams.MovingComps.SetFrom(MoverComp);
				WorkingSimTickParams.SimBlackboard = SimBlackboard;
				WorkingSimTickParams.TimeStep = SubTimeStep;
				WorkingSimTickParams.ProposedMove = CombinedMove;

				// Check for any transitions, first those registered with the current movement mode, then global ones that could occur from any mode
				FTransitionEvalResult EvalResult = FTransitionEvalResult::NoTransition;
				TObjectPtr<UBaseMovementModeTransition> TransitionToTrigger;

				for (UBaseMovementModeTransition* Transition : CurrentMode->Transitions)
				{
					if (IsValid(Transition) && ((SubStepCount == 0) || !Transition->bFirstSubStepOnly))
					{
						EvalResult = Transition->Evaluate(WorkingSimTickParams);

						if (!EvalResult.NextMode.IsNone())
						{
							if (EvalResult.NextMode != CurrentModeName || Transition->bAllowModeReentry)
							{
								TransitionToTrigger = Transition;
								break;
							}
						}
					}
				}

				if (TransitionToTrigger == nullptr)
				{
					for (UBaseMovementModeTransition* Transition : GlobalTransitions)
					{
						if (IsValid(Transition))
						{
							EvalResult = Transition->Evaluate(WorkingSimTickParams);

							if (!EvalResult.NextMode.IsNone())
							{
								if (EvalResult.NextMode != CurrentModeName || Transition->bAllowModeReentry)
								{
									TransitionToTrigger = Transition;
									break;
								}
							}
						}
					}
				}

				if (TransitionToTrigger && !EvalResult.NextMode.IsNone())
				{
					OutputState.MovementEndState.NextModeName = EvalResult.NextMode;
					OutputState.MovementEndState.RemainingMs = WorkingSimTickParams.TimeStep.StepMs; 	// Pass all remaining time to next mode
					TransitionToTrigger->Trigger(WorkingSimTickParams);

					MoverComp->OnMovementTransitionTriggered.Broadcast(TransitionToTrigger);
				}
				else
				{
					CurrentMode->SimulationTick(WorkingSimTickParams, OutputState);
				}

				OutputState.MovementEndState.RemainingMs = FMath::Clamp(OutputState.MovementEndState.RemainingMs, 0.0f, SubTimeStep.StepMs);
			}

			// If nothing else is trying to change the mode, follow the active mode's recommendation (if any)
			if (!QueuedModeTransition->IsSet())
			{
				QueueNextMode(OutputState.MovementEndState.NextModeName);
			}

			// Check if all of the time for this Substep was refunded
			if (FMath::IsNearlyEqual(SubTimeStep.StepMs, OutputState.MovementEndState.RemainingMs, UE_KINDA_SMALL_NUMBER))
			{
				NumConsecutiveFullRefundedSubsteps++;
				// if we've done this sub step a lot before go ahead and just advance time to avoid freezing editor
				if (NumConsecutiveFullRefundedSubsteps >= MaxConsecutiveFullRefundedSubsteps)
				{
					UE_LOGF(LogMover, Warning, "Movement mode %ls and %ls on %ls are stuck giving time back to each other. Overriding to advance to next substep.", *CurrentModeName.ToString(), *OutputState.MovementEndState.NextModeName.ToString(), *MoverComp->GetOwner()->GetName());
					TotalUsedMs += SubTimeStep.StepMs;
				}
			}
			else
			{
				NumConsecutiveFullRefundedSubsteps = 0;
			}

			//GEngine->AddOnScreenDebugMessage(-1, -0.1f, FColor::White, FString::Printf(TEXT("NextModeName: %s  Queued: %s"), *Output.MovementEndState.NextModeName.ToString(), *NextModeName.ToString()));
		}

		// Flush any instant effects queued during the substep loop from the simulation into the state machine.
		if (SimContext.Simulation)
		{
			QueueInstantMovementEffects(SimContext.Simulation->ConsumeQueuedInstantMovementEffects());
		}
	
		if (HasAnyInstantEffectsQueued())
		{
			FApplyMovementEffectParams EndSubStepEffectParams;
			EndSubStepEffectParams.MoverComp = MoverComp;
			EndSubStepEffectParams.StartState = &WorkingSubstepStartData;
			EndSubStepEffectParams.TimeStep = &SubTimeStep;
			EndSubStepEffectParams.UpdatedComponent = UpdatedComponent;
			EndSubStepEffectParams.UpdatedPrimitive = UpdatedPrimitive;
	
			// Apply any instant effects that were queued up during this tick and didn't get handled in a substep
			if (ApplyInstantEffects(EndSubStepEffectParams, OutputState.SyncState))
			{
				if (CurrentModeName != OutputState.SyncState.MovementMode)
				{
					SetModeImmediately(OutputState.SyncState.MovementMode);
				}
			}
		}

		const float RemainingMs = FMath::Clamp(OutputState.MovementEndState.RemainingMs, 0.0f, SubTimeStep.StepMs);
		const float SubstepUsedMs = (SubTimeStep.StepMs - RemainingMs);
		CurrentBaseTimeStep.BaseSimTimeMs = SubTimeStep.BaseSimTimeMs + SubstepUsedMs;
		TotalUsedMs += SubstepUsedMs;

		// Switch modes if necessary (note that this will allow exit/enter on the same state)
		AdvanceToNextMode();
		OutputState.SyncState.MovementMode = CurrentModeName;

		for (TSharedPtr<FMovementModifierBase> Modifier : ActiveModifiers)
		{
			Modifier->OnPostMovement(MoverComp, SubTimeStep, OutputState.SyncState, OutputState.AuxState);
		}
		
		SubTimeStep.BaseSimTimeMs += SubstepUsedMs;
		SubTimeStep.StepMs = RemainingMs;

		bIsWorkingStartStateReady = false;
		++SubStepCount;
	}
}

void UMovementModeStateMachine::OnSimulationPreRollback(const FMoverSyncState* InvalidSyncState, const FMoverSyncState* SyncState, const FMoverAuxStateContext* InvalidAuxState, const FMoverAuxStateContext* AuxState, const FMoverTimeStep& NewBaseTimeStep)
{
	CurrentBaseTimeStep = NewBaseTimeStep;
	RollbackModifiers(InvalidSyncState, SyncState, InvalidAuxState, AuxState);
}

void UMovementModeStateMachine::OnSimulationRollback(const FMoverSyncState* SyncState, const FMoverAuxStateContext* AuxState, const FMoverTimeStep& NewBaseTimeStep)
{
	ClearQueuedMode();

	if (CurrentModeName != SyncState->MovementMode)
	{
		SetModeImmediatelyInternal(SyncState->MovementMode, /*bShouldReenter*/ false, /*bIsFromRollback*/ true);
	}

	RollBackQueuedLayeredMoves(NewBaseTimeStep);

	{
		UE::TRWScopeLock QueueLock(LayeredMoveQueueLock, SLT_Write);
		QueuedLayeredMoveInstances.Empty();
	}
	
	RollBackQueuedInstantMovementEffects(NewBaseTimeStep);

	{
		UE::TRWScopeLock QueueLock(ModifiersQueueLock, SLT_Write);
		QueuedMovementModifiers.Empty();
	}
}

void UMovementModeStateMachine::RollBackQueuedInstantMovementEffects(const FMoverTimeStep& NewBaseTimeStep)
{
	UE::TRWScopeLock QueueLock(InstantEffectsQueueLock, SLT_Write);

	for (int EffectIndex = 0; EffectIndex < QueuedInstantEffects.Num();)
	{
		const FScheduledInstantMovementEffect& QueuedEffect = QueuedInstantEffects[EffectIndex];
		// We roll back all effects which were issued on or after NewBaseTimeStep
		if (QueuedEffect.SchedulingInfo.ShouldRollBackAtTimeStep(NewBaseTimeStep))
		{
			QueuedInstantEffects.RemoveAt(EffectIndex);
		}
		else
		{
			EffectIndex++;
		}
	}
}

void UMovementModeStateMachine::RollBackQueuedLayeredMoves(const FMoverTimeStep& NewBaseTimeStep)
{
	UE::TRWScopeLock QueueLock(LayeredMoveQueueLock, SLT_Write);
	
	for (int MoveIndex = 0; MoveIndex < QueuedLayeredMoves.Num();)
	{
		const FScheduledLayeredMove& QueuedMove = QueuedLayeredMoves[MoveIndex];
		// We roll back all moves which were issued on or after NewBaseTimeStep
		if (QueuedMove.SchedulingInfo.ShouldRollBackAtTimeStep(NewBaseTimeStep))
		{
			QueuedLayeredMoves.RemoveAt(MoveIndex);
		}
		else
		{
			MoveIndex++;
		}
	}
}

const UBaseMovementMode* UMovementModeStateMachine::GetCurrentMode() const
{
	if (CurrentModeName != NAME_None && Modes.Contains(CurrentModeName))
	{
		return Modes[CurrentModeName];
	}

	return nullptr;
}

FName UMovementModeStateMachine::GetQueuedModeName() const
{
	if (QueuedModeTransition)
	{
		return QueuedModeTransition->GetNextModeName();
	}

	return NAME_None;
}

const UBaseMovementMode* UMovementModeStateMachine::FindMovementMode(FName ModeName) const
{
	if (ModeName != NAME_None && Modes.Contains(ModeName))
	{
		return Modes[ModeName];
	}

	return nullptr;
}

void UMovementModeStateMachine::QueueInstantMovementEffect(const FScheduledInstantMovementEffect& ScheduledInstantMovementEffect)
{
	UE::TRWScopeLock QueueLock(InstantEffectsQueueLock, SLT_Write);
	QueuedInstantEffects.Add(ScheduledInstantMovementEffect);
}

void UMovementModeStateMachine::QueueInstantMovementEffects(const TArray<FScheduledInstantMovementEffect>& ScheduledInstantMovementEffects)
{
	UE::TRWScopeLock QueueLock(InstantEffectsQueueLock, SLT_Write);
	for (const FScheduledInstantMovementEffect& ScheduledInstantMovementEffect : ScheduledInstantMovementEffects)
	{
		QueuedInstantEffects.Add(ScheduledInstantMovementEffect);
	}
}

void UMovementModeStateMachine::QueueInstantMovementEffect_Internal(const FScheduledInstantMovementEffect& ScheduledInstantMovementEffect)
{
	ensure(!IsInGameThread());
	QueueInstantMovementEffect(ScheduledInstantMovementEffect);
}

void UMovementModeStateMachine::QueueLayeredMove(const FScheduledLayeredMove& ScheduledLayeredMove)
{
	UE::TRWScopeLock QueueLock(LayeredMoveQueueLock, SLT_Write);
	QueuedLayeredMoves.Add(ScheduledLayeredMove);
}

void UMovementModeStateMachine::QueueLayeredMoves(const TArray<FScheduledLayeredMove>& ScheduledLayeredMoves)
{
	UE::TRWScopeLock QueueLock(LayeredMoveQueueLock, SLT_Write);
	for (const FScheduledLayeredMove& ScheduledLayeredMove : ScheduledLayeredMoves)
	{
		QueuedLayeredMoves.Add(ScheduledLayeredMove);
	}
}

void UMovementModeStateMachine::QueueLayeredMove_Internal(const FScheduledLayeredMove& ScheduledLayeredMove)
{
	ensure(!IsInGameThread());
	QueueLayeredMove(ScheduledLayeredMove);
}

void UMovementModeStateMachine::QueueActiveLayeredMove(const TSharedPtr<FLayeredMoveInstance>& LayeredMove)
{
	UE::TRWScopeLock QueueLock(LayeredMoveQueueLock, SLT_Write);
	QueuedLayeredMoveInstances.Add(LayeredMove);
}

FMovementModifierHandle UMovementModeStateMachine::QueueMovementModifier(TSharedPtr<FMovementModifierBase> Modifier)
{
	if (ensure(Modifier.IsValid()))
	{
		UE::TRWScopeLock QueueLock(ModifiersQueueLock, SLT_Write);

		QueuedMovementModifiers.Add(Modifier);
		Modifier->GenerateHandle();

		return Modifier->GetHandle();
	}

	return 0;
}

void UMovementModeStateMachine::CancelModifierFromHandle(FMovementModifierHandle ModifierHandle)
{
	{
		UE::TRWScopeLock QueueLock(ModifiersQueueLock, SLT_Write);
		QueuedMovementModifiers.RemoveAll([ModifierHandle, this]
		(const TSharedPtr<FMovementModifierBase>& Modifier)
		{
			if (Modifier.IsValid())
			{
				if (Modifier->GetHandle() == ModifierHandle)
				{
					return true;
				}
			}
			else
			{
				return true;	
			}

			return false;
		});
	}
	
	{
		UE::TRWScopeLock QueueLock(ModifierCancelQueueLock, SLT_Write);
		ModifiersToCancel.Add(ModifierHandle);
	}
}

const FMovementModifierBase* UMovementModeStateMachine::FindQueuedModifier(FMovementModifierHandle ModifierHandle) const
{
	for (const TSharedPtr<FMovementModifierBase>& QueuedModifier : QueuedMovementModifiers)
	{
		if (QueuedModifier->GetHandle() == ModifierHandle)
		{
			return QueuedModifier.Get();
		}
	}

	return nullptr;
}

const FMovementModifierBase* UMovementModeStateMachine::FindQueuedModifierByType(const UScriptStruct* ModifierType) const
{
	for (const TSharedPtr<FMovementModifierBase>& QueuedModifier : QueuedMovementModifiers)
	{
		if (QueuedModifier->GetScriptStruct()->IsChildOf(ModifierType))
		{
			return QueuedModifier.Get();
		}
	}

	return nullptr;
}

void UMovementModeStateMachine::CancelFeaturesWithTag(FGameplayTag TagToCancel, bool bRequireExactMatch)
{
	// Cancel all matching queued movement features
	{
		UE::TRWScopeLock QueueLock(ModifiersQueueLock, SLT_Write);
		QueuedMovementModifiers.RemoveAll([TagToCancel, bRequireExactMatch] (const TSharedPtr<FMovementModifierBase>& Modifier)
			{
				return (Modifier.IsValid() && Modifier->HasGameplayTag(TagToCancel, bRequireExactMatch));
			});
	}

	{
		UE::TRWScopeLock QueueLock(LayeredMoveQueueLock, SLT_Write);
		QueuedLayeredMoves.RemoveAll([TagToCancel, bRequireExactMatch](const FScheduledLayeredMove& LayeredMove)
			{
				return (LayeredMove.Move.IsValid() && LayeredMove.Move->HasGameplayTag(TagToCancel, bRequireExactMatch));
			});
	}

	{
		UE::TRWScopeLock QueueLock(LayeredMoveQueueLock, SLT_Write);
		QueuedLayeredMoveInstances.RemoveAll([TagToCancel, bRequireExactMatch](const TSharedPtr<FLayeredMoveInstance>& LayeredMoveInstance)
			{
				return (LayeredMoveInstance.IsValid() && LayeredMoveInstance->HasGameplayTag(TagToCancel, bRequireExactMatch));
			});
	}

	// TODO: also support cancellation of queued instant effects if they end up supporting gameplay tags

	// Request cancellation of any matching ACTIVE movement features during the next simulation tick
	{
		UE::TRWScopeLock QueueLock(ModifierCancelQueueLock, SLT_Write);
		TagCancellationRequests.Add(TPair<FGameplayTag, bool>(TagToCancel, bRequireExactMatch));
	}
}

bool UMovementModeStateMachine::HasFeaturesWithTag(FGameplayTag TagToMatch, bool bRequireExactMatch) const
{
	bool bResult = false;

	// Compare all matching queued movement features
	{
		UE::TRWScopeLock QueueLock(ModifiersQueueLock, SLT_ReadOnly);
		const TSharedPtr<FMovementModifierBase>* FindResult = QueuedMovementModifiers.FindByPredicate([TagToMatch, bRequireExactMatch] (const TSharedPtr<FMovementModifierBase>& Modifier)
			{
				return (Modifier.IsValid() && Modifier->HasGameplayTag(TagToMatch, bRequireExactMatch));
			});
		bResult = FindResult != nullptr;
	}

	if (!bResult)
	{
		UE::TRWScopeLock QueueLock(LayeredMoveQueueLock, SLT_ReadOnly);
		const FScheduledLayeredMove* FindResult = QueuedLayeredMoves.FindByPredicate([TagToMatch, bRequireExactMatch](const FScheduledLayeredMove& LayeredMove)
			{
				return (LayeredMove.Move.IsValid() && LayeredMove.Move->HasGameplayTag(TagToMatch, bRequireExactMatch));
			});
		bResult = FindResult != nullptr;
	}

	if (!bResult)
	{
		UE::TRWScopeLock QueueLock(LayeredMoveQueueLock, SLT_ReadOnly);
		const TSharedPtr<FLayeredMoveInstance>* FindResult = QueuedLayeredMoveInstances.FindByPredicate([TagToMatch, bRequireExactMatch](const TSharedPtr<FLayeredMoveInstance>& LayeredMoveInstance)
			{
				return (LayeredMoveInstance.IsValid() && LayeredMoveInstance->HasGameplayTag(TagToMatch, bRequireExactMatch));
			});
		bResult = FindResult != nullptr;
	}

	return bResult;
}


void UMovementModeStateMachine::ConstructDefaultModes()
{
	RegisterMovementMode(UNullMovementMode::NullModeName, UNullMovementMode::StaticClass(), true);

	DefaultModeName = NAME_None;
	CurrentModeName = UNullMovementMode::NullModeName;

	ClearQueuedMode();
}

void UMovementModeStateMachine::AdvanceToNextMode()
{
	const FName NextModeName = QueuedModeTransition->GetNextModeName();
	const bool bShouldNextModeReenter = QueuedModeTransition->ShouldReenter();
	const bool bIsCausedByRollback = QueuedModeTransition->IsCausedByRollback();

	ClearQueuedMode();

	if ((NextModeName != NAME_None && Modes.Contains(NextModeName)) &&
		(CurrentModeName != NextModeName || bShouldNextModeReenter))
	{
		UE_LOGF(LogMover, Verbose, "AdvanceToNextMode: %ls from %ls to %ls", 
			*GetNameSafe(GetTypedOuter<AActor>()), *CurrentModeName.ToString(), *NextModeName.ToString());

		const FName PreviousModeName = CurrentModeName;
		CurrentModeName = NextModeName;

		FMoverEventContext MoverEventContext;
		MoverEventContext.EventTimeMs = CurrentBaseTimeStep.BaseSimTimeMs;
		MoverEventContext.ServerFrame = CurrentBaseTimeStep.ServerFrame;
		MoverEventContext.bIsDuringResimulation = CurrentBaseTimeStep.bIsResimulating;
		MoverEventContext.bIsCausedByRollback = bIsCausedByRollback;

		FMoverSimContext SimContext = GetSimContext();

		if (PreviousModeName != NAME_None && Modes.Contains(PreviousModeName))
		{
			Modes[PreviousModeName]->Deactivate(MoverEventContext, NextModeName, SimContext);
		}

		// signal movement mode change event
		const UMoverComponent* MoverComp = CastChecked<UMoverComponent>(SimContext.SimulationOwner);
		

		if (Simulation)
		{
			FRollbackBlackboardSimWrapper RollbackBlackboard = Simulation->GetRollbackBlackboardSimWrapper();

			FMovementModeChangeRecord ModeChangeRecord;
			ModeChangeRecord.ModeName = CurrentModeName;
			ModeChangeRecord.PrevModeName = PreviousModeName;
			ModeChangeRecord.Frame = CurrentBaseTimeStep.ServerFrame;
			ModeChangeRecord.SimTimeMs = CurrentBaseTimeStep.BaseSimTimeMs;

			RollbackBlackboard.TrySet(CommonBlackboard::LastModeChangeRecord, ModeChangeRecord);
		}

		FMoverSyncState* InProgressSyncState = nullptr;
		FMoverAuxStateContext* InProgressAuxState = nullptr;

		if (WorkingSimTickEndData)
		{
			InProgressSyncState = &WorkingSimTickEndData->SyncState;
			InProgressAuxState = &WorkingSimTickEndData->AuxState;
		}

		Modes[CurrentModeName]->Activate(MoverEventContext, PreviousModeName, SimContext, WorkingSimTickParams.StartState, InProgressSyncState, InProgressAuxState);

		MoverComp->OnMovementModeChanged.Broadcast(PreviousModeName, NextModeName);
	}
}

void UMovementModeStateMachine::FlushQueuedMovesToGroup(const FMoverTimeStep& TimeStep, FLayeredMoveGroup& Group)
{
	UE::TRWScopeLock QueueLock(LayeredMoveQueueLock, SLT_Write);

	for (int MoveIndex = 0; MoveIndex < QueuedLayeredMoves.Num(); )
	{
		const FScheduledLayeredMove& QueuedMove = QueuedLayeredMoves[MoveIndex];
		bool bShouldFlushMove = false;
		if (QueuedMove.SchedulingInfo.bIsFixedDt)
		{
			bShouldFlushMove = QueuedMove.SchedulingInfo.ShouldExecuteAtFrame(TimeStep.ServerFrame);
		}
		else
		{
			double MaxServerTimeMsThisFrame = TimeStep.BaseSimTimeMs + TimeStep.StepMs;
			bShouldFlushMove = QueuedMove.SchedulingInfo.ShouldExecuteAtTimeMs(MaxServerTimeMsThisFrame);
		}
		if (bShouldFlushMove)
		{
			// First enqueue, then remove, to make sure any shared pointer ref doesn't go to 0 and accidentally delete underlying objects. Don't ask me how I know.
			Group.QueueLayeredMove(QueuedMove.Move);
			QueuedLayeredMoves.RemoveAt(MoveIndex);
		}
		else
		{
			MoveIndex++;
		}
	}
}

void UMovementModeStateMachine::ActivateQueuedMoves(FLayeredMoveInstanceGroup& Group)
{
	UE::TRWScopeLock QueueLock(LayeredMoveQueueLock, SLT_Write);

	if (!QueuedLayeredMoveInstances.IsEmpty())
	{
		for (TSharedPtr<FLayeredMoveInstance>& QueuedMove : QueuedLayeredMoveInstances)
		{
			Group.QueueLayeredMove(QueuedMove);
		}
		
		QueuedLayeredMoveInstances.Empty();
	}
}

void UMovementModeStateMachine::FlushQueuedModifiersToGroup(FMovementModifierGroup& ModifierGroup)
{
	UE::TRWScopeLock QueueLock(ModifiersQueueLock, SLT_Write);

	if (!QueuedMovementModifiers.IsEmpty())
	{
		for (TSharedPtr<FMovementModifierBase>& QueuedModifier : QueuedMovementModifiers)
		{
			ModifierGroup.QueueMovementModifier(QueuedModifier);
		}
		
		QueuedMovementModifiers.Empty();
	}
}

void UMovementModeStateMachine::FlushModifierCancellationsToGroup(FMovementModifierGroup& ActiveModifierGroup)
{
	UE::TRWScopeLock QueueLock(ModifierCancelQueueLock, SLT_Write);
	
	for (FMovementModifierHandle HandleToCancel : ModifiersToCancel)
	{
		ActiveModifierGroup.CancelModifierFromHandle(HandleToCancel);
	}
	
	ModifiersToCancel.Empty();
}

void UMovementModeStateMachine::FlushTagCancellationsToSyncState(FMoverSyncState& SyncState)
{
	UE::TRWScopeLock QueueLock(TagCancellationRequestsLock, SLT_Write);

	for (TPair<FGameplayTag, bool> TagCancelRequest : TagCancellationRequests)
	{
		SyncState.MovementModifiers.CancelModifiersByTag(TagCancelRequest.Key, TagCancelRequest.Value);
		SyncState.LayeredMoves.CancelMovesByTag(TagCancelRequest.Key, TagCancelRequest.Value);
		SyncState.LayeredMoveInstances.CancelMovesByTag(TagCancelRequest.Key, TagCancelRequest.Value);
	}

	TagCancellationRequests.Empty();
}

void UMovementModeStateMachine::RollbackModifiers(const FMoverSyncState* InvalidSyncState, const FMoverSyncState* SyncState, const FMoverAuxStateContext* InvalidAuxState, const FMoverAuxStateContext* AuxState)
{
	{
		UE::TRWScopeLock QueueLock(ModifiersQueueLock, SLT_Write);
		QueuedMovementModifiers.Empty();
	}
	
	if (UMoverComponent* MoverComp = Cast<UMoverComponent>(GetOuter()))
	{
		for (auto ModifierFromRollbackIt = SyncState->MovementModifiers.GetActiveModifiersIterator(); ModifierFromRollbackIt; ++ModifierFromRollbackIt)
		{
			bool bContainsModifier = false;
			for (auto ModifierFromCacheIt = InvalidSyncState->MovementModifiers.GetActiveModifiersIterator(); ModifierFromCacheIt; ++ModifierFromCacheIt)
			{
				if (ModifierFromRollbackIt->Get()->Matches(ModifierFromCacheIt->Get()))
				{
					bContainsModifier = true;

					// Rolled back version of the modifier will be missing the handle; we fix that here
					ModifierFromRollbackIt->Get()->OverwriteHandleIfInvalid(ModifierFromCacheIt->Get()->GetHandle());

					break;
				}
			}
		
			if (!bContainsModifier)
			{
				UE_LOGF(LogMover, Log, "Modifier(%ls) was started on %ls after a rollback.", *ModifierFromRollbackIt->Get()->ToSimpleString(), *GetNameSafe(MoverComp->GetOwner()));
				ModifierFromRollbackIt->Get()->OnStart(MoverComp, MoverComp->GetLastTimeStep(), *SyncState, *AuxState);
			}
		}

		for (auto ModifierFromCacheIt = InvalidSyncState->MovementModifiers.GetActiveModifiersIterator(); ModifierFromCacheIt; ++ModifierFromCacheIt)
		{
			bool bContainsModifier = false;
			for (auto ModifierFromRollbackIt = SyncState->MovementModifiers.GetActiveModifiersIterator(); ModifierFromRollbackIt; ++ModifierFromRollbackIt)
			{
				if (ModifierFromRollbackIt->Get()->Matches(ModifierFromCacheIt->Get()))
				{
					bContainsModifier = true;
					break;
				}
			}
	
			if (!bContainsModifier)
			{
				UE_LOGF(LogMover, Log, "Modifier(%ls) was ended on %ls after a rollback.", *ModifierFromCacheIt->Get()->ToSimpleString(), *GetNameSafe(MoverComp->GetOwner()));
				ModifierFromCacheIt->Get()->OnEnd(MoverComp, MoverComp->GetLastTimeStep(), *SyncState, *AuxState);
			}
		}
	}
}

bool UMovementModeStateMachine::HasAnyInstantEffectsQueued() const 
{
	UE::TRWScopeLock QueueLock(InstantEffectsQueueLock, SLT_ReadOnly);

	return !QueuedInstantEffects.IsEmpty();
}

bool UMovementModeStateMachine::ApplyInstantEffects(FApplyMovementEffectParams& ApplyEffectParams, FMoverSyncState& OutputState)
{
	bool bInstantMovementEffectApplied = false;

	UE::TRWScopeLock QueueLock(InstantEffectsQueueLock, SLT_Write);

	for (int EffectIndex = 0; EffectIndex < QueuedInstantEffects.Num(); )
	{
		const FScheduledInstantMovementEffect& QueuedEffect = QueuedInstantEffects[EffectIndex];
		bool bShouldApplyEffect = false;
		if (QueuedEffect.SchedulingInfo.bIsFixedDt)
		{
			bShouldApplyEffect = QueuedEffect.SchedulingInfo.ShouldExecuteAtFrame(ApplyEffectParams.TimeStep->ServerFrame);
		}
		else
		{
			double MaxServerTimeMsThisFrame = ApplyEffectParams.TimeStep->BaseSimTimeMs + ApplyEffectParams.TimeStep->StepMs;
			bShouldApplyEffect = QueuedEffect.SchedulingInfo.ShouldExecuteAtTimeMs(MaxServerTimeMsThisFrame);
		}
		if (bShouldApplyEffect)
		{
			bInstantMovementEffectApplied |= QueuedEffect.Effect->ApplyMovementEffect(ApplyEffectParams, OutputState);
			QueuedInstantEffects.RemoveAt(EffectIndex);
			ProcessEvents(ApplyEffectParams.OutputEvents);
			ApplyEffectParams.OutputEvents.Empty();
		}
		else
		{
			EffectIndex++;
		}
	}
	
	return bInstantMovementEffectApplied;
}

bool UMovementModeStateMachine::UpdateWorkingStateFromSyncState(FMoverSyncState& OutputState)
{
	bool bModeSetFromInstantEffect = false;
	
	// Copying over our sync state collection to SubstepStartData so it is effectively the input sync state later for the movement mode. Doing this makes sure state modification from Instant Effects isn't overridden later by the movement mode
	for (auto SyncDataIt = OutputState.SyncStateCollection.GetCollectionDataIterator(); SyncDataIt; ++SyncDataIt)
	{
		if (SyncDataIt->Get())
		{
			WorkingSubstepStartData.SyncState.SyncStateCollection.AddDataByCopy(SyncDataIt->Get());
		}
	}
	
	if (CurrentModeName != OutputState.MovementMode)
	{
		bModeSetFromInstantEffect = true;
		SetModeImmediately(OutputState.MovementMode);
		WorkingSubstepStartData.SyncState.MovementMode = CurrentModeName;
	}
	
	return bModeSetFromInstantEffect;
}


void UMovementModeStateMachine::ProcessEvents(const TArray<TSharedPtr<FMoverSimulationEventData>>& InEvents)
{
	FMoverSimContext SimContext = GetSimContext();
	UMoverComponent* MoverComp = CastChecked<UMoverComponent>(SimContext.SimulationOwner);
	
	for (TSharedPtr<FMoverSimulationEventData> Event : InEvents)
	{
		if (const FMoverSimulationEventData* EventData = Event.Get())
		{
			ProcessSimulationEvent(*EventData);

#if !UE_BUILD_SHIPPING
			ensureMsgf(IsInGameThread(), TEXT("Dispatching an event to the mover component from outside the game thread, this is not thread safe"));
#endif
			MoverComp->DispatchSimulationEvent(*EventData);
		}
	}
}

void UMovementModeStateMachine::ProcessSimulationEvent(const FMoverSimulationEventData& EventData)
{
}


FMoverSimContext UMovementModeStateMachine::GetSimContext() const
{
	FMoverSimContext SimContext;
	
	if (Simulation)
	{
		SimContext.Simulation = Simulation;
		SimContext.Blackboard = Simulation->GetRollbackBlackboardSimWrapper();
		SimContext.SimulationOwner = CastChecked<UMoverComponent>(Simulation->GetOuter());
	}

	return SimContext;
}

void UMovementModeStateMachine::PostInitProperties()
{
	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		QueuedModeTransition = NewObject<UImmediateMovementModeTransition>(this, TEXT("QueuedModeTransition"), RF_Transient);
	}
}
