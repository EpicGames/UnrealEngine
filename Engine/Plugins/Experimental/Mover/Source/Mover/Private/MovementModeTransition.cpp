// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovementModeTransition.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "MoverComponent.h"
#include "MoverSimulationTypes.h"
#include "MoverTypes.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(MovementModeTransition)

const FTransitionEvalResult FTransitionEvalResult::NoTransition = FTransitionEvalResult();

UWorld* UBaseMovementModeTransition::GetWorld() const
{
	if (UMoverComponent* MoverComponent = GetMoverComponent())
	{
		return MoverComponent->GetWorld();
	}
	return nullptr;
}

void UBaseMovementModeTransition::OnRegistered()
{
	K2_OnRegistered();
}

void UBaseMovementModeTransition::OnUnregistered()
{
	K2_OnUnregistered();
}

UMoverComponent* UBaseMovementModeTransition::K2_GetMoverComponent() const
{
	// Transitions can belong to either a mode or the component itself - either way they're always ultimately outer'd to a mover comp
	return GetTypedOuter<UMoverComponent>();
}

FTransitionEvalResult UBaseMovementModeTransition::Evaluate_Implementation(const FSimulationTickParams& Params) const
{
	return FTransitionEvalResult::NoTransition;
}

void UBaseMovementModeTransition::Trigger_Implementation(const FSimulationTickParams& Params)
{
}

#if WITH_EDITOR
EDataValidationResult UBaseMovementModeTransition::IsDataValid(FDataValidationContext& Context) const
{
	return EDataValidationResult::Valid;
}
#endif // WITH_EDITOR

UImmediateMovementModeTransition::UImmediateMovementModeTransition(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Clear();
}

FTransitionEvalResult UImmediateMovementModeTransition::Evaluate_Implementation(const FSimulationTickParams& Params) const
{
	if (NextMode != NAME_None)
	{
		if (bAllowModeReentry)
		{
			return FTransitionEvalResult(NextMode);
		}
		else if (NextMode != Params.StartState.SyncState.MovementMode)
		{
			return FTransitionEvalResult(NextMode);
		}
	}

	return FTransitionEvalResult::NoTransition;
}

void UImmediateMovementModeTransition::Trigger_Implementation(const FSimulationTickParams& Params)
{
	Clear();
}

void UImmediateMovementModeTransition::SetNextMode(FName DesiredModeName, bool bShouldReenter, bool bIsFromRollback)
{
	NextMode = DesiredModeName;
	bAllowModeReentry = bShouldReenter;
	bIsCausedByRollback = bIsFromRollback;
}

void UImmediateMovementModeTransition::Clear()
{
	NextMode = NAME_None;
	bAllowModeReentry = false;
	bIsCausedByRollback = false;
}
