// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovementModifier.h"
#include "MoverComponent.h"
#include "MoverModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovementModifier)

const float MovementModifier_InvalidTime = -UE_BIG_NUMBER;

void FMovementModifierHandle::GenerateHandle()
{
	static MODIFIER_HANDLE_TYPE LocalModifierIDGenerator = 0;
	MODIFIER_HANDLE_TYPE LocalID = ++LocalModifierIDGenerator;
	
	// TODO: might want to change this from a magic number 0
	if (LocalID == 0)
	{
		LocalID = ++LocalModifierIDGenerator;
	}

	Handle = LocalID;
}

FMovementModifierBase::FMovementModifierBase()
	: DurationMs(-1)
	, StartSimTimeMs(MovementModifier_InvalidTime)
{
}

void FMovementModifierBase::StartModifier(UMoverComponent* MoverComp, const FMoverTimeStep& TimeStep, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState)
{
	StartSimTimeMs = TimeStep.BaseSimTimeMs;
	OnStart(MoverComp, TimeStep, SyncState, AuxState);
}

void FMovementModifierBase::EndModifier(UMoverComponent* MoverComp, const FMoverTimeStep& TimeStep, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState)
{
	OnEnd(MoverComp, TimeStep, SyncState, AuxState);
}

void FMovementModifierBase::StartModifier_Async(const FMovementModifierParams_Async& Params)
{
	check(Params.TimeStep);
	StartSimTimeMs = Params.TimeStep->BaseSimTimeMs;
	OnStart_Async(Params);
}

void FMovementModifierBase::EndModifier_Async(const FMovementModifierParams_Async& Params)
{
	OnEnd_Async(Params);
}

bool FMovementModifierBase::IsFinished(double CurrentSimTimeMs) const
{
	const bool bHasStarted = (StartSimTimeMs >= 0.f);
	const bool bTimeExpired = bHasStarted && (DurationMs > 0.f) && (StartSimTimeMs + DurationMs <= CurrentSimTimeMs);
	const bool bDidTickOnceAndExpire = bHasStarted && (DurationMs == 0.f);

	return bTimeExpired || bDidTickOnceAndExpire;
}

FMovementModifierBase* FMovementModifierBase::Clone() const
{
	// If child classes don't override this, saved modifiers will not work
	checkf(false, TEXT("FMovementModifierBase::Clone() being called erroneously from %s. A FMovementModifierBase should never be queued directly and Clone should always be overridden in child structs!"), *GetNameSafe(GetScriptStruct()));
	return nullptr;
}

void FMovementModifierBase::NetSerialize(FArchive& Ar)
{
	Ar << DurationMs;
	Ar << StartSimTimeMs;
}

UScriptStruct* FMovementModifierBase::GetScriptStruct() const
{
	return FMovementModifierBase::StaticStruct();
}

FString FMovementModifierBase::ToSimpleString() const
{
	return GetScriptStruct()->GetName();
}

bool FMovementModifierBase::Matches(const FMovementModifierBase* Other) const
{
	// TODO: Consider checking other factors other than just type
	return Other != nullptr && GetScriptStruct() == Other->GetScriptStruct();
}

FMovementModifierHandle FMovementModifierBase::GetHandle() const
{
	return LocalModifierHandle;
}

void FMovementModifierBase::GenerateHandle()
{
	LocalModifierHandle.GenerateHandle();
}

void FMovementModifierBase::OverwriteHandleIfInvalid(const FMovementModifierHandle& ValidModifierHandle)
{
	if (ValidModifierHandle.IsValid() && !LocalModifierHandle.IsValid())
	{
		LocalModifierHandle = ValidModifierHandle;
	}
}

void FMovementModifierGroup::NetSerialize(FArchive& Ar, uint8 MaxNumModifiersToSerialize)
{
	// TODO: Warn if some sources will be dropped
	const uint8 NumActiveMovesToSerialize = FMath::Min<int32>(ActiveModifiers.Num(), MaxNumModifiersToSerialize);
	const uint8 NumQueuedMovesToSerialize = NumActiveMovesToSerialize < MaxNumModifiersToSerialize ? MaxNumModifiersToSerialize - NumActiveMovesToSerialize : 0;
	NetSerializeMovementModifierArray(Ar, ActiveModifiers, NumActiveMovesToSerialize);
	NetSerializeMovementModifierArray(Ar, QueuedModifiers, NumQueuedMovesToSerialize);
}

void FMovementModifierGroup::QueueMovementModifier(TSharedPtr<FMovementModifierBase> Modifier)
{
	if (ensure(Modifier.IsValid()))
	{
		QueuedModifiers.Add(Modifier);
		UE_LOGF(LogMover, VeryVerbose, "Queued Movement Modifier (%ls)", *Modifier->ToSimpleString());
	}
}

void FMovementModifierGroup::CancelModifierFromHandle(const FMovementModifierHandle& HandleToCancel)
{
	for (TSharedPtr<FMovementModifierBase> ActiveModifier : ActiveModifiers)
	{ 
		if (HandleToCancel == ActiveModifier->GetHandle())
		{
			ActiveModifier->DurationMs = 0;
		}
	}

	QueuedModifiers.RemoveAll([HandleToCancel](const TSharedPtr<FMovementModifierBase>& Modifier)
		{
			return (!Modifier.IsValid() || HandleToCancel == Modifier->GetHandle());
		});

}

void FMovementModifierGroup::CancelModifiersByTag(FGameplayTag Tag, bool bRequiresExactMatch)
{
	for (TSharedPtr<FMovementModifierBase> ActiveModifier : ActiveModifiers)
	{
		if (ActiveModifier.IsValid() && ActiveModifier->HasGameplayTag(Tag, bRequiresExactMatch))
		{
			ActiveModifier->DurationMs = 0;
		}
	}

	QueuedModifiers.RemoveAll([Tag, bRequiresExactMatch](const TSharedPtr<FMovementModifierBase>& Modifier)
		{
			return (!Modifier.IsValid() || Modifier->HasGameplayTag(Tag, bRequiresExactMatch));
		});
}


TArray<TSharedPtr<FMovementModifierBase>> FMovementModifierGroup::GenerateActiveModifiers(UMoverComponent* MoverComp, const FMoverTimeStep& TimeStep, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState)
{
	FlushModifierArrays(MoverComp, TimeStep, SyncState, AuxState);
	return ActiveModifiers;
}

TArray<TSharedPtr<FMovementModifierBase>> FMovementModifierGroup::GenerateActiveModifiers_Async(const FMovementModifierParams_Async& Params)
{
	FlushModifierArrays_Async(Params);
	return ActiveModifiers;
}

static void CopyModifierArray(TArray<TSharedPtr<FMovementModifierBase>>& Dest, const TArray<TSharedPtr<FMovementModifierBase>>& Src)
{
	bool bCanCopyInPlace = (UE::Mover::DisableDataCopyInPlace == 0 && Dest.Num() == Src.Num());
	if (bCanCopyInPlace)
	{
		// If copy in place is enabled and the arrays are the same size, copy by index
		for (int32 i = 0; i < Dest.Num(); ++i)
		{
			if (FMovementModifierBase* SrcData = Src[i].Get())
			{
				FMovementModifierBase* DestData = Dest[i].Get();
				UScriptStruct* SourceStruct = SrcData->GetScriptStruct();

				if (DestData && SourceStruct == DestData->GetScriptStruct())
				{
					// Same type so copy in place
					SourceStruct->CopyScriptStruct(DestData, SrcData, 1);
				}
				else
				{
					// Different type so replace the shared ptr with a clone
					Dest[i] = TSharedPtr<FMovementModifierBase>(SrcData->Clone());
				}
			}
			else
			{
				// Found invalid source, fall back to full copy
				bCanCopyInPlace = false;
				break;
			}
		}
	}

	if (!bCanCopyInPlace)
	{
		// Deep copy active modifiers
		Dest.Empty(Src.Num());
		for (int i = 0; i < Src.Num(); ++i)
		{
			if (Src[i].IsValid())
			{
				FMovementModifierBase* CopyOfSourcePtr = Src[i]->Clone();
				Dest.Add(TSharedPtr<FMovementModifierBase>(CopyOfSourcePtr));
			}
			else
			{
				UE_LOGF(LogMover, Warning, "CopyModifierArray trying to copy invalid Other Modifier");
			}
		}
	}

}


FMovementModifierGroup& FMovementModifierGroup::operator=(const FMovementModifierGroup& Other)
{
	// Perform deep copy of this Group
	if (this != &Other)
	{
		CopyModifierArray(ActiveModifiers, Other.ActiveModifiers);
		CopyModifierArray(QueuedModifiers, Other.QueuedModifiers);
	}

	return *this;
}

bool FMovementModifierGroup::operator==(const FMovementModifierGroup& Other) const
{
	if (ActiveModifiers.Num() != Other.ActiveModifiers.Num())
	{
		return false;
	}
	if (QueuedModifiers.Num() != Other.QueuedModifiers.Num())
	{
		return false;
	}


	for (int32 i = 0; i < ActiveModifiers.Num(); ++i)
	{
		if (ActiveModifiers[i].IsValid() == Other.ActiveModifiers[i].IsValid())
		{
			if (ActiveModifiers[i].IsValid())
			{
				// TODO: Implement deep equality checks
			}
		}
		else
		{
			return false; // Mismatch in validity
		}
	}
	for (int32 i = 0; i < QueuedModifiers.Num(); ++i)
	{
		if (QueuedModifiers[i].IsValid() == Other.QueuedModifiers[i].IsValid())
		{
			if (QueuedModifiers[i].IsValid())
			{
				// TODO: Implement deep equality checks
			}
		}
		else
		{
			return false; // Mismatch in validity
		}
	}
	return true;
}

bool FMovementModifierGroup::operator!=(const FMovementModifierGroup& Other) const
{
	return !(FMovementModifierGroup::operator==(Other));
}


bool FMovementModifierGroup::HasSameContents(const FMovementModifierGroup& Other) const
{
	// Only compare the types of modifiers contained, not the state
	if (ActiveModifiers.Num() != Other.ActiveModifiers.Num() ||
		QueuedModifiers.Num() != Other.QueuedModifiers.Num())
	{
		return false;
	}

	for (int32 i = 0; i < ActiveModifiers.Num(); ++i)
	{
		if (ActiveModifiers[i]->GetScriptStruct() != Other.ActiveModifiers[i]->GetScriptStruct())
		{
			return false;
		}
	}

	for (int32 i = 0; i < QueuedModifiers.Num(); ++i)
	{
		if (QueuedModifiers[i]->GetScriptStruct() != Other.QueuedModifiers[i]->GetScriptStruct())
		{
			return false;
		}
	}

	return true;
}


void FMovementModifierGroup::AddStructReferencedObjects(FReferenceCollector& Collector) const
{
	for (const TSharedPtr<FMovementModifierBase>& Modifier : ActiveModifiers)
	{
		if (Modifier.IsValid())
		{
			Modifier->AddReferencedObjects(Collector);
		}
	}

	for (const TSharedPtr<FMovementModifierBase>& Modifier : QueuedModifiers)
	{
		if (Modifier.IsValid())
		{
			Modifier->AddReferencedObjects(Collector);
		}
	}
}

FString FMovementModifierGroup::ToSimpleString() const
{
	return FString::Printf(TEXT("FMovementModifierGroup: Active: %i Queued: %i"), ActiveModifiers.Num(), QueuedModifiers.Num());
}

TArray<TSharedPtr<FMovementModifierBase>>::TConstIterator FMovementModifierGroup::GetActiveModifiersIterator() const
{
	return ActiveModifiers.CreateConstIterator();
}

TArray<TSharedPtr<FMovementModifierBase>>::TConstIterator FMovementModifierGroup::GetQueuedModifiersIterator() const
{
	return QueuedModifiers.CreateConstIterator();
}

void FMovementModifierGroup::FlushModifierArrays(UMoverComponent* MoverComp, const FMoverTimeStep& TimeStep, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState)
{
	// Remove any finished moves
	ActiveModifiers.RemoveAll([MoverComp, TimeStep, SyncState, AuxState, this]
		(const TSharedPtr<FMovementModifierBase>& Modifier)
		{
			if (Modifier.IsValid())
			{
				if (Modifier->IsFinished(TimeStep.BaseSimTimeMs))
				{
					Modifier->EndModifier(MoverComp, TimeStep, SyncState, AuxState);
					return true;
				}
			}
			else
			{
				return true;	
			}

			return false;
		});

	// Make any queued moves active
	for (TSharedPtr<FMovementModifierBase>& QueuedModifier : QueuedModifiers)
	{
		bool bModifierAlreadyActive = false;
		for (TSharedPtr<FMovementModifierBase>& ActiveModifier : ActiveModifiers)
		{
			// We don't really need to assign the QueuedModifier a start time but it would help if modifiers are compared based off of start time as well
			QueuedModifier->StartSimTimeMs = TimeStep.BaseSimTimeMs;

			// We only want to queue this queued modifier if it wasn't already added from state received from authority. If we already have the modifier just assign it a handle since it's already been activated.
			if (QueuedModifier->Matches(ActiveModifier.Get()))
			{
				ActiveModifier->OverwriteHandleIfInvalid(QueuedModifier->GetHandle());
				bModifierAlreadyActive = true;
				break;
			}
		}

		if (!bModifierAlreadyActive)
		{
			ActiveModifiers.Add(QueuedModifier);
			QueuedModifier->StartModifier(MoverComp, TimeStep, SyncState, AuxState);
		}
	}

	QueuedModifiers.Empty();
}

void FMovementModifierGroup::FlushModifierArrays_Async(const FMovementModifierParams_Async& Params)
{
	check(Params.TimeStep);

	// Remove any finished moves
	ActiveModifiers.RemoveAll([Params, this]
	(const TSharedPtr<FMovementModifierBase>& Modifier)
		{
			if (Modifier.IsValid())
			{
				if (Modifier->IsFinished(Params.TimeStep->BaseSimTimeMs))
				{
					Modifier->EndModifier_Async(Params);
					return true;
				}
			}
			else
			{
				return true;
			}

			return false;
		});

	// Make any queued moves active
	for (TSharedPtr<FMovementModifierBase>& QueuedModifier : QueuedModifiers)
	{
		bool bModifierAlreadyActive = false;
		for (TSharedPtr<FMovementModifierBase>& ActiveModifier : ActiveModifiers)
		{
			// We don't really need to assign the QueuedModifier a start time but it would help if modifiers are compared based off of start time as well
			QueuedModifier->StartSimTimeMs = Params.TimeStep->BaseSimTimeMs;

			// We only want to queue this queued modifier if it wasn't already added from state received from authority. If we already have the modifier just assign it a handle since it's already been activated.
			if (QueuedModifier->Matches(ActiveModifier.Get()))
			{
				ActiveModifier->OverwriteHandleIfInvalid(QueuedModifier->GetHandle());
				bModifierAlreadyActive = true;
				break;
			}
		}

		if (!bModifierAlreadyActive)
		{
			ActiveModifiers.Add(QueuedModifier);
			QueuedModifier->StartModifier_Async(Params);
		}
	}

	QueuedModifiers.Empty();
}

struct FMovementModifierDeleter
{
	FORCEINLINE void operator()(FMovementModifierBase* Object) const
	{
		check(Object);
		UScriptStruct* ScriptStruct = Object->GetScriptStruct();
		check(ScriptStruct);
		ScriptStruct->DestroyStruct(Object);
		FMemory::Free(Object);
	}
};

/* static */ void FMovementModifierGroup::NetSerializeMovementModifierArray(FArchive& Ar, TArray< TSharedPtr<FMovementModifierBase> >& ModifierArray, uint8 MaxNumModifiersToSerialize)
{
	uint8 NumModifiersToSerialize;
	if (Ar.IsSaving())
	{
		UE_CLOGF(ModifierArray.Num() > MaxNumModifiersToSerialize, LogMover, Warning, "Too many Modifiers (%d!) to net serialize. Clamping to %d",
			ModifierArray.Num(), MaxNumModifiersToSerialize);

		NumModifiersToSerialize = FMath::Min<int32>(ModifierArray.Num(), MaxNumModifiersToSerialize);
	}

	Ar << NumModifiersToSerialize;

	if (Ar.IsLoading())
	{
		ModifierArray.SetNumZeroed(NumModifiersToSerialize);
	}

	for (int32 i = 0; i < NumModifiersToSerialize && !Ar.IsError(); ++i)
	{
		TCheckedObjPtr<UScriptStruct> ScriptStruct = ModifierArray[i].IsValid() ? ModifierArray[i]->GetScriptStruct() : nullptr;
		UScriptStruct* ScriptStructLocal = ScriptStruct.Get();
		Ar << ScriptStruct;

		if (ScriptStruct.IsValid())
		{
			// Restrict replication to derived classes of FMovementModifierBase for security reasons:
			// If FMovementModifierGroup is replicated through a Server RPC, we need to prevent clients from sending us
			// arbitrary ScriptStructs due to the allocation/reliance on GetCppStructOps below which could trigger a server crash
			// for invalid structs. All provided sources are direct children of FMovementModifierBase and we never expect to have deep hierarchies
			// so this should not be too costly
			bool bIsDerivedFromBase = false;
			UStruct* CurrentSuperStruct = ScriptStruct->GetSuperStruct();
			while (CurrentSuperStruct)
			{
				if (CurrentSuperStruct == FMovementModifierBase::StaticStruct())
				{
					bIsDerivedFromBase = true;
					break;
				}
				CurrentSuperStruct = CurrentSuperStruct->GetSuperStruct();
			}

			if (bIsDerivedFromBase)
			{
				if (Ar.IsLoading())
				{
					if (ModifierArray[i].IsValid() && ScriptStructLocal == ScriptStruct.Get())
					{
						// What we have locally is the same type as we're being serialized into, so we don't need to
						// reallocate - just use existing structure
					}
					else
					{
						// For now, just reset/reallocate the data when loading.
						// Longer term if we want to generalize this and use it for property replication, we should support
						// only reallocating when necessary
						FMovementModifierBase* NewModifier = (FMovementModifierBase*)FMemory::Malloc(ScriptStruct->GetCppStructOps()->GetSize());
						ScriptStruct->InitializeStruct(NewModifier);

						ModifierArray[i] = TSharedPtr<FMovementModifierBase>(NewModifier, FMovementModifierDeleter());
					}
				}

				ModifierArray[i]->NetSerialize(Ar);
			}
			else
			{
				UE_LOGF(LogMover, Error, "FMovementModifierGroup::NetSerialize: ScriptStruct not derived from FMovementModifierBase attempted to serialize.");
				Ar.SetError();
				break;
			}
		}
		else if (ScriptStruct.IsError())
		{
			UE_LOGF(LogMover, Error, "FMovementModifierGroup::NetSerialize: Invalid ScriptStruct serialized.");
			Ar.SetError();
			break;
		}
	}
}

void FMovementModifierGroup::Reset()
{
	QueuedModifiers.Empty();
	ActiveModifiers.Empty();
}

bool FMovementModifierGroup::ShouldReconcile(const FMovementModifierGroup& Other) const
{
	// Only compare the types of modifiers contained, not the state
	if (ActiveModifiers.Num() != Other.ActiveModifiers.Num() ||
		QueuedModifiers.Num() != Other.QueuedModifiers.Num())
	{
		return true;
	}


	for (int32 i = 0; i < ActiveModifiers.Num(); ++i)
	{
		if (ActiveModifiers[i]->GetScriptStruct() != Other.ActiveModifiers[i]->GetScriptStruct())
		{
			return true;
		}
	}

	for (int32 i = 0; i < QueuedModifiers.Num(); ++i)
	{
		if (QueuedModifiers[i]->GetScriptStruct() != Other.QueuedModifiers[i]->GetScriptStruct())
		{
			return true;
		}
	}

	return false;
}