// Copyright Epic Games, Inc. All Rights Reserved.


#include "MoverComponent.h"
#include "MoverGameplayTagLog.h"
#include "MoverSimulationTypes.h"
#include "MovementModeStateMachine.h"
#include "MoverSimulation.h"
#include "MotionWarpingMoverAdapter.h"
#include "DefaultMovementSet/Modes/WalkingMode.h"
#include "DefaultMovementSet/Modes/FallingMode.h"
#include "DefaultMovementSet/Modes/FlyingMode.h"
#include "MoveLibrary/MovementMixer.h"
#include "MoveLibrary/MovementUtils.h"
#include "MoveLibrary/FloorQueryUtils.h"
#include "MoveLibrary/RollbackBlackboardWrappers.h"
#include "InputContainerStruct.h"
#include "MoverLog.h"
#include "InstantMovementEffect.h"
#include "Backends/MoverNetworkPredictionLiaison.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimMontage.h"
#include "Components/MeshComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/ScopedMovementUpdate.h"
#include "Engine/World.h"
#include "GameFramework/PhysicsVolume.h"
#include "Misc/AssertionMacros.h"
#include "Misc/TransactionObjectEvent.h"
#include "Blueprint/BlueprintExceptionInfo.h"
#include "UObject/ObjectSaveContext.h"
#include "Components/SkeletalMeshComponent.h"
#include "MotionWarpingComponent.h"
#include "ChaosVisualDebugger/MoverCVDRuntimeTrace.h"
#include "MoverSchedulingUtils.h"

#if WITH_EDITOR
#include "Misc/DataValidation.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoverComponent)

#define LOCTEXT_NAMESPACE "Mover"

namespace MoverCVars
{
	bool bEnableGameplayTagLog = false;
	FAutoConsoleVariableRef CVarEnableGameplayTagLog(
		TEXT("mover.debug.EnableGameplayTagLog"),
		bEnableGameplayTagLog,
		TEXT("Enable Mover gameplay tag event logging across the worker thread (WT:*) and game thread (GT:*)."),
		ECVF_Default
	);

	bool bEnableAsyncMontageSimMaster = true;
	FAutoConsoleVariableRef CVarEnableAsyncMontageSimMaster(
		TEXT("mover.anim.AsyncMontageSimMaster"),
		bEnableAsyncMontageSimMaster,
		TEXT("true (default): simulation is master of montage position, GT advances to interpolated simulation position in post sim. false: GT starts playing montage in presim, then snaps position to interpolated sim in post sim to avoid drift"),
		ECVF_Default
	);
}

namespace MoverComponentCVars
{
	static int32 WarnOnPostSimDifference = 0;
	FAutoConsoleVariableRef CVarMoverWarnOnPostSimDifference(
		TEXT("mover.debug.WarnOnPostSimDifference"),
		WarnOnPostSimDifference,
		TEXT("If != 0, then any differences between the sim sync state and the component locations just after movement simulation will emit warnings.\n")
	);

	// When enabled, GetGameplayTags() and HasGameplayTag() use the AllGameplayTags snapshot in the synchronous (non-async) backend,
	// matching the behavior of the async (ChaosMover) backend. The snapshot is rebuilt each FinalizeFrame and whenever external tags
	// are added or removed. When disabled (default), both functions perform a live walk of the active sync state instead.
	static int32 UseSnapshotGameplayTagsForSyncBackend = 0;
	FAutoConsoleVariableRef CVarUseSnapshotGameplayTagsForSyncBackend(
		TEXT("mover.UseSnapshotGameplayTagsForSyncBackend"),
		UseSnapshotGameplayTagsForSyncBackend,
		TEXT("If != 0, GetGameplayTags() and HasGameplayTag() use the cached AllGameplayTags snapshot in the synchronous backend rather than performing a live walk.\n")
	);

} // end MoverComponentCVars



namespace MoverComponentConstants
{
	const FVector DefaultGravityAccel	= FVector(0.0, 0.0, -980.0);
	const FVector DefaultUpDir			= FVector(0.0, 0.0, 1.0);
}


static constexpr float ROTATOR_TOLERANCE = (1e-3f);

UMoverComponent::UMoverComponent()
{
	PrimaryComponentTick.TickGroup = TG_PrePhysics;
	PrimaryComponentTick.bCanEverTick = false;

	BasedMovementTickFunction.bCanEverTick = true;
	BasedMovementTickFunction.bStartWithTickEnabled = false;
	BasedMovementTickFunction.SetTickFunctionEnable(false);
	BasedMovementTickFunction.TickGroup = TG_PostPhysics;

	bWantsInitializeComponent = true;
	bAutoActivate = true;

	PersistentSyncStateDataTypes.Add(FMoverDataPersistence(FMoverDefaultSyncState::StaticStruct(), true));

	BackendClass = UMoverNetworkPredictionLiaisonComponent::StaticClass();
}


void UMoverComponent::InitializeComponent()
{
	TGuardValue<bool> InInitializeComponentGuard(bInInitializeComponent, true);

	Super::InitializeComponent();

	const UWorld* MyWorld = GetWorld();

	if (MyWorld && MyWorld->IsGameWorld())
	{

		FindDefaultUpdatedComponent();


		// Instantiate our sister backend component that will actually talk to the system driving the simulation
		if (BackendClass)
		{
			UActorComponent* NewLiaisonComp = Cast<UActorComponent>(GetOwner()->FindComponentByInterface<IMoverBackendLiaisonInterface>());
			if (!NewLiaisonComp)
			{
				NewLiaisonComp = NewObject<UActorComponent>(this, BackendClass, TEXT("BackendLiaisonComponent"));
			}

			BackendLiaisonComp.SetObject(NewLiaisonComp);
			BackendLiaisonComp.SetInterface(CastChecked<IMoverBackendLiaisonInterface>(NewLiaisonComp));
			if (BackendLiaisonComp)
			{
				NewLiaisonComp->RegisterComponent();
				if (!NewLiaisonComp->HasBeenInitialized())
				{
					NewLiaisonComp->InitializeComponent();
				}
				NewLiaisonComp->SetNetAddressable();
			}
		}
		else
		{
			UE_LOGF(LogMover, Error, "No backend class set on %ls. Mover actor will not function.", *GetNameSafe(GetOwner()));
		}
	}

	// Gather initial state to fulfill queries
	FMoverSyncState DefaultMoverSyncState;
	GetDefaultInputAndState(OUT CachedLastProducedInputCmd, OUT DefaultMoverSyncState, OUT CachedLastAuxState);
	MoverSyncStateDoubleBuffer.SetBufferedData(DefaultMoverSyncState);
	CachedLastUsedInputCmd = CachedLastProducedInputCmd;
	LastMoverDefaultSyncState = MoverSyncStateDoubleBuffer.GetReadable().SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
}


void UMoverComponent::UninitializeComponent()
{
	if (UActorComponent* LiaisonAsComp = Cast<UActorComponent>(BackendLiaisonComp.GetObject()))
	{
		LiaisonAsComp->DestroyComponent();
	}
	BackendLiaisonComp = nullptr;
	Simulation = nullptr;

	if (SimBlackboard)
	{
		SimBlackboard->InvalidateAll();
		SimBlackboard = nullptr;
	}

	Super::UninitializeComponent();
}


void UMoverComponent::OnRegister()
{
	TGuardValue<bool> InOnRegisterGuard(bInOnRegister, true);

	Super::OnRegister();

	RefreshSharedSettings();
	
	// JAH NOTE: here we may need to do something to cause the simulation to re-initialize since its modes pointers are now wrong
	//  Likely: can we just trigger a re-init of the Simulation?  Might need to be via the backend component's OnRegister
	FindAndRegisterDefaults();
}


void UMoverComponent::RegisterComponentTickFunctions(bool bRegister)
{
	Super::RegisterComponentTickFunctions(bRegister);

	// Super may start up the tick function when we don't want to.
	UpdateTickRegistration();

	// If the owner ticks, make sure we tick first. This is to ensure the owner's location will be up to date when it ticks.
	AActor* Owner = GetOwner();

	if (bRegister && PrimaryComponentTick.bCanEverTick && Owner && Owner->CanEverTick())
	{
		Owner->PrimaryActorTick.AddPrerequisite(this, PrimaryComponentTick);
	}


	if (bRegister)
	{
		if (SetupActorComponentTickFunction(&BasedMovementTickFunction))
		{
			BasedMovementTickFunction.TargetMoverComp = this;
			BasedMovementTickFunction.AddPrerequisite(this, this->PrimaryComponentTick);
		}
	}
	else
	{
		if (BasedMovementTickFunction.IsTickFunctionRegistered())
		{
			BasedMovementTickFunction.UnRegisterTickFunction();
		}
	}
}


void UMoverComponent::PostLoad()
{
	Super::PostLoad();

	RefreshSharedSettings();
}

void UMoverComponent::BeginPlay()
{
	Super::BeginPlay();

	ensureMsgf(Simulation != nullptr, TEXT("MoverComponent requires a Simulation to be set during initialization. The backend should have done this already."));

	static URollbackBlackboard::EntrySettings ModeChangeRecordSettings;
	ModeChangeRecordSettings.SizingPolicy = EBlackboardSizingPolicy::FixedDeclaredSize;
	ModeChangeRecordSettings.FixedSize = 4;
	ModeChangeRecordSettings.PersistencePolicy = EBlackboardPersistencePolicy::Forever;
	ModeChangeRecordSettings.RollbackPolicy = EBlackboardRollbackPolicy::InvalidatedOnRollback;
	RollbackBlackboardExternalWrapper.CreateEntry<FMovementModeChangeRecord>(CommonBlackboard::LastModeChangeRecord, ModeChangeRecordSettings);

	FindDefaultUpdatedComponent();
	ensureMsgf(UpdatedComponent != nullptr, TEXT("No root component found on %s. Simulation initialization will most likely fail."), *GetPathNameSafe(GetOwner()));

	WorldToGravityTransform = FQuat::FindBetweenNormals(FVector::UpVector, GetUpDirection());
	GravityToWorldTransform = WorldToGravityTransform.Inverse();
	
	AActor* MyActor = GetOwner();
	if (MyActor)
	{
		// If no primary visual component is already set, fall back to searching for any kind of mesh,
		// favoring a direct scene child of the UpdatedComponent.
		if (!PrimaryVisualComponent)
		{
			if (UpdatedComponent)
			{
				for (USceneComponent* ChildComp : UpdatedComponent->GetAttachChildren())
				{
					if (ChildComp->IsA<UMeshComponent>())
					{
						SetPrimaryVisualComponent(ChildComp);
						break;
					}
				}
			}

			if (!PrimaryVisualComponent)
			{
				SetPrimaryVisualComponent(MyActor->FindComponentByClass<UMeshComponent>());
			}
		}

		ensureMsgf(UpdatedComponent, TEXT("A Mover actor (%s) must have an UpdatedComponent"), *GetNameSafe(MyActor));

		// Optional motion warping support
		if (UMotionWarpingComponent* WarpingComp = MyActor->FindComponentByClass<UMotionWarpingComponent>())
		{
			UMotionWarpingMoverAdapter* WarpingAdapter = WarpingComp->CreateOwnerAdapter<UMotionWarpingMoverAdapter>();
			WarpingAdapter->SetMoverComp(this);
		}

		// If an InputProducer isn't already set, check if the actor is one
		if (!InputProducer &&
			MyActor->GetClass()->ImplementsInterface(UMoverInputProducerInterface::StaticClass()))
		{
			InputProducer = MyActor;
		}

		if (InputProducer)
		{
			InputProducers.AddUnique(InputProducer);
		}

		TSet<UActorComponent*> Components = MyActor->GetComponents();
		for (UActorComponent* Component : Components)
		{
			if (IsValid(Component) &&
				Component->GetClass()->ImplementsInterface(UMoverInputProducerInterface::StaticClass()))
			{
				InputProducers.AddUnique(Component);
			}
		}
	}
	
	if (!MovementMixer)
	{
		MovementMixer = NewObject<UMovementMixer>(this, TEXT("Default Movement Mixer"));
	}
}

void UMoverComponent::EndPlay(const EEndPlayReason::Type Reason)
{
	Simulation = nullptr;
	BackendLiaisonComp = nullptr;
	SimBlackboard = nullptr;

	Super::EndPlay(Reason);
}

void UMoverComponent::BindProcessGeneratedMovement(FMover_ProcessGeneratedMovement ProcessGeneratedMovementEvent)
{
	ProcessGeneratedMovement = ProcessGeneratedMovementEvent;
}

void UMoverComponent::UnbindProcessGeneratedMovement()
{
	ProcessGeneratedMovement.Clear();
}

void UMoverComponent::ProduceInput(const int32 DeltaTimeMS, FMoverInputCmdContext* Cmd)
{
	Cmd->InputCollection.Empty();
	
	for (TObjectPtr<UObject> InputProducerComponent : InputProducers)
	{
		if (IsValid(InputProducerComponent))
		{
			IMoverInputProducerInterface::Execute_ProduceInput(InputProducerComponent, DeltaTimeMS, IN OUT *Cmd);
		}
	}

	CachedLastProducedInputCmd = *Cmd;
}


void UMoverComponent::FinalizeFrame(const FMoverSyncState* SyncState, const FMoverAuxStateContext* AuxState)
{
	const FMoverDefaultSyncState* MoverState = SyncState->SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();

	// TODO: Revisit this location check -- it seems simplistic now that we have composable state. Consider supporting a version that allows each sync state data struct a chance to react.
	// The component will often be in the "right place" already on FinalizeFrame, so a comparison check makes sense before setting it.
	if (MoverState &&
			(UpdatedComponent->GetComponentLocation().Equals(MoverState->GetLocation_WorldSpace()) == false ||
			 UpdatedComponent->GetComponentQuat().Rotator().Equals(MoverState->GetOrientation_WorldSpace(), ROTATOR_TOLERANCE) == false))
	{
		SetFrameStateFromContext(SyncState, AuxState, /* bDueToRollback= */ false);
	}
	else
	{
		UpdateCachedFrameState(SyncState, AuxState);
	}

	// In the NP and standalone backends, tag state is not accumulated via simulation events during SimulationTick.
	// Instead, tags are derived by walking the committed sync state once per finalized frame, and transition
	// events (add/remove) are then fired based on the delta from the previous frame.
	// (ChaosMover has its own FinalizeFrame with a different signature that calls SetSimulationOutput
	// directly and never reaches this code path.)
	{
		FGameplayTagContainer CollectedTags;
		CollectGameplayTagsFromSyncState(*SyncState, CollectedTags);
		const FGameplayTagContainer PreviousGameplayTags = InternalGameplayTags;
		InternalGameplayTags = CollectedTags;
		RebuildAllGameplayTags();
		DispatchGameplayTagTransitionEvents(PreviousGameplayTags, InternalGameplayTags, {}, CachedLastSimTickTimeStep);
	}

	if (PrimaryVisualComponent)
	{
		if (!PrimaryVisualComponent->GetRelativeTransform().Equals(BaseVisualComponentTransform))
		{
			PrimaryVisualComponent->SetRelativeTransform(BaseVisualComponentTransform);
		}
	}

	if (OnPostFinalize.IsBound())
	{
		OnPostFinalize.Broadcast(MoverSyncStateDoubleBuffer.GetReadable(), CachedLastAuxState);
	}
}

void UMoverComponent::FinalizeUnchangedFrame()
{
	CachedLastSimTickTimeStep.BaseSimTimeMs = BackendLiaisonComp->GetCurrentSimTimeMs();
	CachedLastSimTickTimeStep.ServerFrame = BackendLiaisonComp->GetCurrentSimFrame();

	if (OnPostFinalize.IsBound())
	{
		OnPostFinalize.Broadcast(MoverSyncStateDoubleBuffer.GetReadable(), CachedLastAuxState);
	}
}


void UMoverComponent::FinalizeSmoothingFrame(const FMoverSyncState* SyncState, const FMoverAuxStateContext* AuxState)
{
	if (PrimaryVisualComponent)
	{
		if (SmoothingMode == EMoverSmoothingMode::VisualComponentOffset && (PrimaryVisualComponent != UpdatedComponent))
		{
			// Offset the visual component so it aligns with the smoothed state transform, while leaving the actual root component in place
			if (const FMoverDefaultSyncState* MoverState = SyncState->SyncStateCollection.FindDataByType<FMoverDefaultSyncState>())
			{
				FTransform ActorTransform = FTransform(MoverState->GetOrientation_WorldSpace(), MoverState->GetLocation_WorldSpace(), FVector::OneVector);
				PrimaryVisualComponent->SetWorldTransform(BaseVisualComponentTransform * ActorTransform);	// smoothed location with base offset applied
			}
		}
	}
}

void UMoverComponent::TickInterpolatedSimProxy(const FMoverTimeStep& TimeStep, const FMoverInputCmdContext& InputCmd, UMoverComponent* MoverComp, const FMoverSyncState& CachedSyncState, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState)
{
	if (bSyncInputsForSimProxy)
	{
		CachedLastUsedInputCmd = InputCmd;

		// Copy any structs that may be inputs from sync state to input cmd - note the use of the special container class that lets the inputs avoid causing rollbacks
		if (FMoverInputContainerDataStruct* InputContainer = static_cast<FMoverInputContainerDataStruct*>(SyncState.SyncStateCollection.FindDataByType(FMoverInputContainerDataStruct::StaticStruct())))
		{
			for (auto InputStructIt = InputContainer->InputCollection.GetCollectionDataIterator(); InputStructIt; ++InputStructIt)
			{
				if (const FMoverDataStructBase* InputDataStruct = InputStructIt->Get())
				{
					CachedLastUsedInputCmd.InputCollection.AddDataByCopy(InputDataStruct);
				}
			}
		}
	}

	TArray<TSharedPtr<FMovementModifierBase>> ModifiersToStart;
	TArray<TSharedPtr<FMovementModifierBase>> ModifiersToEnd;

	for (auto ModifierFromSyncStateIt = SyncState.MovementModifiers.GetActiveModifiersIterator(); ModifierFromSyncStateIt; ++ModifierFromSyncStateIt)
	{
		const TSharedPtr<FMovementModifierBase> ModifierFromSyncState = *ModifierFromSyncStateIt;
		
		bool bContainsModifier = false;
		for (auto ModifierFromCacheIt = CachedSyncState.MovementModifiers.GetActiveModifiersIterator(); ModifierFromCacheIt; ++ModifierFromCacheIt)
		{
			const TSharedPtr<FMovementModifierBase> ModifierFromCache = *ModifierFromCacheIt;
			
			if (ModifierFromSyncState->Matches(ModifierFromCache.Get()))
			{
				bContainsModifier = true;
				break;
			}
		}

		if (!bContainsModifier)
		{
			ModifiersToStart.Add(ModifierFromSyncState);
		}
	}

	for (auto ModifierFromCacheIt = CachedSyncState.MovementModifiers.GetActiveModifiersIterator(); ModifierFromCacheIt; ++ModifierFromCacheIt)
	{
		const TSharedPtr<FMovementModifierBase> ModifierFromCache = *ModifierFromCacheIt;
		
		bool bContainsModifier = false;
		for (auto ModifierFromSyncStateIt = SyncState.MovementModifiers.GetActiveModifiersIterator(); ModifierFromSyncStateIt; ++ModifierFromSyncStateIt)
		{
			const TSharedPtr<FMovementModifierBase> ModifierFromSyncState = *ModifierFromSyncStateIt;
			
			if (ModifierFromSyncState->Matches(ModifierFromCache.Get()))
			{
				bContainsModifier = true;
				break;
			}
		}

		if (!bContainsModifier)
		{
			ModifiersToEnd.Add(ModifierFromCache);
		}
	}

	for (TSharedPtr<FMovementModifierBase> Modifier : ModifiersToStart)
	{
		Modifier->GenerateHandle();
		Modifier->OnStart(MoverComp, TimeStep, SyncState, AuxState);
	}

	for (auto ModifierIt = SyncState.MovementModifiers.GetActiveModifiersIterator(); ModifierIt; ++ModifierIt)
	{
		if (ModifierIt->IsValid())
		{
			ModifierIt->Get()->OnPreMovement(this, TimeStep);
			ModifierIt->Get()->OnPostMovement(this, TimeStep, SyncState, AuxState);
		}
	}

	for (TSharedPtr<FMovementModifierBase> Modifier : ModifiersToEnd)
	{
		Modifier->OnEnd(MoverComp, TimeStep, SyncState, AuxState);
	}

	// Emit events when movement mode changes
	if (CachedSyncState.MovementMode != SyncState.MovementMode)
	{
		OnMovementModeChanged.Broadcast(CachedSyncState.MovementMode, SyncState.MovementMode);
	}
}


void UMoverComponent::OnSimulationStateInitialized(const FMoverSyncState& InitialSyncState, const FMoverAuxStateContext& InitialAuxState)
{
	// Our initial state is decided, but we haven't produced any inputs yet so just fill it in with a default
	CachedLastUsedInputCmd = CachedLastProducedInputCmd = FMoverInputCmdContext();
	// TODO: here is where we'd add persistent input cmd struct types once they're supported

	MoverSyncStateDoubleBuffer.SetBufferedData(InitialSyncState);
	LastMoverDefaultSyncState = MoverSyncStateDoubleBuffer.GetReadable().SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	CachedLastAuxState = InitialAuxState;
}


UBaseMovementMode* UMoverComponent::FindMovementMode(TSubclassOf<UBaseMovementMode> MovementMode) const
{
	return FindMode_Mutable(MovementMode);
}

void UMoverComponent::K2_FindMovementModifier(FMovementModifierHandle ModifierHandle, bool& bFoundModifier, int32& TargetAsRawBytes) const
{
	// This will never be called, the exec version below will be hit instead
	checkNoEntry();
}

DEFINE_FUNCTION(UMoverComponent::execK2_FindMovementModifier)
{
	P_GET_STRUCT(FMovementModifierHandle, ModifierHandle);
	P_GET_UBOOL_REF(bFoundModifier);

	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	
	void* ModifierPtr = Stack.MostRecentPropertyAddress;
	FStructProperty* StructProp = CastField<FStructProperty>(Stack.MostRecentProperty);

	P_FINISH;

	bFoundModifier = false;
	
	if (!ModifierPtr)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			LOCTEXT("MoverComponent_FindMovementModifier_UnresolvedTarget", "Failed to resolve the OutMovementModifier for FindMovementModifier")
		);
	
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else if (!StructProp)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			LOCTEXT("MoverComponent_FindMovementModifier_TargetNotStruct", "FindMovementModifier: Target for OutMovementModifier is not a valid type. It must be a Struct and a child of FMovementModifierBase.")
		);
	
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else if (!StructProp->Struct || !StructProp->Struct->IsChildOf(FMovementModifierBase::StaticStruct()))
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			LOCTEXT("MoverComponent_FindMovementModifier_BadType", "FindMovementModifier: Target for OutMovementModifier is not a valid type. Must be a child of FMovementModifierBase.")
		);
	
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else
	{
		P_NATIVE_BEGIN;
		
		if (const FMovementModifierBase* FoundActiveMove = P_THIS->FindMovementModifier(ModifierHandle))
		{
			StructProp->Struct->CopyScriptStruct(ModifierPtr, FoundActiveMove);
			bFoundModifier = true;
		}

		P_NATIVE_END;
	}
}

bool UMoverComponent::IsModifierActiveOrQueued(const FMovementModifierHandle& ModifierHandle) const
{
	return FindMovementModifier(ModifierHandle) ? true : false;
}

const FMovementModifierBase* UMoverComponent::FindMovementModifier(const FMovementModifierHandle& ModifierHandle) const
{
	if (!ModifierHandle.IsValid())
	{
		return nullptr;
	}

	const FMoverSyncState& CachedSyncState = MoverSyncStateDoubleBuffer.GetReadable();
	
	// Check active modifiers for modifier handle
	for (auto ActiveModifierFromSyncStateIt = CachedSyncState.MovementModifiers.GetActiveModifiersIterator(); ActiveModifierFromSyncStateIt; ++ActiveModifierFromSyncStateIt)
	{
		const TSharedPtr<FMovementModifierBase> ActiveModifierFromSyncState = *ActiveModifierFromSyncStateIt;

		if (ModifierHandle == ActiveModifierFromSyncState->GetHandle())
		{
			return ActiveModifierFromSyncState.Get();
		}
	}

	// Check queued modifiers for modifier handle
	for (auto QueuedModifierFromSyncStateIt = CachedSyncState.MovementModifiers.GetQueuedModifiersIterator(); QueuedModifierFromSyncStateIt; ++QueuedModifierFromSyncStateIt)
	{
		const TSharedPtr<FMovementModifierBase> QueuedModifierFromSyncState = *QueuedModifierFromSyncStateIt;

		if (ModifierHandle == QueuedModifierFromSyncState->GetHandle())
		{
			return QueuedModifierFromSyncState.Get();
		}
	}
	
	if (Simulation)
	{
		return Simulation->FindQueuedModifier(ModifierHandle);
	}
	
	return nullptr;
}

const FMovementModifierBase* UMoverComponent::FindMovementModifierByType(const UScriptStruct* DataStructType) const
{


	const FMoverSyncState& CachedSyncState = MoverSyncStateDoubleBuffer.GetReadable();
	
	// Check active modifiers for modifier handle
	for (auto ActiveModifierFromSyncStateIt = CachedSyncState.MovementModifiers.GetActiveModifiersIterator(); ActiveModifierFromSyncStateIt; ++ActiveModifierFromSyncStateIt)
	{
		const TSharedPtr<FMovementModifierBase> ActiveModifierFromSyncState = *ActiveModifierFromSyncStateIt;

		if (ActiveModifierFromSyncState->GetScriptStruct()->IsChildOf(DataStructType))
		{
			return ActiveModifierFromSyncState.Get();
		}
	}

	// Check queued modifiers for modifier handle
	for (auto QueuedModifierFromSyncStateIt = CachedSyncState.MovementModifiers.GetQueuedModifiersIterator(); QueuedModifierFromSyncStateIt; ++QueuedModifierFromSyncStateIt)
	{
		const TSharedPtr<FMovementModifierBase> QueuedModifierFromSyncState = *QueuedModifierFromSyncStateIt;

		if (QueuedModifierFromSyncState->GetScriptStruct()->IsChildOf(DataStructType))
		{
			return QueuedModifierFromSyncState.Get();
		}
	}
	
	if (Simulation)
	{
		return Simulation->FindQueuedModifierByType(DataStructType);
	}

	return nullptr;
}

bool UMoverComponent::HasGameplayTag(FGameplayTag TagToFind, bool bExactMatch) const
{
	if ((BackendLiaisonComp && BackendLiaisonComp->IsAsync()) || MoverComponentCVars::UseSnapshotGameplayTagsForSyncBackend != 0)
	{
		return bExactMatch ? AllGameplayTags.HasTagExact(TagToFind) : AllGameplayTags.HasTag(TagToFind);
	}

	return HasGameplayTagInState(MoverSyncStateDoubleBuffer.GetReadable(), TagToFind, bExactMatch);
}

bool UMoverComponent::HasGameplayTagInState(const FMoverSyncState& SyncState, FGameplayTag TagToFind, bool bExactMatch) const 
{
	// Check loose / external tags
	if (bExactMatch)
	{
		if (ExternalGameplayTags.HasTagExact(TagToFind))
		{
			return true;
		}
	}
	else
	{
		if (ExternalGameplayTags.HasTag(TagToFind))
		{
			return true;
		}
	}

	// Check active Movement Mode
	if (const UBaseMovementMode* ActiveMovementMode = FindMovementModeByName(SyncState.MovementMode))
	{
		if (ActiveMovementMode->HasGameplayTag(TagToFind, bExactMatch))
		{
			return true;
		}
	}

	// Search Movement Modifiers
	for (auto ModifierFromSyncStateIt = SyncState.MovementModifiers.GetActiveModifiersIterator(); ModifierFromSyncStateIt; ++ModifierFromSyncStateIt)
	{
		if (const TSharedPtr<FMovementModifierBase> ModifierFromSyncState = *ModifierFromSyncStateIt)
		{
			if (ModifierFromSyncState.IsValid() && ModifierFromSyncState->HasGameplayTag(TagToFind, bExactMatch))
			{
				return true;
			}
		}
	}

	// Search Layered Moves
	for (const TSharedPtr<FLayeredMoveBase>& LayeredMove : SyncState.LayeredMoves.GetActiveMoves())
	{
		if (LayeredMove->HasGameplayTag(TagToFind, bExactMatch))
		{
			return true;
		}
	}

	return false;
}


void UMoverComponent::AddGameplayTag(FGameplayTag TagToAdd)
{
	if (!ExternalGameplayTags.HasTagExact(TagToAdd))
	{
		ExternalGameplayTags.AddTag(TagToAdd);
		RebuildAllGameplayTags();

		if (InternalGameplayTags.HasTagExact(TagToAdd))
		{
			UE_LOGF(LogMover, Verbose, "External gameplay tag '%ls' added but tag is already active via the simulation - event suppressed to avoid a duplicate added event.", *TagToAdd.ToString());
		}
		else
		{
			FMoverEventContext EventContext;
			if (BackendLiaisonComp)
			{
				EventContext.EventTimeMs = BackendLiaisonComp->GetCurrentSimTimeMs();
				EventContext.ServerFrame = BackendLiaisonComp->GetCurrentSimFrame();
			}

			OnGameplayTagAdded.Broadcast(TagToAdd, EventContext, /* bIsExternalTag = */ true);
		}
	}
}

void UMoverComponent::AddGameplayTags(const FGameplayTagContainer& TagsToAdd)
{
	for (const FGameplayTag& Tag : TagsToAdd)
	{
		AddGameplayTag(Tag);
	}
}

void UMoverComponent::RemoveGameplayTag(FGameplayTag TagToRemove)
{
	if (ExternalGameplayTags.HasTagExact(TagToRemove))
	{
		ExternalGameplayTags.RemoveTag(TagToRemove);
		RebuildAllGameplayTags();

		if (InternalGameplayTags.HasTagExact(TagToRemove))
		{
			UE_LOGF(LogMover, Verbose, "External gameplay tag '%ls' removed but tag is still active via the simulation - event suppressed to avoid a duplicate removed event.", *TagToRemove.ToString());
		}
		else
		{
			FMoverEventContext EventContext;
			if (BackendLiaisonComp)
			{
				EventContext.EventTimeMs = BackendLiaisonComp->GetCurrentSimTimeMs();
				EventContext.ServerFrame = BackendLiaisonComp->GetCurrentSimFrame();
			}

			OnGameplayTagRemoved.Broadcast(TagToRemove, EventContext, /* bIsExternalTag = */ true);
		}
	}
}

void UMoverComponent::RemoveGameplayTags(const FGameplayTagContainer& TagsToRemove)
{
	for (const FGameplayTag& Tag : TagsToRemove)
	{
		RemoveGameplayTag(Tag);
	}
}

UObject* UMoverComponent::GetMovementOwner() const
{
	return GetOwner();
}

UObject* UMoverComponent::GetUpdatedObject() const
{
	return UpdatedComponent;
}

FTransform UMoverComponent::GetTransform() const
{
	if (LastMoverDefaultSyncState)
	{
		return LastMoverDefaultSyncState->GetTransform_WorldSpace();
	}

	return FTransform::Identity;
}

void UMoverComponent::RequestStopMovement()
{
	// Todo when stop movement function is implemented
}

void UMoverComponent::OnPreSimulate(const FMoverTimeStep& TimeStep, const FMoverTickStartData& StartingData)
{
	// Temporarily disabled for standalone async support
	//ensureMsgf(IsInGameThread(), TEXT("%hs must only be called on the game thread"), __FUNCTION__);

	// Send mover info to the Chaos Visual Debugger (this will do nothing if CVD is not recording, or the mover info data channel not enabled)
	UE::MoverUtils::FMoverCVDRuntimeTrace::TraceMoverData(this, &StartingData.InputCmd, &StartingData.SyncState);

	if (OnPreSimulationTick.IsBound())
	{
		OnPreSimulationTick.Broadcast(TimeStep, StartingData.InputCmd);
	}

	FlushPendingMoveRegistrations();
}

void UMoverComponent::FlushPendingMoveRegistrations()
{
	for (const TSubclassOf<ULayeredMoveLogic>& PendingRegistrantClass : MovesPendingRegistration)
	{
		TObjectPtr<ULayeredMoveLogic> RegisteredMove = NewObject<ULayeredMoveLogic>(this, PendingRegistrantClass);
		RegisteredMoves.Add(RegisteredMove);
	}

	for (const TSubclassOf<ULayeredMoveLogic>& PendingUnregistrantClass : MovesPendingUnregistration)
	{
		RegisteredMoves.RemoveAll([&PendingUnregistrantClass, this]
			(const TObjectPtr<ULayeredMoveLogic>& MoveLogic)
			{
				if (MoveLogic->GetClass() == PendingUnregistrantClass)
				{
					return true;
				}

				return false;
			});
	}

	MovesPendingRegistration.Empty();
	MovesPendingUnregistration.Empty();

}

void UMoverComponent::OnPostSimulate(const FMoverTimeStep& TimeStep, const FMoverTickStartData& StartData, FMoverTickEndData& EndData)
{
	// Temporarily disabled for standalone async support
	//ensureMsgf(IsInGameThread(), TEXT("%hs must only be called on the game thread"), __FUNCTION__);

	if (OnPostMovement.IsBound())
	{
		// Event allowing final changes to the simulation state (uncommon to use)
		OnPostMovement.Broadcast(TimeStep, EndData.SyncState, EndData.AuxState);
	}

	if (bSyncInputsForSimProxy)
	{
		// stow all inputs away in a special container struct that avoids causing potential rollbacks 
		// so they can be available to other clients even if they're only interpolated sim proxies
		if (FMoverInputContainerDataStruct* InputContainer = static_cast<FMoverInputContainerDataStruct*>(EndData.SyncState.SyncStateCollection.FindOrAddDataByType(FMoverInputContainerDataStruct::StaticStruct())))
		{
			for (auto InputCmdIt = StartData.InputCmd.InputCollection.GetCollectionDataIterator(); InputCmdIt; ++InputCmdIt)
			{
				if (InputCmdIt->Get())
				{
					InputContainer->InputCollection.AddDataByCopy(InputCmdIt->Get());
				}
			}
		}
	}


	if (MoverComponentCVars::WarnOnPostSimDifference)
	{
		if (FMoverDefaultSyncState* OutputSyncState = EndData.SyncState.SyncStateCollection.FindMutableDataByType<FMoverDefaultSyncState>())
		{
			if (UpdatedComponent->GetComponentLocation().Equals(OutputSyncState->GetLocation_WorldSpace()) == false ||
				UpdatedComponent->GetComponentQuat().Equals(OutputSyncState->GetOrientation_WorldSpace().Quaternion(), UE_KINDA_SMALL_NUMBER) == false)
			{
				UE_LOGF(LogMover, Warning, "Detected pos/rot difference between Mover actor (%ls) sync state and scene component after sim ticking. This indicates a movement mode may not be authoring the final state correctly.", *GetNameSafe(UpdatedComponent->GetOwner()));
			}
		}
	}

	CachedLastUsedInputCmd = StartData.InputCmd;
	CachedLastSimTickTimeStep = TimeStep;

	if (TimeStep.ServerFrame > CachedNewestSimTickTimeStep.ServerFrame || TimeStep.BaseSimTimeMs > CachedNewestSimTickTimeStep.BaseSimTimeMs)
	{
		CachedNewestSimTickTimeStep = TimeStep;
	}

	OnPostSimulationTick.Broadcast(TimeStep);

	if (bSupportsKinematicBasedMovement)
	{
		UpdateBasedMovementScheduling(EndData);
	}
}

void UMoverComponent::UpdateCachedFrameState(const FMoverSyncState* SyncState, const FMoverAuxStateContext* AuxState)
{
	// TODO integrate dirty tracking
	FMoverSyncState& BufferedSyncState = MoverSyncStateDoubleBuffer.GetWritable();
	BufferedSyncState = *SyncState;
	LastMoverDefaultSyncState = BufferedSyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	MoverSyncStateDoubleBuffer.Flip();

	// TODO: when AuxState starts getting used we need to double buffer it here as well
	CachedLastAuxState = *AuxState;
	CachedLastSimTickTimeStep.BaseSimTimeMs = BackendLiaisonComp->GetCurrentSimTimeMs();
	CachedLastSimTickTimeStep.ServerFrame = BackendLiaisonComp->GetCurrentSimFrame();
}

void UMoverComponent::SetFrameStateFromContext(const FMoverSyncState* SyncState, const FMoverAuxStateContext* AuxState, bool bDueToRollback)
{
	UpdateCachedFrameState(SyncState, AuxState);

	if (FMoverDefaultSyncState* MoverState = const_cast<FMoverDefaultSyncState*>(LastMoverDefaultSyncState))
	{
		const bool bShouldRebaseBasedState = bDueToRollback;

		if (bShouldRebaseBasedState && MoverState->GetMovementBase())
		{
			// Note that this is modifying our cached mover state from what we received from Network Prediction. We are resampling
			// the transform of the movement base, in case it has changed as well during the rollback.
			MoverState->UpdateCurrentMovementBase();
		}

		// The state's properties are usually worldspace already, but may need to be adjusted to match the current movement base
		const FVector WorldLocation = MoverState->GetLocation_WorldSpace();
		const FRotator WorldOrientation = MoverState->GetOrientation_WorldSpace();
		const FVector WorldVelocity = MoverState->GetVelocity_WorldSpace();

		// Apply the desired transform to the scene component

		// For rollbacks, we want any physics objects attached to us to come along for the ride without affecting their local state
		ETeleportType TeleportType = bDueToRollback ? ETeleportType::TeleportPhysics : ETeleportType::None;

		// If we can, then we can utilize grouped movement updates to reduce the number of calls to SendPhysicsTransform
		if (IsUsingDeferredGroupMovement())
		{
			// Signal to the USceneComponent that we are moving that this should be in a grouped update
			// and not apply changes on the worker thread immediately
			FScopedMovementUpdate MovementUpdate(
				UpdatedComponent,
				EScopedUpdate::DeferredGroupUpdates,
				/*bRequireOverlapsEventFlagToQueueOverlaps*/ true);

			FTransform Transform(WorldOrientation, WorldLocation, UpdatedComponent->GetComponentTransform().GetScale3D());
			UpdatedComponent->SetWorldTransform(Transform, /*bSweep*/false, nullptr, TeleportType);
			UpdatedComponent->ComponentVelocity = WorldVelocity;
		}
		else
		{
			FTransform Transform(WorldOrientation, WorldLocation, UpdatedComponent->GetComponentTransform().GetScale3D());
			UpdatedComponent->SetWorldTransform(Transform, /*bSweep*/false, nullptr, TeleportType);
			UpdatedComponent->ComponentVelocity = WorldVelocity;
		}
	}
}


void UMoverComponent::GetDefaultInputAndState(FMoverInputCmdContext& OutInputCmd, FMoverSyncState& OutSyncState, FMoverAuxStateContext& OutAuxState) const
{
	OutInputCmd = FMoverInputCmdContext();
	// TODO: here is where we'd add persistent input cmd struct types once they're supported

	OutSyncState = FMoverSyncState();

	// Add all initial persistent sync state types
	for (const FMoverDataPersistence& PersistentSyncEntry : PersistentSyncStateDataTypes)
	{
		if (PersistentSyncEntry.RequiredType.Get()) // This can happen if a previously existing required type was removed, causing a crash
		{
			OutSyncState.SyncStateCollection.FindOrAddDataByType(PersistentSyncEntry.RequiredType);
		}
	}

	// Mirror the scene component transform if we have one, otherwise it will be left at origin
	FMoverDefaultSyncState* MoverState = OutSyncState.SyncStateCollection.FindMutableDataByType<FMoverDefaultSyncState>();
	if (MoverState && UpdatedComponent)
	{
		MoverState->SetTransforms_WorldSpace(
			UpdatedComponent->GetComponentLocation(),
			UpdatedComponent->GetComponentRotation(),
			UpdatedComponent->GetComponentVelocity(),
			FVector::ZeroVector);
	}

	OutSyncState.MovementMode = StartingMovementMode;
	
	OutAuxState = FMoverAuxStateContext();

}

void UMoverComponent::SetSimulation(UMoverSimulation& InSimulation)
{
	Simulation = &InSimulation;
	check(Simulation->RollbackBlackboard != nullptr);
	
	SimBlackboard = InSimulation.GetBlackboard_Mutable();

	RollbackBlackboardExternalWrapper = FRollbackBlackboardExternalWrapper(*Simulation->RollbackBlackboard);
}

void UMoverComponent::HandleImpact(FMoverOnImpactParams& ImpactParams)
{
	OnHandleImpact(ImpactParams);
}

void UMoverComponent::OnHandleImpact(const FMoverOnImpactParams& ImpactParams)
{
	// TODO: Handle physics impacts here - ie when player runs into box, impart force onto box
}

void UMoverComponent::UpdateBasedMovementScheduling(const FMoverTickEndData& SimOutput)
{
	// If we have a dynamic movement base, enable later based movement tick
	UPrimitiveComponent* SyncStateDynamicBase = nullptr;
	if (const FMoverDefaultSyncState* OutputSyncState = SimOutput.SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>())
	{
		if (UBasedMovementUtils::IsADynamicBase(OutputSyncState->GetMovementBase()))
		{
			SyncStateDynamicBase = OutputSyncState->GetMovementBase();
		}
	}

	// Remove any stale dependency
	if (MovementBaseDependency && (MovementBaseDependency != SyncStateDynamicBase))
	{
		UBasedMovementUtils::RemoveTickDependency(BasedMovementTickFunction, MovementBaseDependency);
		MovementBaseDependency = nullptr;

		// Denote that there will no longer be based movement changes, since we no longer have a base
		RollbackBlackboardExternalWrapper.TrySet(CommonBlackboard::AccumulatedBasedTransformDelta, FTransform::Identity);
		OnBasedMovementApplied.Broadcast(FTransform::Identity, GetLastTimeStep());
	}

	// Set up current dependencies
	if (SyncStateDynamicBase)
	{
		BasedMovementTickFunction.SetTickFunctionEnable(true);

		if (UBasedMovementUtils::IsBaseSimulatingPhysics(SyncStateDynamicBase))
		{
			BasedMovementTickFunction.TickGroup = TG_PostPhysics;
		}
		else
		{
			BasedMovementTickFunction.TickGroup = TG_PrePhysics;
		}

		if (MovementBaseDependency == nullptr)
		{
			UBasedMovementUtils::AddTickDependency(BasedMovementTickFunction, SyncStateDynamicBase);
			MovementBaseDependency = SyncStateDynamicBase;
		}
	}
	else
	{
		BasedMovementTickFunction.SetTickFunctionEnable(false);
		MovementBaseDependency = nullptr;

		SimBlackboard->Invalidate(CommonBlackboard::LastFoundDynamicMovementBase);
	}
}


void UMoverComponent::FindDefaultUpdatedComponent()
{
	if (!IsValid(UpdatedComponent))
	{
		USceneComponent* NewUpdatedComponent = nullptr;

		const AActor* MyActor = GetOwner();
		const UWorld* MyWorld = GetWorld();

		if (MyActor && MyWorld && MyWorld->IsGameWorld())
		{
			NewUpdatedComponent = MyActor->GetRootComponent();
		}

		SetUpdatedComponent(NewUpdatedComponent);
	}
}


void UMoverComponent::FindAndRegisterDefaults()
{
	FindDefaultUpdatedComponent();

	if (UMoverNetworkPredictionLiaisonComponent* NppBackend = Cast<UMoverNetworkPredictionLiaisonComponent>(BackendLiaisonComp.GetObject()))
	{
		NppBackend->ResetSimulation();
	}
}

void UMoverComponent::UpdateTickRegistration()
{
	const bool bHasUpdatedComponent = (UpdatedComponent != NULL);
	SetComponentTickEnabled(bHasUpdatedComponent && bAutoActivate);
}

void UMoverComponent::OnSimulationRollback(const FMoverSyncState* SyncState, const FMoverAuxStateContext* AuxState, const FMoverTimeStep& NewBaseTimeStep)
{
	SetFrameStateFromContext(SyncState, AuxState, /* bDueToRollback? */ true);


	// Mover Gameplay Tag Logging
	const FString DebugActorName = FString::Printf(TEXT("%s[%s]"), *GetNameSafe(GetOwner()), GetOwner() && GetOwner()->GetWorld() ? *GetOwner()->GetWorld()->GetOutermost()->GetName() : TEXT("null"));
	const FString DebugActorRole = GetOwner() ? UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()) : FString(TEXT("null"));
	MOVER_TAG_LOG(LogMover, "[GT:RollbackNotify] Actor=%s Role=%s Frame=%d IsResim=%d IsRollback=%d | FromFrame=%d FromTimeMs=%.3f",
		*DebugActorName, *DebugActorRole, NewBaseTimeStep.ServerFrame, (int32)NewBaseTimeStep.bIsResimulating, (int32)true,
		CachedLastSimTickTimeStep.ServerFrame, CachedLastSimTickTimeStep.BaseSimTimeMs);

	OnPostSimulationRollback.Broadcast(NewBaseTimeStep, CachedLastSimTickTimeStep);
}


#if WITH_EDITOR

void UMoverComponent::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	RefreshSharedSettings();
}

void UMoverComponent::PostCDOCompiled(const FPostCDOCompiledContext& Context)
{
	Super::PostCDOCompiled(Context);

	RefreshSharedSettings();
}


void UMoverComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if ((PropertyChangedEvent.Property) && (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UMoverComponent, MovementModes)))
	{
		RefreshSharedSettings();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}


void UMoverComponent::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);

	if ((TransactionEvent.GetEventType() == ETransactionObjectEventType::Finalized || TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo) &&
		TransactionEvent.HasPropertyChanges() &&
		TransactionEvent.GetChangedProperties().Contains(GET_MEMBER_NAME_CHECKED(UMoverComponent, MovementModes)))
	{
		RefreshSharedSettings();		
	}
}


EDataValidationResult UMoverComponent::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = Super::IsDataValid(Context);

	if (!ValidateSetup(Context))
	{
		Result = EDataValidationResult::Invalid;
	}

	return Result;
}


bool UMoverComponent::ValidateSetup(FDataValidationContext& Context) const
{
	bool bHasMatchingStartingMode = false;
	bool bDidFindAnyProblems = false;
	bool bIsAsyncBackend = false;

	// Verify backend liaison
	if (!BackendClass)
	{
		Context.AddError(FText::Format(LOCTEXT("MissingBackendClassError", "No BackendClass property specified on {0}. Mover actor will not function."),
			FText::FromString(GetNameSafe(GetOwner()))));

		bDidFindAnyProblems = true;
	}
	else if (!BackendClass->ImplementsInterface(UMoverBackendLiaisonInterface::StaticClass()))
	{
		Context.AddError(FText::Format(LOCTEXT("InvalidBackendClassError", "BackendClass {0} on {1} does not implement IMoverBackendLiaisonInterface. Mover actor will not function."),
			FText::FromString(BackendClass->GetName()),
			FText::FromString(GetNameSafe(GetOwner()))));

		bDidFindAnyProblems = true;
	}
	else
	{
		IMoverBackendLiaisonInterface* BackendCDOAsInterface = Cast<IMoverBackendLiaisonInterface>(BackendClass->GetDefaultObject());
		if (BackendCDOAsInterface)
		{
			bIsAsyncBackend = BackendCDOAsInterface->IsAsync();
			if (BackendCDOAsInterface->ValidateData(Context, *this) == EDataValidationResult::Invalid)
			{
				bDidFindAnyProblems = true;
			}
		}
	}

	// Verify all movement modes
	for (const TPair<FName, TObjectPtr<UBaseMovementMode>>& Element : MovementModes)
	{
		if (StartingMovementMode == Element.Key)
		{
			bHasMatchingStartingMode = true;
		}

		// Verify movement mode is valid
		if (!Element.Value)
		{
			Context.AddError(FText::Format(LOCTEXT("InvalidMovementModeError", "Invalid movement mode on {0}, mapped as {1}. Mover actor will not function."),
				FText::FromString(GetNameSafe(GetOwner())),
				FText::FromName(Element.Key)));

			bDidFindAnyProblems = true;
		}
		else if (Element.Value->IsDataValid(Context) == EDataValidationResult::Invalid)
		{
			bDidFindAnyProblems = true;
		}

		// Verify that the movement mode's shared settings object exists (if any)
		if (Element.Value)
		{
			if (bIsAsyncBackend && !Element.Value->bSupportsAsync)
			{
				Context.AddError(FText::Format(LOCTEXT("InvalidModeAsyncSupportsError", "Movement mode on {0}, mapped as {1} does not support asynchrony but its backend is asynchronous"),
						FText::FromString(GetNameSafe(GetOwner())),
						FText::FromName(Element.Key)));

				bDidFindAnyProblems = true;
			}

			for (TSubclassOf<UObject>& Type : Element.Value->SharedSettingsClasses)
			{
				if (Type.Get() == nullptr)
				{
					Context.AddError(FText::Format(LOCTEXT("InvalidModeSettingsError", "Movement mode on {0}, mapped as {1}, has an invalid SharedSettingsClass. You may need to remove the invalid settings class."),
						FText::FromString(GetNameSafe(GetOwner())),
						FText::FromName(Element.Key)));

					bDidFindAnyProblems = true;
				}
				else if (FindSharedSettings(Type) == nullptr)
				{
					Context.AddError(FText::Format(LOCTEXT("MissingModeSettingsError", "Movement mode on {0}, mapped as {1}, is missing its desired SharedSettingsClass {2}. You may need to save the asset and/or recompile."),
						FText::FromString(GetNameSafe(GetOwner())),
						FText::FromName(Element.Key),
						FText::FromString(Type->GetName())));

					bDidFindAnyProblems = true;
				}
			}

			for (const UBaseMovementModeTransition* Transition : Element.Value->Transitions)
			{
				if (!IsValid(Transition))
				{
					continue;
				}

				if (bIsAsyncBackend && !Transition->bSupportsAsync)
				{
					Context.AddError(FText::Format(LOCTEXT("InvalidModeTransitionAsyncSupportError", "Transition on mode {0} on {1} does not support asynchrony but its backend is asynchronous"),
						FText::FromName(Element.Key),
						FText::FromString(GetNameSafe(GetOwner()))));

					bDidFindAnyProblems = true;
				}

				for (const TSubclassOf<UObject>& Type : Transition->SharedSettingsClasses)
				{
					if (Type.Get() == nullptr)
					{
						Context.AddError(FText::Format(LOCTEXT("InvalidModeTransitionSettingsError", "Transition on mode {0} on {1}, has an invalid SharedSettingsClass. You may need to remove the invalid settings class."),
							FText::FromName(Element.Key),
							FText::FromString(GetNameSafe(GetOwner()))));

						bDidFindAnyProblems = true;
					}
					else if (FindSharedSettings(Type) == nullptr)
					{
						Context.AddError(FText::Format(LOCTEXT("MissingModeTransitionSettingsError", "Transition on mode {0} on {1}, is missing its desired SharedSettingsClass {2}. You may need to save the asset and/or recompile."),
							FText::FromName(Element.Key),
							FText::FromString(GetNameSafe(GetOwner())),
							FText::FromString(Type->GetName())));

						bDidFindAnyProblems = true;
					}
				}
			}
		}
	}

	// Verify we have a matching starting mode
	if (!bHasMatchingStartingMode && StartingMovementMode != NAME_None)
	{
		Context.AddError(FText::Format(LOCTEXT("InvalidStartingModeError", "Invalid StartingMovementMode {0} specified on {1}. Mover actor will not function."),
			FText::FromName(StartingMovementMode),
			FText::FromString(GetNameSafe(GetOwner()))));

		bDidFindAnyProblems = true;
	}

	// Verify transitions
	for (const UBaseMovementModeTransition* Transition : Transitions)
	{
		if (!IsValid(Transition))
		{
			Context.AddError(FText::Format(LOCTEXT("InvalidTransitionError", "Invalid or missing transition object on {0}. Clean up the Transitions array."),
				FText::FromString(GetNameSafe(GetOwner()))));

			bDidFindAnyProblems = true;
			continue;
		}

		for (const TSubclassOf<UObject>& Type : Transition->SharedSettingsClasses)
		{
			if (Type.Get() == nullptr)
			{
				Context.AddError(FText::Format(LOCTEXT("InvalidTransitionSettingsError", "Transition on {0}, has an invalid SharedSettingsClass. You may need to remove the invalid settings class."),
					FText::FromString(GetNameSafe(GetOwner()))));

				bDidFindAnyProblems = true;
			}
			else if (FindSharedSettings(Type) == nullptr)
			{
				Context.AddError(FText::Format(LOCTEXT("MissingTransitionSettingsError", "Transition on {0}, is missing its desired SharedSettingsClass {2}. You may need to save the asset and/or recompile."),
					FText::FromString(GetNameSafe(GetOwner())),
					FText::FromString(Type->GetName())));

				bDidFindAnyProblems = true;
			}
		}
	}

	// Verify persistent types
	for (const FMoverDataPersistence& PersistentSyncEntry : PersistentSyncStateDataTypes)
	{
		if (!PersistentSyncEntry.RequiredType || !PersistentSyncEntry.RequiredType->IsChildOf(FMoverDataStructBase::StaticStruct()))
		{
			Context.AddError(FText::Format(LOCTEXT("InvalidSyncStateTypeError", "RequiredType '{0}' is not a valid type or is missing. Must be a child of FMoverDataStructBase."),
				FText::FromString(GetNameSafe(PersistentSyncEntry.RequiredType))));

			bDidFindAnyProblems = true;
		}
	}

	// Verify that the up direction override is a normalized vector
	if (bHasUpDirectionOverride)
	{
		if (!UpDirectionOverride.IsNormalized())
		{
			Context.AddError(FText::Format(LOCTEXT("InvalidUpDirectionOverrideError", "UpDirectionOverride {0} needs to be a normalized vector, but it is not. {1}"),
				FText::FromString(UpDirectionOverride.ToString()),
				FText::FromString(GetNameSafe(GetOwner()))));

			bDidFindAnyProblems = true;
		}
	}

	return !bDidFindAnyProblems;
}

TArray<FString> UMoverComponent::GetStartingMovementModeNames()
{
	TArray<FString> PossibleModeNames;

	PossibleModeNames.Add(TEXT(""));

	for (const TPair<FName, TObjectPtr<UBaseMovementMode>>& Element : MovementModes)
	{
		FString ModeNameAsString;
		Element.Key.ToString(ModeNameAsString);
		PossibleModeNames.Add(ModeNameAsString);
	}

	return PossibleModeNames;
}

#endif // WITH_EDITOR


void UMoverComponent::PhysicsVolumeChanged(APhysicsVolume* NewVolume)
{
	// This itself feels bad. When will this be called? Its impossible to know what is allowed and not allowed to be done in this callback.
	// Callbacks instead should be trapped within the simulation update function. This isn't really possible though since the UpdateComponent
	// is the one that will call this.
}


void UMoverComponent::RefreshSharedSettings()
{
	TArray<TObjectPtr<UObject>> UnreferencedSettingsObjs = SharedSettings;

	// Add any missing settings
	for (const TPair<FName, TObjectPtr<UBaseMovementMode>>& Element : MovementModes)
	{
		if (UBaseMovementMode* Mode = Element.Value.Get())
		{
			for (TSubclassOf<UObject>& SharedSettingsType : Mode->SharedSettingsClasses)
			{
				if (SharedSettingsType.Get() == nullptr)
				{
					UE_LOGF(LogMover, Warning, "Invalid shared setting class detected on Movement Mode %ls.", *Mode->GetName());
					continue;
				}

				bool bFoundMatchingClass = false;
				for (const TObjectPtr<UObject>& SettingsObj : SharedSettings)
				{
					if (SettingsObj && SettingsObj->IsA(SharedSettingsType))
					{
						bFoundMatchingClass = true;
						UnreferencedSettingsObjs.Remove(SettingsObj);
						break;
					}
				}

				if (!bFoundMatchingClass)
				{
					UObject* NewSettings = NewObject<UObject>(this, SharedSettingsType, NAME_None, GetMaskedFlags(RF_PropagateToSubObjects) | RF_Transactional);
					SharedSettings.Add(NewSettings);
				}
			}

			for (const UBaseMovementModeTransition* Transition : Mode->Transitions)
			{
				if (!IsValid(Transition))
				{
					continue;
				}

				for (const TSubclassOf<UObject>& SharedSettingsType : Transition->SharedSettingsClasses)
				{
					if (SharedSettingsType.Get() == nullptr)
					{
						UE_LOGF(LogMover, Warning, "Invalid shared setting class detected on Transition on Movement Mode %ls.", *Mode->GetName());
						continue;
					}

					bool bFoundMatchingClass = false;
					for (const TObjectPtr<UObject>& SettingsObj : SharedSettings)
					{
						if (SettingsObj && SettingsObj->IsA(SharedSettingsType))
						{
							bFoundMatchingClass = true;
							UnreferencedSettingsObjs.Remove(SettingsObj);
							break;
						}
					}

					if (!bFoundMatchingClass)
					{
						UObject* NewSettings = NewObject<UObject>(this, SharedSettingsType, NAME_None, GetMaskedFlags(RF_PropagateToSubObjects) | RF_Transactional);
						SharedSettings.Add(NewSettings);
					}
				}
			}
		}
	}

	for (const UBaseMovementModeTransition* Transition : Transitions)
	{
		if (!IsValid(Transition))
		{
			continue;
		}

		for (const TSubclassOf<UObject>& SharedSettingsType : Transition->SharedSettingsClasses)
		{
			if (SharedSettingsType.Get() == nullptr)
			{
				UE_LOGF(LogMover, Warning, "Invalid shared setting class detected on Transition.");
				continue;
			}

			bool bFoundMatchingClass = false;
			for (const TObjectPtr<UObject>& SettingsObj : SharedSettings)
			{
				if (SettingsObj && SettingsObj->IsA(SharedSettingsType))
				{
					bFoundMatchingClass = true;
					UnreferencedSettingsObjs.Remove(SettingsObj);
					break;
				}
			}

			if (!bFoundMatchingClass)
			{
				UObject* NewSettings = NewObject<UObject>(this, SharedSettingsType, NAME_None, GetMaskedFlags(RF_PropagateToSubObjects) | RF_Transactional);
				SharedSettings.Add(NewSettings);
			}
		}
	}

	// Remove any settings that are no longer used
	for (const TObjectPtr<UObject>& SettingsObjToRemove : UnreferencedSettingsObjs)
	{
		SharedSettings.Remove(SettingsObjToRemove);
	}

	// Sort by name for array order consistency
	Algo::Sort(SharedSettings, [](const TObjectPtr<UObject>& LHS, const TObjectPtr<UObject>& RHS) 
		{ return (LHS->GetClass()->GetPathName() < RHS.GetClass()->GetPathName()); });
}


const TArray<TObjectPtr<ULayeredMoveLogic>>* UMoverComponent::GetRegisteredMoves() const
{
	return &RegisteredMoves;
}

void UMoverComponent::K2_RegisterMove(TSubclassOf<ULayeredMoveLogic> MoveClass)
{
	MovesPendingUnregistration.Remove(MoveClass);
	if (!MovesPendingRegistration.Contains(MoveClass))
	{
		const bool bAlreadyRegistered = RegisteredMoves.ContainsByPredicate([MoveClass](const TObjectPtr<ULayeredMoveLogic>& Move) { return Move->GetClass() == MoveClass; });
		if (!bAlreadyRegistered)
		{
			MovesPendingRegistration.AddUnique(MoveClass);
		}
	}
}

void UMoverComponent::K2_RegisterMoves(TArray<TSubclassOf<ULayeredMoveLogic>> MoveClasses)
{
	for (const TSubclassOf<ULayeredMoveLogic>& MoveClass : MoveClasses)
	{
		K2_RegisterMove(MoveClass);
	}
}

void UMoverComponent::K2_UnregisterMove(TSubclassOf<ULayeredMoveLogic> MoveClass)
{
	MovesPendingRegistration.Remove(MoveClass);
	if (!MovesPendingUnregistration.Contains(MoveClass))
	{
		const bool bAlreadyUnregistered = RegisteredMoves.ContainsByPredicate([MoveClass](const TObjectPtr<ULayeredMoveLogic>& Move) { return Move->GetClass() == MoveClass; });
		if (!bAlreadyUnregistered)
		{
			MovesPendingUnregistration.AddUnique(MoveClass);
		}
	}
}

bool UMoverComponent::K2_QueueLayeredMoveActivationWithContext(TSubclassOf<ULayeredMoveLogic> MoveLogicClass, const int32& MoveAsRawData)
{
	// This will never be called, the exec version below will be hit instead
	checkNoEntry();
	return false;
}

DEFINE_FUNCTION(UMoverComponent::execK2_QueueLayeredMoveActivationWithContext)
{
	P_GET_OBJECT(UClass, MoveLogicClass);
	
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(nullptr);

	const FStructProperty* MoveActivationProperty = CastField<FStructProperty>(Stack.MostRecentProperty);
	uint8* MoveActivationPtr = Stack.MostRecentPropertyAddress;
	
	P_FINISH
	
	P_NATIVE_BEGIN;

	// TODO NS: throw some helpful warnings of what wasn't valid
	const bool bHasValidActivationStructProp = MoveActivationProperty && MoveActivationProperty->Struct && MoveActivationProperty->Struct->IsChildOf(FLayeredMoveActivationParams::StaticStruct());

	bool bHasValidMoveData = MoveLogicClass && bHasValidActivationStructProp; 
	if (bHasValidMoveData)
	{
		const FLayeredMoveActivationParams* MoveActivationContext = reinterpret_cast<FLayeredMoveActivationParams*>(MoveActivationPtr);
		bHasValidMoveData = P_THIS->MakeAndQueueLayeredMove(MoveLogicClass, MoveActivationContext);
	}
	
	*(bool*)RESULT_PARAM = bHasValidMoveData;
	
	P_NATIVE_END;
}

bool UMoverComponent::QueueLayeredMoveActivation(TSubclassOf<ULayeredMoveLogic> MoveLogicClass)
{
	return MakeAndQueueLayeredMove(MoveLogicClass, nullptr);
}

void UMoverComponent::K2_QueueLayeredMove(const int32& MoveAsRawData)
{
	// This will never be called, the exec version below will be hit instead
	checkNoEntry();
}

DEFINE_FUNCTION(UMoverComponent::execK2_QueueLayeredMove)
{
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	void* MovePtr = Stack.MostRecentPropertyAddress;
	FStructProperty* StructProp = CastField<FStructProperty>(Stack.MostRecentProperty);

	P_FINISH;

	P_NATIVE_BEGIN;

	const bool bHasValidStructProp = StructProp && StructProp->Struct && StructProp->Struct->IsChildOf(FLayeredMoveBase::StaticStruct());

	if (ensureMsgf((bHasValidStructProp && MovePtr), TEXT("An invalid type (%s) was sent to a QueueLayeredMove node. A struct derived from FLayeredMoveBase is required. No layered move will be queued."),
		StructProp ? *GetNameSafe(StructProp->Struct) : *Stack.MostRecentProperty->GetClass()->GetName()))
	{
		// Could we steal this instead of cloning? (move semantics)
		FLayeredMoveBase* MoveAsBasePtr = reinterpret_cast<FLayeredMoveBase*>(MovePtr);
		FLayeredMoveBase* ClonedMove = MoveAsBasePtr->Clone();

		P_THIS->QueueLayeredMove(TSharedPtr<FLayeredMoveBase>(ClonedMove));
	}

	P_NATIVE_END;
}

void UMoverComponent::ScheduleLayeredMove(TSharedPtr<FLayeredMoveBase> LayeredMove)
{
	ensureMsgf(IsInGameThread(), TEXT("UMoverComponent::ScheduleLayeredMove should only be called from the game thread. Inspect code for incorrect calls."));
	FMoverTimeStep TimeStep;
	if (ensureMsgf(BackendLiaisonComp, TEXT("UMoverComponent::ScheduleLayeredMove was unexpectedly called with a null backend liaison component. The move will be ignored.")))
	{
		if (ensure(LayeredMove.IsValid()))
		{
			TimeStep.BaseSimTimeMs = BackendLiaisonComp->GetCurrentSimTimeMs();
			TimeStep.ServerFrame = BackendLiaisonComp->GetCurrentSimFrame();
			// TimeStep.StepMs is not used by UE::Mover::MakeScheduledMove so we don't set it here
			QueueScheduledLayeredMove(UE::Mover::MakeScheduledMove(BackendLiaisonComp, TimeStep, LayeredMove));
		}
	}
}

void UMoverComponent::QueueLayeredMove(TSharedPtr<FLayeredMoveBase> LayeredMove)
{
	// Temporarily disabled for standalone async support
	//ensureMsgf(IsInGameThread(), TEXT("UMoverComponent::QueueLayeredMove(TSharedPtr<FLayeredMoveBase>) should only be called from the game thread. Inspect code for incorrect calls."));

	FMoverTimeStep TimeStep;
	if (ensureMsgf(BackendLiaisonComp, TEXT("UMoverComponent::QueueLayeredMove was unexpectedly called with a null backend liaison component. The move will be ignored.")))
	{
		if (ensure(LayeredMove.IsValid()))
		{
			TimeStep.BaseSimTimeMs = BackendLiaisonComp->GetCurrentSimTimeMs();
			TimeStep.ServerFrame = BackendLiaisonComp->GetCurrentSimFrame();
			// TimeStep.StepMs is not used by UE::Mover::ScheduleMove so we don't set it here
			QueueScheduledLayeredMove(UE::Mover::MakeScheduledMoveForNextFrame(BackendLiaisonComp, TimeStep, LayeredMove));
		}
	}
}

void UMoverComponent::QueueScheduledLayeredMove(const FScheduledLayeredMove& ScheduledLayeredMove)
{
	if (Simulation)
	{
		Simulation->QueueLayeredMove(ScheduledLayeredMove);
	}

#if !UE_BUILD_SHIPPING
	ENetMode NetMode = GetWorld() ? GetWorld()->GetNetMode() : NM_MAX;
	UE_LOGF(LogMover, Verbose, "(%ls) UMoverComponent::QueueScheduledLayeredMove: %ls thread queueing a layered move scheduled at [%ls]: %ls.",
		*ToString(NetMode), IsInGameThread() ? TEXT("Game") : TEXT("Physics"), *ScheduledLayeredMove.SchedulingInfo.ToString(), ScheduledLayeredMove.Move.IsValid() ? *ScheduledLayeredMove.Move->ToSimpleString() : TEXT("INVALID MOVE"));
#endif // !UE_BUILD_SHIPPING
}

FMovementModifierHandle UMoverComponent::K2_QueueMovementModifier(const int32& MoveAsRawData)
{
	// This will never be called, the exec version below will be hit instead
	checkNoEntry();
	return 0;
}

DEFINE_FUNCTION(UMoverComponent::execK2_QueueMovementModifier)
{
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	void* MovePtr = Stack.MostRecentPropertyAddress;
	FStructProperty* StructProp = CastField<FStructProperty>(Stack.MostRecentProperty);

	P_FINISH;

	P_NATIVE_BEGIN;

	const bool bHasValidStructProp = StructProp && StructProp->Struct && StructProp->Struct->IsChildOf(FMovementModifierBase::StaticStruct());

	if (ensureMsgf((bHasValidStructProp && MovePtr), TEXT("An invalid type (%s) was sent to a QueueMovementModifier node. A struct derived from FMovementModifierBase is required. No modifier will be queued."),
		StructProp ? *GetNameSafe(StructProp->Struct) : *Stack.MostRecentProperty->GetClass()->GetName()))
	{
		// Could we steal this instead of cloning? (move semantics)
		FMovementModifierBase* MoveAsBasePtr = reinterpret_cast<FMovementModifierBase*>(MovePtr);
		FMovementModifierBase* ClonedMove = MoveAsBasePtr->Clone();

		FMovementModifierHandle ModifierID = P_THIS->QueueMovementModifier(TSharedPtr<FMovementModifierBase>(ClonedMove));
		*static_cast<FMovementModifierHandle*>(RESULT_PARAM) = ModifierID;
	}

	P_NATIVE_END;
}

FMovementModifierHandle UMoverComponent::QueueMovementModifier(TSharedPtr<FMovementModifierBase> Modifier)
{
	if (Simulation)
	{
		return Simulation->QueueMovementModifier(Modifier);
	}

	return FMovementModifierHandle(MODIFIER_INVALID_HANDLE);
}

void UMoverComponent::CancelModifierFromHandle(FMovementModifierHandle ModifierHandle)
{
	if (Simulation)
	{
		Simulation->CancelModifierFromHandle(ModifierHandle);
	}
}

void UMoverComponent::CancelFeaturesWithTag(FGameplayTag TagToCancel, bool bRequireExactMatch)
{
	if (Simulation)
	{
		Simulation->CancelFeaturesWithTag(TagToCancel, bRequireExactMatch);
	}
}

bool UMoverComponent::HasFeaturesWithTag(FGameplayTag TagToMatch, bool bRequireExactMatch) const
{
	if (Simulation)
	{
		return Simulation->HasFeaturesWithTag(TagToMatch, bRequireExactMatch);
	}

	return false;
}

void UMoverComponent::K2_QueueInstantMovementEffect(const int32& EffectAsRawData)
{
	// This will never be called, the exec version below will be hit instead
	checkNoEntry();
}

DEFINE_FUNCTION(UMoverComponent::execK2_QueueInstantMovementEffect)
{
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	void* EffectPtr = Stack.MostRecentPropertyAddress;
	FStructProperty* StructProp = CastField<FStructProperty>(Stack.MostRecentProperty);

	P_FINISH;

	P_NATIVE_BEGIN;

	const bool bHasValidStructProp = StructProp && StructProp->Struct && StructProp->Struct->IsChildOf(FInstantMovementEffect::StaticStruct());

	if (ensureMsgf((bHasValidStructProp && EffectPtr), TEXT("An invalid type (%s) was sent to a QueueInstantMovementEffect node. A struct derived from FInstantMovementEffect is required. No Movement Effect will be queued."),
		StructProp ? *GetNameSafe(StructProp->Struct) : *Stack.MostRecentProperty->GetClass()->GetName()))
	{
		// Could we steal this instead of cloning? (move semantics)
		FInstantMovementEffect* EffectAsBasePtr = reinterpret_cast<FInstantMovementEffect*>(EffectPtr);
		FInstantMovementEffect* ClonedMove = EffectAsBasePtr->Clone();

		P_THIS->QueueInstantMovementEffect(TSharedPtr<FInstantMovementEffect>(ClonedMove));
	}

	P_NATIVE_END;
}

void UMoverComponent::K2_ScheduleInstantMovementEffect(const int32& EffectAsRawData)
{
	// This will never be called, the exec version below will be hit instead
	checkNoEntry();
}

DEFINE_FUNCTION(UMoverComponent::execK2_ScheduleInstantMovementEffect)
{
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	void* EffectPtr = Stack.MostRecentPropertyAddress;
	FStructProperty* StructProp = CastField<FStructProperty>(Stack.MostRecentProperty);

	P_FINISH;

	P_NATIVE_BEGIN;

	const bool bHasValidStructProp = StructProp && StructProp->Struct && StructProp->Struct->IsChildOf(FInstantMovementEffect::StaticStruct());

	if (ensureMsgf((bHasValidStructProp && EffectPtr), TEXT("An invalid type (%s) was sent to a QueueInstantMovementEffect node. A struct derived from FInstantMovementEffect is required. No Movement Effect will be queued."),
		StructProp ? *GetNameSafe(StructProp->Struct) : *Stack.MostRecentProperty->GetClass()->GetName()))
	{
		// Could we steal this instead of cloning? (move semantics)
		FInstantMovementEffect* EffectAsBasePtr = reinterpret_cast<FInstantMovementEffect*>(EffectPtr);
		FInstantMovementEffect* ClonedMove = EffectAsBasePtr->Clone();

		P_THIS->ScheduleInstantMovementEffect(TSharedPtr<FInstantMovementEffect>(ClonedMove));
	}

	P_NATIVE_END;
}

void UMoverComponent::ScheduleInstantMovementEffect(TSharedPtr<FInstantMovementEffect> InstantMovementEffect)
{
	ensureMsgf(IsInGameThread(), TEXT("UMoverComponent::ScheduleInstantMovementEffect should only be called from the game thread. Inspect code for incorrect calls."));
	FMoverTimeStep TimeStep;
	if (ensureMsgf(BackendLiaisonComp, TEXT("UMoverComponent::ScheduleInstantMovementEffect was unexpectedly called with a null backend liaison component. The instant movement effect will be ignored.")))
	{
		if (ensure(InstantMovementEffect.IsValid()))
		{
			TimeStep.BaseSimTimeMs = BackendLiaisonComp->GetCurrentSimTimeMs();
			TimeStep.ServerFrame = BackendLiaisonComp->GetCurrentSimFrame();
			// TimeStep.StepMs is not used by UE::Mover::MakeScheduledEffect so we don't set it here
			QueueScheduledInstantMovementEffect(UE::Mover::MakeScheduledEffect(BackendLiaisonComp, TimeStep, InstantMovementEffect));
		}
	}
}

void UMoverComponent::QueueInstantMovementEffect_Internal(const FMoverTimeStep& TimeStep, TSharedPtr<FInstantMovementEffect> InstantMovementEffect)
{
	if (ensureMsgf(BackendLiaisonComp, TEXT("UMoverComponent::QueueInstantMovementEffect_Internal was unexpectedly called with a null backend liaison component. The instant movement effect will be ignored.")))
	{
		QueueScheduledInstantMovementEffect(UE::Mover::MakeScheduledEffect(BackendLiaisonComp, TimeStep, InstantMovementEffect));
	}
}

void UMoverComponent::QueueInstantMovementEffect(TSharedPtr<FInstantMovementEffect> InstantMovementEffect)
{
	ensureMsgf(IsInGameThread(), TEXT("UMoverComponent::QueueInstantMovementEffect(TSharedPtr<FInstantMovementEffect>) should only be called from the game thread. Inspect code for incorrect calls."));
	FMoverTimeStep TimeStep;
	if (ensureMsgf(BackendLiaisonComp, TEXT("UMoverComponent::QueueInstantMovementEffect was unexpectedly called with a null backend liaison component. The instant movement effect will be ignored.")))
	{
		if (ensure(InstantMovementEffect.IsValid()))
		{
			TimeStep.BaseSimTimeMs = BackendLiaisonComp->GetCurrentSimTimeMs();
			TimeStep.ServerFrame = BackendLiaisonComp->GetCurrentSimFrame();
			// TimeStep.StepMs is not used by UE::Mover::MakeScheduledEffect so we don't set it here
			QueueScheduledInstantMovementEffect(UE::Mover::MakeScheduledEffectForNextFrame(BackendLiaisonComp, TimeStep, InstantMovementEffect));
		}
	}
}

void UMoverComponent::QueueScheduledInstantMovementEffect(const FScheduledInstantMovementEffect& InstantMovementEffect)
{
	if (Simulation)
	{
		Simulation->QueueInstantMovementEffect(InstantMovementEffect);
	}

#if !UE_BUILD_SHIPPING
	ENetMode NetMode = GetWorld() ? GetWorld()->GetNetMode() : NM_MAX;
	UE_LOGF(LogMover, Verbose, "(%ls) UMoverComponent::QueueScheduledInstantMovementEffect: %ls thread queueing an instant movement effect scheduled at [%ls]: %ls.",
		*ToString(NetMode), IsInGameThread() ? TEXT("Game") : TEXT("Physics"), *InstantMovementEffect.SchedulingInfo.ToString(), InstantMovementEffect.Effect.IsValid() ? *InstantMovementEffect.Effect->ToSimpleString() : TEXT("INVALID INSTANT EFFECT"));
#endif // !UE_BUILD_SHIPPING
}

UBaseMovementMode* UMoverComponent::FindMovementModeByName(FName MovementModeName) const
{
	if (const TObjectPtr<UBaseMovementMode>* FoundMode = MovementModes.Find(MovementModeName))
	{
		return *FoundMode;
	}
	return nullptr;
}

void UMoverComponent::K2_FindActiveLayeredMove(bool& DidSucceed, int32& TargetAsRawBytes) const
{
	// This will never be called, the exec version below will be hit instead
	checkNoEntry();
}

DEFINE_FUNCTION(UMoverComponent::execK2_FindActiveLayeredMove)
{
	P_GET_UBOOL_REF(DidSucceed);

	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	
	void* MovePtr = Stack.MostRecentPropertyAddress;
	FStructProperty* StructProp = CastField<FStructProperty>(Stack.MostRecentProperty);

	P_FINISH;

	DidSucceed = false;
	
	if (!MovePtr)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			LOCTEXT("MoverComponent_GetActiveLayeredMove_UnresolvedTarget", "Failed to resolve the OutLayeredMove for GetActiveLayeredMove")
		);

		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else if (!StructProp)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			LOCTEXT("MoverComponent_GetActiveLayeredMove_TargetNotStruct", "GetActiveLayeredMove: Target for OutLayeredMove is not a valid type. It must be a Struct and a child of FLayeredMoveBase.")
		);

		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else if (!StructProp->Struct || !StructProp->Struct->IsChildOf(FLayeredMoveBase::StaticStruct()))
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			LOCTEXT("MoverComponent_GetActiveLayeredMove_BadType", "GetActiveLayeredMove: Target for OutLayeredMove is not a valid type. Must be a child of FLayeredMoveBase.")
		);

		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else
	{
		P_NATIVE_BEGIN;
		
		if (const FLayeredMoveBase* FoundActiveMove = P_THIS->FindActiveLayeredMoveByType(StructProp->Struct))
		{
			StructProp->Struct->CopyScriptStruct(MovePtr, FoundActiveMove);
			DidSucceed = true;
		}

		P_NATIVE_END;
	}
}

const FLayeredMoveBase* UMoverComponent::FindActiveLayeredMoveByType(const UScriptStruct* LayeredMoveStructType) const
{
	const FMoverSyncState& CachedSyncState = MoverSyncStateDoubleBuffer.GetReadable();
	return CachedSyncState.LayeredMoves.FindActiveMove(LayeredMoveStructType);
}

void UMoverComponent::QueueNextMode(FName DesiredModeName, bool bShouldReenter)
{
	DoQueueNextMode(DesiredModeName, bShouldReenter);
}

void UMoverComponent::DoQueueNextMode(FName DesiredModeName, bool bShouldReenter)
{
	if (HasBegunPlay() && Simulation)
	{
		Simulation->QueueNextMode({ DesiredModeName, bShouldReenter });
	}
}

UBaseMovementMode* UMoverComponent::AddMovementModeFromClass(FName ModeName, TSubclassOf<UBaseMovementMode> MovementMode)
{
	if (!MovementMode)
	{
		UE_LOGF(LogMover, Warning, "Attempted to add a movement mode that wasn't valid. AddMovementModeFromClass will not add anything. (%ls)", *GetNameSafe(GetOwner()));
		return nullptr;
	}
	if (MovementMode->HasAnyClassFlags(CLASS_Abstract))
	{
		UE_LOGF(LogMover, Warning, "The Movement Mode class (%ls) is abstract and is not a valid class to instantiate. AddMovementModeFromClass will not do anything. (%ls)", *GetNameSafe(MovementMode), *GetNameSafe(GetOwner()));
		return nullptr;
	}

	TObjectPtr<UBaseMovementMode> AddedMovementMode =  NewObject<UBaseMovementMode>(this, MovementMode);
	return AddMovementModeFromObject(ModeName, AddedMovementMode) ? AddedMovementMode : nullptr;
}

bool UMoverComponent::AddMovementModeFromObject(FName ModeName, UBaseMovementMode* MovementMode)
{
	if (MovementMode)
	{
		if (MovementMode->GetClass()->HasAnyClassFlags(CLASS_Abstract))
		{
			UE_LOGF(LogMover, Warning, "The Movement Mode class (%ls) is abstract and is not a valid class to instantiate. AddMovementModeFromObject will not do anything. (%ls)", *GetNameSafe(MovementMode), *GetNameSafe(GetOwner()));
			return false;
		}
		
		if (TObjectPtr<UBaseMovementMode>* FoundMovementMode = MovementModes.Find(ModeName))
		{
			if (FoundMovementMode->Get()->GetClass() == MovementMode->GetClass())
			{
				UE_LOGF(LogMover, Warning, "Added the same movement mode (%ls) for a movement mode name (%ls). AddMovementModeFromObject will add the mode but is likely unwanted/unnecessary behavior. (%ls)", *GetNameSafe(MovementMode), *ModeName.ToString(), *GetNameSafe(GetOwner()));
			}

			RemoveMovementMode(ModeName);
		}
		
		if (MovementMode->GetOuter() != this)
		{
			UE_LOGF(LogMover, Verbose, "Movement modes are expected to be parented to the MoverComponent. The %ls movement mode was reparented to %ls! (%ls)", *GetNameSafe(MovementMode), *GetNameSafe(this), *GetNameSafe(GetOwner()));
			MovementMode->Rename(nullptr, this, REN_DoNotDirty | REN_NonTransactional);
		}
		
		MovementModes.Add(ModeName, MovementMode);

		if (Simulation)
		{
			Simulation->RegisterMovementMode({ ModeName, MovementMode, false });
		}
	}
	else
	{
		UE_LOGF(LogMover, Warning, "Attempted to add %ls movement mode that wasn't valid to %ls. AddMovementModeFromObject did not add anything. (%ls)", *GetNameSafe(MovementMode), *GetNameSafe(this), *GetNameSafe(GetOwner()));
		return false;
	}

	return true;
}

bool UMoverComponent::RemoveMovementMode(FName ModeName)
{
	TObjectPtr<UBaseMovementMode>* ModeToRemove = MovementModes.Find(ModeName);
	const bool ModeRemoved = MovementModes.Remove(ModeName) > 0;
	if (ModeRemoved && ModeToRemove)
	{
		if (Simulation)
		{
			Simulation->UnregisterMovementMode({ ModeName });
		}

		ModeToRemove->Get()->ConditionalBeginDestroy();
	}
	
	return ModeRemoved; 
}


/** Converts localspace root motion to a specific alternate worldspace location, taking the relative transform of the localspace component into account. */
static FTransform ConvertLocalRootMotionToAltWorldSpace(const FTransform& LocalRootMotionTransform, const FTransform& AltWorldspaceTransform, const USceneComponent& RelativeComp)
{
	const FTransform TrueActorToWorld = RelativeComp.GetOwner()->GetTransform();
	const FTransform RelativeCompToActor = TrueActorToWorld.GetRelativeTransform(RelativeComp.GetComponentTransform());

	const FTransform AltComponentWorldTransform = RelativeCompToActor.Inverse() * AltWorldspaceTransform;

	const FTransform NewComponentToWorld = LocalRootMotionTransform * AltComponentWorldTransform;
	const FTransform NewActorTransform = RelativeCompToActor * NewComponentToWorld;

	FTransform ActorDeltaTransform = NewActorTransform.GetRelativeTransform(AltWorldspaceTransform);
	
	return FTransform(ActorDeltaTransform.GetRotation(), NewActorTransform.GetTranslation() - AltWorldspaceTransform.GetTranslation());
}

FTransform UMoverComponent::ConvertLocalRootMotionToWorld(const FTransform& LocalRootMotionTransform, float DeltaSeconds, const FTransform* AlternateActorToWorld, const FMotionWarpingUpdateContext* OptionalWarpingContext) const
{
	// Optionally process/warp localspace root motion
	const FTransform ProcessedLocalRootMotion = ProcessLocalRootMotionDelegate.IsBound()
		? ProcessLocalRootMotionDelegate.Execute(LocalRootMotionTransform, DeltaSeconds, OptionalWarpingContext)
		: LocalRootMotionTransform;

	// Convert processed localspace root motion to worldspace
	FTransform WorldSpaceRootMotion;

	if (USkeletalMeshComponent* SkeletalMesh = GetPrimaryVisualComponent<USkeletalMeshComponent>())
	{
		if (AlternateActorToWorld)
		{
			WorldSpaceRootMotion = ConvertLocalRootMotionToAltWorldSpace(ProcessedLocalRootMotion, *AlternateActorToWorld, *SkeletalMesh);
		}
		else
		{
			WorldSpaceRootMotion = SkeletalMesh->ConvertLocalRootMotionToWorld(ProcessedLocalRootMotion);
		}
	}
	else
	{
		const FTransform PresentationActorToWorldTransform = AlternateActorToWorld ? *AlternateActorToWorld : GetOwner()->GetTransform();
		const FVector DeltaWorldTranslation = ProcessedLocalRootMotion.GetTranslation() - PresentationActorToWorldTransform.GetTranslation();

		const FQuat NewWorldRotation = PresentationActorToWorldTransform.GetRotation() * ProcessedLocalRootMotion.GetRotation();
		const FQuat DeltaWorldRotation = NewWorldRotation * PresentationActorToWorldTransform.GetRotation().Inverse();

		WorldSpaceRootMotion.SetComponents(DeltaWorldRotation, DeltaWorldTranslation, FVector::OneVector);
	}

	// Optionally process/warp worldspace root motion
	return ProcessWorldRootMotionDelegate.IsBound()
		? ProcessWorldRootMotionDelegate.Execute(WorldSpaceRootMotion, DeltaSeconds, OptionalWarpingContext)
		: WorldSpaceRootMotion;
}


FTransform UMoverComponent::GetUpdatedComponentTransform() const
{
	if (UpdatedComponent)
	{
		return UpdatedComponent->GetComponentTransform();
	}
	return FTransform::Identity;
}


void UMoverComponent::SetUpdatedComponent(USceneComponent* NewUpdatedComponent)
{
	// Remove delegates from old component
	if (UpdatedComponent)
	{
		UpdatedComponent->SetShouldUpdatePhysicsVolume(false);
		UpdatedComponent->SetPhysicsVolume(nullptr, true);
		UpdatedComponent->PhysicsVolumeChangedDelegate.RemoveDynamic(this, &UMoverComponent::PhysicsVolumeChanged);

		// remove from tick prerequisite
		UpdatedComponent->PrimaryComponentTick.RemovePrerequisite(this, PrimaryComponentTick);
	}

	if (UpdatedCompAsPrimitive)
	{
		UpdatedCompAsPrimitive->OnComponentBeginOverlap.RemoveDynamic(this, &UMoverComponent::OnBeginOverlap);
	}

	// Don't assign pending kill components, but allow those to null out previous UpdatedComponent.
	UpdatedComponent = GetValid(NewUpdatedComponent);
	UpdatedCompAsPrimitive = Cast<UPrimitiveComponent>(UpdatedComponent);

	// Assign delegates
	if (IsValid(UpdatedComponent))
	{
		UpdatedComponent->SetShouldUpdatePhysicsVolume(true);
		UpdatedComponent->PhysicsVolumeChangedDelegate.AddUniqueDynamic(this, &UMoverComponent::PhysicsVolumeChanged);

		if (!bInOnRegister && !bInInitializeComponent)
		{
			// UpdateOverlaps() in component registration will take care of this.
			UpdatedComponent->UpdatePhysicsVolume(true);
		}

		// force ticks after movement component updates
		UpdatedComponent->PrimaryComponentTick.AddPrerequisite(this, PrimaryComponentTick);
	}

	if (IsValid(UpdatedCompAsPrimitive))
	{
		UpdatedCompAsPrimitive->OnComponentBeginOverlap.AddDynamic(this, &UMoverComponent::OnBeginOverlap);
	}

	UpdateTickRegistration();
}


USceneComponent* UMoverComponent::GetUpdatedComponent() const
{
	return UpdatedComponent.Get();
}

USceneComponent* UMoverComponent::GetPrimaryVisualComponent() const
{
	return PrimaryVisualComponent.Get();
}

void UMoverComponent::SetPrimaryVisualComponent(USceneComponent* SceneComponent)
{
	if (SceneComponent && 
		ensureMsgf(SceneComponent->GetOwner() == GetOwner(), TEXT("Primary visual component must be owned by the same actor. MoverComp owner: %s  VisualComp owner: %s"), *GetNameSafe(GetOwner()), *GetNameSafe(SceneComponent->GetOwner())))
	{
		PrimaryVisualComponent = SceneComponent;
		BaseVisualComponentTransform = SceneComponent->GetRelativeTransform();
	}
	else
	{
		PrimaryVisualComponent = nullptr;
		BaseVisualComponentTransform = FTransform::Identity;
	}
}

FVector UMoverComponent::GetVelocity() const
{ 
	if (LastMoverDefaultSyncState)
	{
		return LastMoverDefaultSyncState->GetVelocity_WorldSpace();
	}

	return FVector::ZeroVector;
}


FVector UMoverComponent::GetMovementIntent() const
{ 
	if (LastMoverDefaultSyncState)
	{
		return LastMoverDefaultSyncState->GetIntent_WorldSpace();
	}

	return FVector::ZeroVector; 
}


FRotator UMoverComponent::GetTargetOrientation() const
{
	// Prefer the input's intended orientation, but if it can't be determined, assume it matches the actual orientation
	const FMoverInputCmdContext& LastInputCmd = GetLastInputCmd();
	if (const FCharacterDefaultInputs* MoverInputs = LastInputCmd.InputCollection.FindDataByType<FCharacterDefaultInputs>())
	{
		const FVector TargetOrientationDir = MoverInputs->GetOrientationIntentDir_WorldSpace();

		if (!TargetOrientationDir.IsNearlyZero())
		{
			return TargetOrientationDir.ToOrientationRotator();
		}
	}
	
	if (LastMoverDefaultSyncState)
	{
		return LastMoverDefaultSyncState->GetOrientation_WorldSpace();
	}

	return GetOwner() ? GetOwner()->GetActorRotation() : FRotator::ZeroRotator;
}


void UMoverComponent::SetGravityOverride(bool bOverrideGravity, FVector NewGravityAcceleration)
{
	bHasGravityOverride = bOverrideGravity;
	GravityAccelOverride = NewGravityAcceleration;
	
	WorldToGravityTransform = FQuat::FindBetweenNormals(FVector::UpVector, -GravityAccelOverride.GetSafeNormal());
	GravityToWorldTransform = WorldToGravityTransform.Inverse();
}


FVector UMoverComponent::GetGravityAcceleration() const
{
	if (bHasGravityOverride)
	{
		return GravityAccelOverride;
	}

	if (UpdatedComponent)
	{
		APhysicsVolume* CurPhysVolume = UpdatedComponent->GetPhysicsVolume();
		if (CurPhysVolume)
		{
			return CurPhysVolume->GetGravityZ() * FVector::UpVector;
		}
	}

	return MoverComponentConstants::DefaultGravityAccel;
}

void UMoverComponent::SetUpDirectionOverride(bool bOverrideUpDirection, FVector UpDirection)
{
	bHasUpDirectionOverride = bOverrideUpDirection;
	if (bOverrideUpDirection)
	{
		if (UpDirection.IsNearlyZero())
		{
			UE_LOGF(LogMover, Warning, "Ignoring the provided UpDirection (%ls) override because it is a zero vector. (%ls)", *UpDirection.ToString(), *GetNameSafe(GetOwner()));
			bHasUpDirectionOverride = false;
			return;
		}
		UpDirectionOverride = UpDirection.GetSafeNormal();
	}
}

FVector UMoverComponent::GetUpDirection() const
{
	// Use the up direction override if enabled
	if (bHasUpDirectionOverride)
	{
		return UpDirectionOverride;
	}
	
	return UMovementUtils::DeduceUpDirectionFromGravity(GetGravityAcceleration());
}

const FPlanarConstraint& UMoverComponent::GetPlanarConstraint() const
{
	return PlanarConstraint;
}

void UMoverComponent::SetPlanarConstraint(const FPlanarConstraint& InConstraint)
{
	PlanarConstraint = InConstraint;
}

void UMoverComponent::SetBaseVisualComponentTransform(const FTransform& ComponentTransform)
{
	BaseVisualComponentTransform = ComponentTransform;
}

FTransform UMoverComponent::GetBaseVisualComponentTransform() const
{
	return BaseVisualComponentTransform;
}

void UMoverComponent::SetUseDeferredGroupMovement(bool bEnable)
{
	bUseDeferredGroupMovement = bEnable;

	// TODO update any necessary dependencies as needed
}

bool UMoverComponent::IsUsingDeferredGroupMovement() const
{
	return bUseDeferredGroupMovement && USceneComponent::IsGroupedComponentMovementEnabled();
}

TArray<FTrajectorySampleInfo> UMoverComponent::GetFutureTrajectory(float FutureSeconds, float SamplesPerSecond)
{
	FMoverPredictTrajectoryParams PredictionParams;
	PredictionParams.NumPredictionSamples = FMath::Max(1, FutureSeconds * SamplesPerSecond);
	PredictionParams.SecondsPerSample = FutureSeconds / (float)PredictionParams.NumPredictionSamples;

	return GetPredictedTrajectory(PredictionParams);
}

TArray<FTrajectorySampleInfo> UMoverComponent::GetPredictedTrajectory(FMoverPredictTrajectoryParams PredictionParams)
{
	if (Simulation)
	{
		FMoverSimContext SimContext;
		SimContext.Simulation = Simulation;
		SimContext.Blackboard = Simulation->GetRollbackBlackboardPredictionWrapper();
		SimContext.SimulationOwner = this;

		FMoverTickStartData StepState;

		// Use the last-known input if none are specified.
		if (PredictionParams.OptionalInputCmds.IsEmpty())
		{
			StepState.InputCmd = GetLastInputCmd();
		}

		// Use preferred starting sync/aux state. Fall back to last-known state if not set.
		if (PredictionParams.OptionalStartSyncState.IsSet())
		{
			StepState.SyncState = PredictionParams.OptionalStartSyncState.GetValue();
		}
		else
		{
			StepState.SyncState = MoverSyncStateDoubleBuffer.GetReadable();
		}

		if (PredictionParams.OptionalStartAuxState.IsSet())
		{
			StepState.AuxState = PredictionParams.OptionalStartAuxState.GetValue();
		}
		else
		{
			StepState.AuxState = CachedLastAuxState;
		}


		FMoverTimeStep FutureTimeStep;
		FutureTimeStep.StepMs = (PredictionParams.SecondsPerSample * 1000.f);

		if (PredictionParams.OptionalStartTime.IsSet())
		{
			FutureTimeStep.BaseSimTimeMs = FMath::Max(0.0, PredictionParams.OptionalStartTime.GetValue().TimeMs);
			FutureTimeStep.ServerFrame   = FMath::Max(0, PredictionParams.OptionalStartTime.GetValue().FrameCount);
		}
		else
		{
			FutureTimeStep.BaseSimTimeMs = FMath::Max(0.0, CachedLastSimTickTimeStep.BaseSimTimeMs);
			FutureTimeStep.ServerFrame   = FMath::Max(0, CachedLastSimTickTimeStep.ServerFrame);
		}

		if (const UBaseMovementMode* CurrentMovementMode = GetMovementMode())
		{
			if (FMoverDefaultSyncState* StepSyncState = StepState.SyncState.SyncStateCollection.FindMutableDataByType<FMoverDefaultSyncState>())
			{
				const bool bOrigHasGravityOverride = bHasGravityOverride;
				const FVector OrigGravityAccelOverride = GravityAccelOverride;

				if (PredictionParams.bDisableGravity)
				{
					SetGravityOverride(true, FVector::ZeroVector);
				}

				TArray<FTrajectorySampleInfo> OutSamples;
				OutSamples.SetNumUninitialized(PredictionParams.NumPredictionSamples);

				FVector PriorLocation = StepSyncState->GetLocation_WorldSpace();
				FRotator PriorOrientation = StepSyncState->GetOrientation_WorldSpace();
				FVector PriorVelocity = StepSyncState->GetVelocity_WorldSpace();

				for (int32 i = 0; i < PredictionParams.NumPredictionSamples; ++i)
				{
					Simulation->RollbackBlackboard->BeginPredictionFrame(FutureTimeStep);

					// If no further inputs are specified, the previous input cmd will continue to be used
					if (i < PredictionParams.OptionalInputCmds.Num())
					{
						StepState.InputCmd = PredictionParams.OptionalInputCmds[i];
					}

					// Capture sample from current step state
					FTrajectorySampleInfo& Sample = OutSamples[i];

					Sample.Transform.SetTranslationAndScale3D(StepSyncState->GetLocation_WorldSpace(), FVector::OneVector);
					Sample.Transform.SetRotation(StepSyncState->GetOrientation_WorldSpace().Quaternion());
					Sample.LinearVelocity = StepSyncState->GetVelocity_WorldSpace();
					Sample.InstantaneousAcceleration = (StepSyncState->GetVelocity_WorldSpace() - PriorVelocity) / PredictionParams.SecondsPerSample;
					Sample.AngularVelocity = (StepSyncState->GetOrientation_WorldSpace() - PriorOrientation) * (1.f / PredictionParams.SecondsPerSample);

					Sample.SimTimeMs = FutureTimeStep.BaseSimTimeMs;

					// Cache prior values
					PriorLocation = StepSyncState->GetLocation_WorldSpace();
					PriorOrientation = StepSyncState->GetOrientation_WorldSpace();
					PriorVelocity = StepSyncState->GetVelocity_WorldSpace();

					// Generate next move from current step state
					FProposedMove StepMove;
					CurrentMovementMode->GenerateMove(SimContext, StepState, FutureTimeStep, StepMove);

					// Advance state based on move
					StepSyncState->SetTransforms_WorldSpace(StepSyncState->GetLocation_WorldSpace() + (StepMove.LinearVelocity * PredictionParams.SecondsPerSample),
						UMovementUtils::ApplyAngularVelocityToRotator(StepSyncState->GetOrientation_WorldSpace(),StepMove.AngularVelocityDegrees, PredictionParams.SecondsPerSample),
						StepMove.LinearVelocity,
						StepMove.AngularVelocityDegrees,
						StepSyncState->GetMovementBase(),
						StepSyncState->GetMovementBaseBoneName());

					FutureTimeStep.BaseSimTimeMs += FutureTimeStep.StepMs;
					++FutureTimeStep.ServerFrame;
				}

				// Put sample locations at visual root location if requested
				if (PredictionParams.bUseVisualComponentRoot)
				{
					if (const USceneComponent* VisualComp = GetPrimaryVisualComponent())
					{
						const FVector VisualCompOffset = VisualComp->GetRelativeLocation();
						const FTransform VisualCompRelativeTransform = VisualComp->GetRelativeTransform();

						for (int32 i=0; i < PredictionParams.NumPredictionSamples; ++i)
						{
							OutSamples[i].Transform = VisualCompRelativeTransform * OutSamples[i].Transform;
						}
					}
				}
				
				if (PredictionParams.bDisableGravity)
				{
					SetGravityOverride(bOrigHasGravityOverride, OrigGravityAccelOverride);
				}

				Simulation->RollbackBlackboard->EndPrediction();

				return OutSamples;
			}
		}
	}

	
	// Editor/editor preview worlds i.e. thumbnails/blueprint actor previews have undefined init orders. We can skip warning in those cases.
	// TODO Revisit if we expect/need trajectory to work in an editor preview world.
	const UWorld* World = GetWorld();
	if (World && World->WorldType != EWorldType::EditorPreview && World->WorldType != EWorldType::Editor)
	{
		UE_LOGF(LogMover, Warning, "Cannot predict trajectory on a MoverComponent that is not initialized. Returning defaulted samples. Actor: %ls  Initialized? %i BegunPlay? %i BeingDestroyed? %i WorldType: %ls", *GetNameSafe(GetOwner()), HasBeenInitialized(), HasBegunPlay(), IsBeingDestroyed(), LexToString(World->WorldType));
	}

	TArray<FTrajectorySampleInfo> BlankDefaultSamples;
	BlankDefaultSamples.AddDefaulted(PredictionParams.NumPredictionSamples);
	return BlankDefaultSamples;
}


FName UMoverComponent::GetMovementModeName() const
{ 
	return MoverSyncStateDoubleBuffer.GetReadable().MovementMode;
}

FName UMoverComponent::GetNextMovementModeName() const
{
	if (Simulation)
	{
		return Simulation->GetNextModeName();
	}

	return GetMovementModeName();
}


const UBaseMovementMode* UMoverComponent::GetMovementMode() const
{
	return GetActiveModeInternal(UBaseMovementMode::StaticClass());
}

UPrimitiveComponent* UMoverComponent::GetMovementBase() const
{
	if (LastMoverDefaultSyncState)
	{
		return LastMoverDefaultSyncState->GetMovementBase();
	}

	return nullptr;
}

FName UMoverComponent::GetMovementBaseBoneName() const
{
	if (LastMoverDefaultSyncState)
	{
		return LastMoverDefaultSyncState->GetMovementBaseBoneName();
	}

	return NAME_None;
}

bool UMoverComponent::HasValidCachedState() const
{
	return true;
}

const FMoverSyncState& UMoverComponent::GetSyncState() const
{
	return MoverSyncStateDoubleBuffer.GetReadable();
}

bool UMoverComponent::TryGetFloorCheckHitResult(FHitResult& OutHitResult) const
{
	FFloorCheckResult FloorCheck;
	if (SimBlackboard != nullptr && SimBlackboard->TryGet(CommonBlackboard::LastFloorResult, FloorCheck))
	{
		OutHitResult = FloorCheck.HitResult;
		return true;
	}
	return false;
}

const UMoverBlackboard* UMoverComponent::GetSimBlackboard() const
{
	return SimBlackboard;
}

UMoverBlackboard* UMoverComponent::GetSimBlackboard_Mutable() const
{
	return SimBlackboard;
}

bool UMoverComponent::HasValidCachedInputCmd() const
{
	return true;
}

const FMoverInputCmdContext& UMoverComponent::GetLastInputCmd() const
{
	return CachedLastUsedInputCmd;
}

const FMoverTimeStep& UMoverComponent::GetLastTimeStep() const
{
	return CachedLastSimTickTimeStep;
}

IMovementSettingsInterface* UMoverComponent::FindSharedSettings_Mutable(const UClass* ByType) const
{
	check(ByType);

	for (const TObjectPtr<UObject>& SettingsObj : SharedSettings)
	{
		if (SettingsObj && SettingsObj->IsA(ByType))
		{
			return Cast<IMovementSettingsInterface>(SettingsObj);
		}
	}

	return nullptr;
}

UObject* UMoverComponent::FindSharedSettings_Mutable_BP(TSubclassOf<UObject> SharedSetting) const
{
	if (SharedSetting->ImplementsInterface(UMovementSettingsInterface::StaticClass()))
    {
    	return Cast<UObject>(FindSharedSettings_Mutable(SharedSetting));
    }
    
    return nullptr;
}

const UObject* UMoverComponent::FindSharedSettings_BP(TSubclassOf<UObject> SharedSetting) const
{
	if (SharedSetting->ImplementsInterface(UMovementSettingsInterface::StaticClass()))
	{
		return Cast<UObject>(FindSharedSettings(SharedSetting));
	}

	return nullptr;
}

UBaseMovementMode* UMoverComponent::FindMode_Mutable(TSubclassOf<UBaseMovementMode> ModeType, bool bRequireExactClass) const
{
	if (ModeType)
	{
		for (const TPair<FName, TObjectPtr<UBaseMovementMode>>& NameModePair : MovementModes)
		{
			if ( (!bRequireExactClass && NameModePair.Value->IsA(ModeType)) || 
				 (NameModePair.Value->GetClass() == ModeType) )
			{
				return NameModePair.Value.Get();
			}
		}		
	}

	return nullptr;
}

UBaseMovementMode* UMoverComponent::FindMode_Mutable(TSubclassOf<UBaseMovementMode> ModeType, FName ModeName, bool bRequireExactClass) const
{
	if (!ModeName.IsNone())
	{
		if (const TObjectPtr<UBaseMovementMode>* FoundMode = MovementModes.Find(ModeName))
		{
			if ((!bRequireExactClass && FoundMode->IsA(ModeType)) || FoundMode->GetClass() == ModeType)
			{
				return *FoundMode;
			} 
		}
	}
	return nullptr;
}

UBaseMovementMode* UMoverComponent::GetActiveModeInternal(TSubclassOf<UBaseMovementMode> ModeType, bool bRequireExactClass) const
{
	if (const TObjectPtr<UBaseMovementMode>* CurrentMode = MovementModes.Find(GetMovementModeName()))
	{
		if ((!bRequireExactClass && CurrentMode->IsA(ModeType)) ||
			CurrentMode->GetClass() == ModeType)
		{
			return CurrentMode->Get();
		}
	}

	return nullptr;
}

bool UMoverComponent::MakeAndQueueLayeredMove(const TSubclassOf<ULayeredMoveLogic>& MoveLogicClass, const FLayeredMoveActivationParams* ActivationParams)
{
	// Find registered type for class passed in
	TObjectPtr<ULayeredMoveLogic> FoundRegisteredMoveLogic = nullptr;
	for (TObjectPtr<ULayeredMoveLogic> RegisteredMoveLogic : RegisteredMoves)
	{
		if (RegisteredMoveLogic.GetClass()->IsChildOf(MoveLogicClass))
		{
			FoundRegisteredMoveLogic = RegisteredMoveLogic;
			break;
		}
	}
			
	ULayeredMoveLogic* ActiveMoveLogic;
	TSharedPtr<FLayeredMoveInstancedData> QueuedInstancedData;

	if (FoundRegisteredMoveLogic)
	{
		ActiveMoveLogic = FoundRegisteredMoveLogic;

		const UScriptStruct* InstancedDataType = FoundRegisteredMoveLogic->GetInstancedDataType();
		if (InstancedDataType && InstancedDataType->IsChildOf(FLayeredMoveInstancedData::StaticStruct()))
		{
			TCheckedObjPtr<UScriptStruct> DataStructType = FoundRegisteredMoveLogic->GetInstancedDataType();
			FLayeredMoveInstancedData* NewMove = (FLayeredMoveInstancedData*)FMemory::Malloc(DataStructType->GetCppStructOps()->GetSize());
			DataStructType->InitializeStruct(NewMove);

			struct FAllocatedLayeredMoveDataDeleter
			{
				FORCEINLINE void operator()(FLayeredMoveInstancedData* MoveData) const
				{
					check(MoveData);
					UScriptStruct* ScriptStruct = MoveData->GetScriptStruct();
					check(ScriptStruct);
					ScriptStruct->DestroyStruct(MoveData);
					FMemory::Free(MoveData);
				}
			};
				
			QueuedInstancedData = TSharedRef<FLayeredMoveInstancedData>(NewMove, FAllocatedLayeredMoveDataDeleter());
			QueuedInstancedData->ActivateFromContext(ActivationParams);
		}
		else
		{
			UE_LOGF(LogMover, Warning, "%ls activation was queued on %ls but the move was NOT queued since it did not have valid data. InstancedDataStructType on Move Logic needs to be a FLayeredMoveInstancedData or child struct of.",
				*MoveLogicClass->GetName(),
				*GetOwner()->GetName());
			
			return false;
		}
	}
	else
	{
		UE_LOGF(LogMover, Warning, "%ls activation was queued on %ls and the move was not registered. Any move activated on a MoverComponent Needs to be Registered with the MoverCompoent. The layered move will not be queued for activation.",
			*MoveLogicClass->GetName(),
			*GetOwner()->GetName());
			
		return false;
	}
	
	const TSharedPtr<FLayeredMoveInstance> ActiveMoveToQueue = MakeShared<FLayeredMoveInstance>(QueuedInstancedData.ToSharedRef(), ActiveMoveLogic);

	if (Simulation)
	{
		Simulation->QueueLayeredMoveInstance(ActiveMoveToQueue);
	}
	
	return true;
}

void UMoverComponent::SetSimulationOutput(const FMoverTimeStep& TimeStep, const UE::Mover::FSimulationOutputData& OutputData)
{
	CachedLastSimTickTimeStep = TimeStep;

	// Keep CachedNewestSimTickTimeStep up to date for backends (e.g. ChaosMover) that drive
	// the mover via SetSimulationOutput rather than SimulationTick.
	if (TimeStep.ServerFrame > CachedNewestSimTickTimeStep.ServerFrame || TimeStep.BaseSimTimeMs > CachedNewestSimTickTimeStep.BaseSimTimeMs)
	{
		CachedNewestSimTickTimeStep = TimeStep;
	}

	CachedLastUsedInputCmd = OutputData.LastUsedInputCmd;

	FMoverSyncState& BufferedSyncState = MoverSyncStateDoubleBuffer.GetWritable();
	BufferedSyncState = OutputData.SyncState;
	LastMoverDefaultSyncState = BufferedSyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>();
	MoverSyncStateDoubleBuffer.Flip();

	// Replace the tag snapshot wholesale before dispatching events, so GetGameplayTags() is always consistent.
	// Preserve the previous snapshot to detect drift, i.e. tags that changed due to a correction and have no corresponding event.
	const FGameplayTagContainer PreviousGameplayTags = InternalGameplayTags;
	InternalGameplayTags = OutputData.InternalGameplayTags;
	RebuildAllGameplayTags();

	DispatchUnexpectedTagDriftEvents(PreviousGameplayTags, InternalGameplayTags, OutputData.Events, TimeStep);

	DispatchSimulationEvents(OutputData);

	// This is for things like the ground info that we want to cache and interpolate but isn't part of the networked sync state.
	// AdditionalOutputData is generic because ground info might not be useful for platforms, say, but platforms may want to store something else.
	SetAdditionalSimulationOutput(OutputData.AdditionalOutputData);

	// Flush montage states immediately after additional output is set, this drives the position of animation progress (in sim master mode)
	// or adjusts their position to prevent drift (when GT is allowed to predictively start playing montages, non sim master mode)
	if (const FMoverSimDrivenMontageData* MontageData = OutputData.AdditionalOutputData.FindDataByType<FMoverSimDrivenMontageData>())
	{
		FlushMontageStates(*MontageData);
	}
}

void UMoverComponent::DispatchSimulationEvents(const UE::Mover::FSimulationOutputData& OutputData)
{
	for (const TSharedPtr<FMoverSimulationEventData>& EventData : OutputData.Events)
	{
		if (const FMoverSimulationEventData* Data = EventData.Get())
		{
			DispatchSimulationEvent(*Data);
		}
	}
}

void UMoverComponent::DispatchGameplayTagTransitionEvents(const FGameplayTagContainer& PreviousTags, const FGameplayTagContainer& NewTags, const TArray<TSharedPtr<FMoverSimulationEventData>>& EventsToSkip, const FMoverTimeStep& TimeStep)
{
	const FMoverEventContext EventContext(TimeStep);

	for (const FGameplayTag& Tag : PreviousTags)
	{
		if (!NewTags.HasTagExact(Tag))
		{
			const bool bAlreadyCovered = EventsToSkip.ContainsByPredicate([&Tag](const TSharedPtr<FMoverSimulationEventData>& E)
			{
				const FMoverGameplayTagChangeEventData* TagEvent = E.IsValid() ? E->CastTo<FMoverGameplayTagChangeEventData>() : nullptr;
				return TagEvent && !TagEvent->bWasAdded && TagEvent->Tag == Tag;
			});

			if (!bAlreadyCovered)
			{
				if (ExternalGameplayTags.HasTagExact(Tag))
				{
					UE_LOGF(LogMover, Verbose, "Gameplay tag '%ls' removed event suppressed on frame %d - tag is also held externally.", *Tag.ToString(), TimeStep.ServerFrame);
				}
				else
				{
					OnGameplayTagRemoved.Broadcast(Tag, EventContext, /* bIsExternalTag = */ false);
				}
			}
		}
	}

	for (const FGameplayTag& Tag : NewTags)
	{
		if (!PreviousTags.HasTagExact(Tag))
		{
			const bool bAlreadyCovered = EventsToSkip.ContainsByPredicate([&Tag](const TSharedPtr<FMoverSimulationEventData>& E)
			{
				const FMoverGameplayTagChangeEventData* TagEvent = E.IsValid() ? E->CastTo<FMoverGameplayTagChangeEventData>() : nullptr;
				return TagEvent && TagEvent->bWasAdded && TagEvent->Tag == Tag;
			});

			if (!bAlreadyCovered)
			{
				if (ExternalGameplayTags.HasTagExact(Tag))
				{
					UE_LOGF(LogMover, Verbose, "Gameplay tag '%ls' added event suppressed on frame %d - tag is also held externally.", *Tag.ToString(), TimeStep.ServerFrame);
				}
				else
				{
					OnGameplayTagAdded.Broadcast(Tag, EventContext, /* bIsExternalTag = */ false);
				}
			}
		}
	}
}

void UMoverComponent::DispatchUnexpectedTagDriftEvents(const FGameplayTagContainer& PreviousTags, const FGameplayTagContainer& NewTags, const TArray<TSharedPtr<FMoverSimulationEventData>>& Events, const FMoverTimeStep& TimeStep)
{
	const AActor* DebugOwner = GetOwner();
	const UWorld* DebugWorld = DebugOwner ? DebugOwner->GetWorld() : nullptr;
	const FString DebugActorName = FString::Printf(TEXT("%s[%s]"), *GetNameSafe(DebugOwner), DebugWorld ? *DebugWorld->GetOutermost()->GetName() : TEXT("null"));
	const FString DebugActorRole = DebugOwner ? UEnum::GetValueAsString(TEXT("Engine.ENetRole"), DebugOwner->GetLocalRole()) : FString(TEXT("null"));

	// Every change between PreviousTags and NewTags should ideally be accounted for by a corresponding
	// FMoverGameplayTagChangeEventData in the simulation output events. These two sources of truth -
	// the PT snapshot and the PT event log - are both produced by the same simulation tick and should
	// always agree. Any divergence is flagged as PT/GT drift via an ensure and a warning log.
	//
	// Exception: during resimulation the GT's tag history may be discontinuous. Events that were
	// in flight before the rollback are pruned, and the PT may not re-emit them if its own
	// LastKnownGameplayTags was already at the post-transition state when the rollback fired
	// (e.g. a prediction that reached IsOnGround is rolled back to a frame where the authority
	// also has IsOnGround, so the PT sees no diff and produces no events). The GT then observes
	// a tag change with no covering event, which is an expected consequence of the rollback
	// correction rather than a real PT/GT mismatch.
	//
	// In all cases we recover by emitting the missing events via DispatchGameplayTagTransitionEvents.
	if (!TimeStep.bIsResimulating)
	{
		for (const FGameplayTag& Tag : PreviousTags)
		{
			if (!NewTags.HasTagExact(Tag))
			{
				const bool bAlreadyCovered = Events.ContainsByPredicate([&Tag](const TSharedPtr<FMoverSimulationEventData>& E)
				{
					const FMoverGameplayTagChangeEventData* TagEvent = E.IsValid() ? E->CastTo<FMoverGameplayTagChangeEventData>() : nullptr;
					return TagEvent && !TagEvent->bWasAdded && TagEvent->Tag == Tag;
				});

				if (!bAlreadyCovered)
				{
					ensureMsgf(false, TEXT("Gameplay tag '%s' was removed on frame %d without a corresponding simulation event - possible PT/GT drift [Actor=%s Role=%s]"), *Tag.ToString(), TimeStep.ServerFrame, *DebugActorName, *DebugActorRole);
					UE_LOGF(LogMover, Warning, "Gameplay tag '%ls' was removed on frame %d without a corresponding simulation event - possible PT/GT drift [Actor=%ls Role=%ls]", *Tag.ToString(), TimeStep.ServerFrame, *DebugActorName, *DebugActorRole);

					// Mover Gameplay Tag Logging
					LogGameplayTagDriftDetect(DebugActorName, *DebugActorRole, TimeStep, Tag, /* bWasAdded= */ false, Events, PreviousTags, NewTags);
				}
			}
		}

		for (const FGameplayTag& Tag : NewTags)
		{
			if (!PreviousTags.HasTagExact(Tag))
			{
				const bool bAlreadyCovered = Events.ContainsByPredicate([&Tag](const TSharedPtr<FMoverSimulationEventData>& E)
				{
					const FMoverGameplayTagChangeEventData* TagEvent = E.IsValid() ? E->CastTo<FMoverGameplayTagChangeEventData>() : nullptr;
					return TagEvent && TagEvent->bWasAdded && TagEvent->Tag == Tag;
				});

				if (!bAlreadyCovered)
				{
					ensureMsgf(false, TEXT("Gameplay tag '%s' was added on frame %d without a corresponding simulation event - possible PT/GT drift [Actor=%s Role=%s]"), *Tag.ToString(), TimeStep.ServerFrame, *DebugActorName, *DebugActorRole);
					UE_LOGF(LogMover, Warning, "Gameplay tag '%ls' was added on frame %d without a corresponding simulation event - possible PT/GT drift [Actor=%ls Role=%ls]", *Tag.ToString(), TimeStep.ServerFrame, *DebugActorName, *DebugActorRole);

					// Mover Gameplay Tag Logging
					LogGameplayTagDriftDetect(DebugActorName, *DebugActorRole, TimeStep, Tag, /* bWasAdded= */ true, Events, PreviousTags, NewTags);
				}
			}
		}
	}

	DispatchGameplayTagTransitionEvents(PreviousTags, NewTags, Events, TimeStep);
}

void UMoverComponent::DispatchSimulationEvent(const FMoverSimulationEventData& EventData)
{
	// This gives the event a callback when it is processed on the game thread
	FMoverSimEventGameThreadContext GTContext({ this });
	EventData.OnEventProcessed(GTContext);

	// Process the simulation event at the mover component (or derived) level
	ProcessSimulationEvent(EventData);

	// Broadcast the event outside mover component
	if (OnPostSimEventReceived.IsBound())
	{
		OnPostSimEventReceived.Broadcast(EventData);
	}
}

void UMoverComponent::ProcessSimulationEvent(const FMoverSimulationEventData& EventData)
{
	// On a mode change call deactivate on the previous mode and activate on the new mode,
	// then broadcast the mode changed event
	if (const FMovementModeChangedEventData* ModeChangedData = EventData.CastTo<FMovementModeChangedEventData>())
	{
		if (ModeChangedData->PreviousModeName != NAME_None)
		{
			if (TObjectPtr<UBaseMovementMode>* PrevModePtr = MovementModes.Find(ModeChangedData->PreviousModeName))
			{
				UBaseMovementMode* PrevMode = PrevModePtr->Get();
				if (PrevMode && PrevMode->bSupportsAsync)
				{
					PrevMode->Deactivate_External(EventData.Context);
				}
			}
		}

		if (ModeChangedData->NewModeName != NAME_None)
		{
			if (TObjectPtr<UBaseMovementMode>* NewModePtr = MovementModes.Find(ModeChangedData->NewModeName))
			{
				UBaseMovementMode* NewMode = NewModePtr->Get();
				if (NewMode && NewMode->bSupportsAsync)
				{
					NewMode->Activate_External(EventData.Context);
				}
			}
		}

		OnMovementModeChanged.Broadcast(ModeChangedData->PreviousModeName, ModeChangedData->NewModeName);
	}
	else if (const FTeleportSucceededEventData* TeleportSucceededEventData = EventData.CastTo<FTeleportSucceededEventData>())
	{
		OnTeleportSucceeded.Broadcast(TeleportSucceededEventData->FromLocation, TeleportSucceededEventData->FromRotation, TeleportSucceededEventData->ToLocation, TeleportSucceededEventData->ToRotation);
	}
	else if (const FTeleportFailedEventData* TeleportFailedEventData = EventData.CastTo<FTeleportFailedEventData>())
	{
		OnTeleportFailed.Broadcast(TeleportFailedEventData->FromLocation, TeleportFailedEventData->FromRotation, TeleportFailedEventData->ToLocation, TeleportFailedEventData->ToRotation, TeleportFailedEventData->TeleportFailureReason);
	}
	else if (const FMoverGameplayTagChangeEventData* TagStateChangeData = EventData.CastTo<FMoverGameplayTagChangeEventData>())
	{
		if (TagStateChangeData->bWasAdded)
		{
			if (ExternalGameplayTags.HasTagExact(TagStateChangeData->Tag))
			{
				UE_LOGF(LogMover, Verbose, "Simulation gameplay tag '%ls' added but tag is already active externally - event suppressed to avoid a duplicate added event.", *TagStateChangeData->Tag.ToString());
			}
			else
			{
				OnGameplayTagAdded.Broadcast(TagStateChangeData->Tag, TagStateChangeData->Context, TagStateChangeData->bIsExternalTag);
			}
		}
		else
		{
			if (ExternalGameplayTags.HasTagExact(TagStateChangeData->Tag))
			{
				UE_LOGF(LogMover, Verbose, "Simulation gameplay tag '%ls' removed but tag is still active externally - event suppressed to avoid a duplicate removed event.", *TagStateChangeData->Tag.ToString());
			}
			else
			{
				OnGameplayTagRemoved.Broadcast(TagStateChangeData->Tag, TagStateChangeData->Context, TagStateChangeData->bIsExternalTag);
			}
		}
	}
}

FGameplayTagContainer UMoverComponent::GetGameplayTags() const
{
	if ((BackendLiaisonComp && BackendLiaisonComp->IsAsync()) || MoverComponentCVars::UseSnapshotGameplayTagsForSyncBackend != 0)
	{
		return AllGameplayTags;
	}

	FGameplayTagContainer LiveTags;
	CollectGameplayTagsFromSyncState(MoverSyncStateDoubleBuffer.GetReadable(), LiveTags);
	LiveTags.AppendTags(ExternalGameplayTags);
	return LiveTags;
}

void UMoverComponent::RebuildAllGameplayTags()
{
	AllGameplayTags = InternalGameplayTags;
	AllGameplayTags.AppendTags(ExternalGameplayTags);
}

void UMoverComponent::CollectGameplayTagsFromSyncState(const FMoverSyncState& SyncState, FGameplayTagContainer& OutTags) const
{
	if (const UBaseMovementMode* ActiveMovementMode = FindMovementModeByName(SyncState.MovementMode))
	{
		ActiveMovementMode->GetGameplayTags(OutTags);
	}

	for (auto ModifierIt = SyncState.MovementModifiers.GetActiveModifiersIterator(); ModifierIt; ++ModifierIt)
	{
		if (const TSharedPtr<FMovementModifierBase> Modifier = *ModifierIt)
		{
			Modifier->GetGameplayTags(OutTags);
		}
	}

	for (const TSharedPtr<FLayeredMoveBase>& LayeredMove : SyncState.LayeredMoves.GetActiveMoves())
	{
		if (LayeredMove.IsValid())
		{
			LayeredMove->GetGameplayTags(OutTags);
		}
	}
}

void UMoverComponent::SetAdditionalSimulationOutput(const FMoverDataCollection& Data)
{
}

void UMoverComponent::FlushMontageStates(const FMoverSimDrivenMontageData& MontageData)
{
	UAnimInstance* AnimInstance = nullptr;
	if (const USkeletalMeshComponent* MeshComp = Cast<USkeletalMeshComponent>(GetPrimaryVisualComponent()))
	{
		AnimInstance = MeshComp->GetAnimInstance();
	}

	// Apply all entries present in the latest simulation snapshot
	TSet<UAnimMontage*> SeenMontages;
	SeenMontages.Reserve(MontageData.MontageStates.Num());
	for (const FMoverSimDrivenMontageEntry& Entry : MontageData.MontageStates)
	{
		SeenMontages.Add(Entry.Montage);
		ApplyMontageEntry(Entry, AnimInstance);
	}

	// Stop any montages that disappeared from simulation output without a normal finish signal (abrupt cancel)
	TArray<UAnimMontage*> ToRemove;
	for (const auto& [Montage, LastEntry] : DrivenMontageStateMap)
	{
		if (!SeenMontages.Contains(Montage))
		{
			if (!LastEntry.bShouldBlendOut && !LastEntry.bIsFinished)
			{
				// Abrupt cancellation -- no lifecycle signal was sent by the simulation, hard stop
				StopDrivenMontage(LastEntry, AnimInstance);
			}
			ToRemove.Add(Montage);
		}
	}
	for (UAnimMontage* Montage : ToRemove)
	{
		DrivenMontageStateMap.Remove(Montage);
	}
}

void UMoverComponent::ApplyMontageEntry(const FMoverSimDrivenMontageEntry& Entry, UAnimInstance* AnimInstance)
{
	// Read previous state before updating so we can detect edge transitions
	const FMoverSimDrivenMontageEntry* Previous = DrivenMontageStateMap.Find(Entry.Montage);
	const bool bWasBlendOutTriggered = Previous ? Previous->bShouldBlendOut : false;

	// Always update to the latest simulation state
	DrivenMontageStateMap.FindOrAdd(Entry.Montage) = Entry;

	if (!AnimInstance || !Entry.Montage)
	{
		return;
	}

	// In async mode. when the simulation owns montage position, PlayRate is held at 0 between flushes
	// so the GT tick cannot advance position independently. We use SetNextPositionWithEvents to
	// move the montage to the sim-authoritative position and fire notifies for the traversed range;
	// this also drains any accumulated FMontageSubStepper::TimeRemaining, preventing the overshoot
	// that occurs when Advance() is called directly with PlayRate locked at 0.
	// In non-async mode the GT tick is the sole driver of position so we leave it alone entirely.
	if (IsBackendAsync())
	{
		// GetInstanceForMontage returns the instance even while blending out, which we need
		// so position stays correct through the entire blend-out window.
		FAnimMontageInstance* MontageInst = AnimInstance->GetInstanceForMontage(Entry.Montage);
		if (!MontageInst && !Previous)
		{
			// Montage not yet started on the GT. Start with rate 1 (PlayRate=0 is not guaranteed
			// to create an active instance), then lock to 0 so the anim graph cannot advance
			// between this call and our SetPlayRate below -- we are in the game thread flush path.
			AnimInstance->Montage_Play(Entry.Montage, /* InPlayRate= */ 1.f);
			MontageInst = AnimInstance->GetInstanceForMontage(Entry.Montage);
		}

		if (MontageInst)
		{
			if (MoverCVars::bEnableAsyncMontageSimMaster)
			{
				if (!Previous)
				{
					// First encounter: jump directly to the sim-authoritative position and lock.
					// SetNextPositionWithEvents(0, CurrentPosition) would fire notifies for the
					// entire range [0, CurrentPosition), retroactively triggering notifies the
					// move started past. SetPosition bypasses that. No TimeRemaining has
					// accumulated yet (montage was just created by Montage_Play above), so
					// skipping the FMontageSubStepper drain path is safe here.
					MontageInst->SetPosition(Entry.CurrentPosition);
					MontageInst->SetPlayRate(0.f);
				}
				else
				{
					// Subsequent frames: advance with events so notifies fire for the traversed
					// range and any accumulated FMontageSubStepper::TimeRemaining is drained,
					// preventing the overshoot that Advance() causes with PlayRate locked at 0.
					MontageInst->SetNextPositionWithEvents(Previous->CurrentPosition, Entry.CurrentPosition);
				}
			}
			else
			{
				// GT-playing mode: snap the montage position to the sim value at each flush but
				// leave PlayRate unlocked so the anim graph animates bones freely between flushes.
				MontageInst->SetPosition(Entry.CurrentPosition);
			}
		}
	}

	// Trigger blend-out once on the first simulation substep that signals it.
	// In async mode the blend-out is always a natural completion driven by the simulation,
	// so pass bInterrupt=false. UAnimInstance::Montage_Stop always passes bInterrupt=true,
	// which would fire OnInterrupted instead of OnBlendOut/OnCompleted.
	if (Entry.bShouldBlendOut && !bWasBlendOutTriggered)
	{
		if (IsBackendAsync())
		{
			if (FAnimMontageInstance* MontageInst = AnimInstance->GetInstanceForMontage(Entry.Montage))
			{
				MontageInst->Stop(FAlphaBlend(Entry.BlendOutTimeSeconds), /* bInterrupt= */ false);
			}
		}
		else
		{
			AnimInstance->Montage_Stop(Entry.BlendOutTimeSeconds, Entry.Montage);
		}
	}
	else if (Entry.bIsFinished && !Entry.bShouldBlendOut)
	{
		// Montage played to natural completion with no blend-out phase configured.
		if (IsBackendAsync())
		{
			if (FAnimMontageInstance* MontageInst = AnimInstance->GetInstanceForMontage(Entry.Montage))
			{
				MontageInst->Stop(FAlphaBlend(0.f), /* bInterrupt= */ false);
			}
		}
		else
		{
			AnimInstance->Montage_Stop(0.f, Entry.Montage);
		}
	}

	// Remove from tracking once the simulation marks the move as fully done
	if (Entry.bIsFinished)
	{
		DrivenMontageStateMap.Remove(Entry.Montage);
	}
}

void UMoverComponent::StopDrivenMontage(const FMoverSimDrivenMontageEntry& LastEntry, UAnimInstance* AnimInstance)
{
	if (AnimInstance && LastEntry.Montage)
	{
		AnimInstance->Montage_Stop(0.f, LastEntry.Montage);
	}
}




#undef LOCTEXT_NAMESPACE
