// Copyright Epic Games, Inc. All Rights Reserved.


#include "MoverDataModelTypes.h"
#include "Components/PrimitiveComponent.h"
#include "MoveLibrary/BasedMovementUtils.h"
#include "MoverLog.h"



// FCharacterDefaultInputs //////////////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoverDataModelTypes)

void FCharacterDefaultInputs::SetMoveInput(EMoveInputType InMoveInputType, const FVector& InMoveInput)
{
	MoveInputType = InMoveInputType;

	// Limit the precision that we store, so that it matches what is NetSerialized (2 decimal place of precision).
	// This ensures the authoring client, server, and any networking peers are all simulating with the same move input.
	// Note: any change to desired precision must be made here and in NetSerialize
	MoveInput.X = FMath::RoundToFloat(InMoveInput.X * 100.0) / 100.0;
	MoveInput.Y = FMath::RoundToFloat(InMoveInput.Y * 100.0) / 100.0;
	MoveInput.Z = FMath::RoundToFloat(InMoveInput.Z * 100.0) / 100.0;
}


FVector FCharacterDefaultInputs::GetMoveInput_WorldSpace() const
{
	if (bUsingMovementBase && MovementBase)
	{
		FVector MoveInputWorldSpace;
		UBasedMovementUtils::TransformBasedDirectionToWorld(MovementBase, MovementBaseBoneName, MoveInput, MoveInputWorldSpace);
		return MoveInputWorldSpace;
	}

	return MoveInput;	// already in world space
}


FVector FCharacterDefaultInputs::GetOrientationIntentDir_WorldSpace() const
{
	if (bUsingMovementBase && MovementBase)
	{
		FVector OrientIntentDirWorldSpace;
		UBasedMovementUtils::TransformBasedDirectionToWorld(MovementBase, MovementBaseBoneName, OrientationIntent, OrientIntentDirWorldSpace);
		return OrientIntentDirWorldSpace;
	}

	return OrientationIntent;	// already in world space
}


FMoverDataStructBase* FCharacterDefaultInputs::Clone() const
{
	// TODO: ensure that this memory allocation jives with deletion method
	FCharacterDefaultInputs* CopyPtr = new FCharacterDefaultInputs(*this);
	return CopyPtr;
}

bool FCharacterDefaultInputs::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	Super::NetSerialize(Ar, Map, bOutSuccess);

	Ar << MoveInputType;

	SerializePackedVector<100, 30>(MoveInput, Ar);	// Note: if you change this serialization, also change in SetMoveInput
	SerializeFixedVector<1, 16>(OrientationIntent, Ar);
	ControlRotation.SerializeCompressedShort(Ar);

	Ar << SuggestedMovementMode;

	Ar.SerializeBits(&bUsingMovementBase, 1);

	if (bUsingMovementBase)
	{
		Ar << MovementBase;
		Ar << MovementBaseBoneName;
	}
	else if (Ar.IsLoading())
	{
		// skip attempts to load movement base properties if flagged as not using a movement base
		MovementBase = nullptr;
		MovementBaseBoneName = NAME_None;
	}

	Ar.SerializeBits(&bIsJumpJustPressed, 1);
	Ar.SerializeBits(&bIsJumpPressed, 1);

	bOutSuccess = true;
	return true;
}


void FCharacterDefaultInputs::ToString(FAnsiStringBuilderBase& Out) const
{
	Super::ToString(Out);

	Out.Appendf("MoveInput: %s (Type %u)\n", TCHAR_TO_ANSI(*MoveInput.ToCompactString()), uint32{EnumToUnderlyingType(MoveInputType)});
	Out.Appendf("OrientationIntent: X=%.2f Y=%.2f Z=%.2f\n", OrientationIntent.X, OrientationIntent.Y, OrientationIntent.Z);
	Out.Appendf("ControlRotation: P=%.2f Y=%.2f R=%.2f\n", ControlRotation.Pitch, ControlRotation.Yaw, ControlRotation.Roll);
	Out.Appendf("SuggestedMovementMode: %s\n", TCHAR_TO_ANSI(*SuggestedMovementMode.ToString()));

	if (MovementBase)
	{
		Out.Appendf("MovementBase: %s (bone %s)\n", TCHAR_TO_ANSI(*GetNameSafe(MovementBase->GetOwner())), TCHAR_TO_ANSI(*MovementBaseBoneName.ToString()));
	}
	else
	{
		Out.Appendf("MovementBase: none\n");
	}

	Out.Appendf("bIsJumpPressed: %i\tbIsJumpJustPressed: %i\n", bIsJumpPressed, bIsJumpJustPressed);
}

bool FCharacterDefaultInputs::ShouldReconcile(const FMoverDataStructBase& AuthorityState) const
{
	const FCharacterDefaultInputs& TypedAuthority = static_cast<const FCharacterDefaultInputs&>(AuthorityState);
	return *this != TypedAuthority;
}

void FCharacterDefaultInputs::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct)
{
	const FCharacterDefaultInputs& TypedFrom = static_cast<const FCharacterDefaultInputs&>(From);
	const FCharacterDefaultInputs& TypedTo = static_cast<const FCharacterDefaultInputs&>(To);

	const FCharacterDefaultInputs* ClosestInputs = Pct + UE_SMALL_NUMBER < 0.5f ? &TypedFrom : &TypedTo;

	// Set bIsJumpJustPressed if previous input was not jumping while next input is jumping but not just pressed (just pressed needs to have happened between the two inputs)
	bIsJumpJustPressed = (!TypedFrom.bIsJumpJustPressed && !TypedFrom.bIsJumpPressed) && (!ClosestInputs->bIsJumpJustPressed && ClosestInputs->bIsJumpPressed);
	bIsJumpPressed = ClosestInputs->bIsJumpPressed;

	// Note, this ignores movement base as this is not used by the physics mover
	SuggestedMovementMode = ClosestInputs->SuggestedMovementMode;

	SetMoveInput(ClosestInputs->GetMoveInputType(), FMath::Lerp(TypedFrom.GetMoveInput(), TypedTo.GetMoveInput(), Pct));

	// Slerp facing direction; preserve zero-vector "no intent" sentinel
	if (!TypedFrom.OrientationIntent.IsNearlyZero() && !TypedTo.OrientationIntent.IsNearlyZero())
	{
		OrientationIntent = FQuat::Slerp(TypedFrom.OrientationIntent.ToOrientationQuat(), TypedTo.OrientationIntent.ToOrientationQuat(), Pct).GetForwardVector();
	}
	else
	{
		OrientationIntent = ClosestInputs->OrientationIntent;
	}

	ControlRotation = FQuat::Slerp(TypedFrom.ControlRotation.Quaternion(), TypedTo.ControlRotation.Quaternion(), Pct).Rotator();
}

void FCharacterDefaultInputs::Merge(const FMoverDataStructBase& From)
{
	const FCharacterDefaultInputs& TypedFrom = static_cast<const FCharacterDefaultInputs&>(From);

	// Booleans: additive (true if either frame had it true)
	bIsJumpJustPressed |= TypedFrom.bIsJumpJustPressed;
	bIsJumpPressed |= TypedFrom.bIsJumpPressed;

	// Vectors/rotators: average adjacent frames (*this is newer, TypedFrom is older)
	if (GetMoveInputType() == TypedFrom.GetMoveInputType())
	{
		SetMoveInput(GetMoveInputType(), (GetMoveInput() + TypedFrom.GetMoveInput()) * 0.5);
	}
	// else: input types differ, keep the MoveInput from *this (newer)

	// Slerp facing direction; keep *this if either frame has the zero "no intent" sentinel
	if (!OrientationIntent.IsNearlyZero() && !TypedFrom.OrientationIntent.IsNearlyZero())
	{
		OrientationIntent = FQuat::Slerp(OrientationIntent.ToOrientationQuat(), TypedFrom.OrientationIntent.ToOrientationQuat(), 0.5f).GetForwardVector();
	}

	// Slerp so opposite yaws don't cancel like component-wise FRotator lerp does
	ControlRotation = FQuat::Slerp(ControlRotation.Quaternion(), TypedFrom.ControlRotation.Quaternion(), 0.5f).Rotator();

	// Discrete values: *this is the newer/accumulated frame, keep existing values unchanged.
	// bUsingMovementBase, MovementBase, MovementBaseBoneName, SuggestedMovementMode - all kept from *this.
	// Note: bUsingMovementBase is discrete (current state), not additive. OR-ing it can produce
	// bUsingMovementBase=true with a null MovementBase when the newer frame left the base.
}

static float CharacterDefaultInputsDecayAmountMultiplier = 1.f;
FAutoConsoleVariableRef CVarCharacterDefaultInputsDecayAmountMultiplier(
	TEXT("Mover.Input.CharacterDefaultInputsDecayAmountMultiplier"),
	CharacterDefaultInputsDecayAmountMultiplier,
	TEXT("Multiplier to use when decaying CharacterDefaultInputs."));

void FCharacterDefaultInputs::Decay(float DecayAmount)
{
	DecayAmount *= CharacterDefaultInputsDecayAmountMultiplier;

	MoveInput *= (1.0f - DecayAmount);
}

// FMoverDefaultSyncState //////////////////////////////////////////////////////////////

FMoverDataStructBase* FMoverDefaultSyncState::Clone() const
{
	// TODO: ensure that this memory allocation jives with deletion method
	FMoverDefaultSyncState* CopyPtr = new FMoverDefaultSyncState(*this);
	return CopyPtr;
}

bool FMoverDefaultSyncState::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	Super::NetSerialize(Ar, Map, bOutSuccess);

	SerializePackedVector<100, 30>(Location, Ar);
	SerializeFixedVector<2, 8>(MoveDirectionIntent, Ar);
	SerializePackedVector<10, 16>(Velocity, Ar);
	SerializePackedVector<10, 16>(AngularVelocityDegrees, Ar);
	Orientation.SerializeCompressedShort(Ar);

	// Optional movement base
	bool bIsUsingMovementBase = (Ar.IsSaving() ? MovementBase.IsValid() : false);
	Ar.SerializeBits(&bIsUsingMovementBase, 1);

	if (bIsUsingMovementBase)
	{
		Ar << MovementBase;
		Ar << MovementBaseBoneName;

		SerializePackedVector<100, 30>(MovementBasePos, Ar);
		MovementBaseQuat.NetSerialize(Ar, Map, bOutSuccess);
	}
	else if (Ar.IsLoading())
	{
		MovementBase = nullptr;
		MovementBaseBoneName = NAME_None;
	}

	bOutSuccess = true;
	return true;
}

void FMoverDefaultSyncState::ToString(FAnsiStringBuilderBase& Out) const
{
	Super::ToString(Out);

	Out.Appendf("Loc: X=%.2f Y=%.2f Z=%.2f\n", Location.X, Location.Y, Location.Z);
	Out.Appendf("Intent: X=%.2f Y=%.2f Z=%.2f\n", MoveDirectionIntent.X, MoveDirectionIntent.Y, MoveDirectionIntent.Z);
	Out.Appendf("Vel: X=%.2f Y=%.2f Z=%.2f\n", Velocity.X, Velocity.Y, Velocity.Z);
	Out.Appendf("Ang Vel: X=%.2f Y=%.2f Z=%.2f\n", AngularVelocityDegrees.X, AngularVelocityDegrees.Y, AngularVelocityDegrees.Z);
	Out.Appendf("Orient: P=%.2f Y=%.2f R=%.2f\n", Orientation.Pitch, Orientation.Yaw, Orientation.Roll);

	if (const UPrimitiveComponent* MovementBasePtr = MovementBase.Get())
	{
		Out.Appendf("MovementBase: %s (bone %s)\n", TCHAR_TO_ANSI(*GetNameSafe(MovementBasePtr->GetOwner())), TCHAR_TO_ANSI(*MovementBaseBoneName.ToString()));
		Out.Appendf("    BasePos: %s   BaseRot: %s\n", TCHAR_TO_ANSI(*MovementBasePos.ToCompactString()), TCHAR_TO_ANSI(*MovementBaseQuat.Rotator().ToCompactString()));
	}
	else
	{
		Out.Appendf("MovementBase: none\n");
	}

}


bool FMoverDefaultSyncState::ShouldReconcile(const FMoverDataStructBase& AuthorityState) const
{
	const FMoverDefaultSyncState* AuthoritySyncState = static_cast<const FMoverDefaultSyncState*>(&AuthorityState);
	const float DistErrorTolerance = 5.f;	// JAH TODO: define these elsewhere as CVars or data asset settings

	const bool bAreInDifferentSpaces = !((MovementBase.HasSameIndexAndSerialNumber(AuthoritySyncState->MovementBase)) && (MovementBaseBoneName == AuthoritySyncState->MovementBaseBoneName));

	bool bIsNearEnough = false;

	if (!bAreInDifferentSpaces)
	{
		if (MovementBase.IsValid())
		{
			bIsNearEnough = GetLocation_BaseSpace().Equals(AuthoritySyncState->GetLocation_BaseSpace(), DistErrorTolerance);
		}
		else
		{
			bIsNearEnough = GetLocation_WorldSpace().Equals(AuthoritySyncState->GetLocation_WorldSpace(), DistErrorTolerance);
		}
	}

	return bAreInDifferentSpaces || !bIsNearEnough;
}


void FMoverDefaultSyncState::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct)
{
	const FMoverDefaultSyncState* FromState = static_cast<const FMoverDefaultSyncState*>(&From);
	const FMoverDefaultSyncState* ToState = static_cast<const FMoverDefaultSyncState*>(&To);

	// TODO: investigate replacing this threshold with a flag indicating that the state (or parts thereof) isn't intended to be interpolated
	static constexpr float TeleportThreshold = 500.f * 500.f;
	if (FVector::DistSquared(FromState->GetLocation_WorldSpace(), ToState->GetLocation_WorldSpace()) > TeleportThreshold)
	{
		*this = *ToState;
	}
	else
	{

		// No matter what base we started from, we always interpolate into the "To" movement base's space
		MovementBase         = ToState->MovementBase;
		MovementBaseBoneName = ToState->MovementBaseBoneName;
		MovementBasePos      = ToState->MovementBasePos;
		MovementBaseQuat     = ToState->MovementBaseQuat;

		FVector FromLocation_ToSpace, FromIntent_ToSpace, FromVelocity_ToSpace, FromAngularVelocity_ToSpace;
		FRotator FromOrientation_ToSpace;

	
		// Bases match (or not using based movement at all)
		if (FromState->MovementBase.HasSameIndexAndSerialNumber(ToState->MovementBase) && FromState->MovementBaseBoneName == ToState->MovementBaseBoneName)
		{
			if (FromState->MovementBase.IsValid())
			{
				MovementBasePos = FMath::Lerp(FromState->MovementBasePos, ToState->MovementBasePos, Pct);
				MovementBaseQuat = FQuat::Slerp(FromState->MovementBaseQuat, ToState->MovementBaseQuat, Pct);
			}

			FromLocation_ToSpace    = FromState->Location;
			FromIntent_ToSpace      = FromState->MoveDirectionIntent;
			FromVelocity_ToSpace    = FromState->Velocity;
			FromAngularVelocity_ToSpace    = FromState->AngularVelocityDegrees;
			FromOrientation_ToSpace = FromState->Orientation;
		}
		else if (ToState->MovementBase.IsValid())	// if moving onto a different base, regardless of coming from a prior base or not
		{
			UBasedMovementUtils::TransformLocationToLocal(ToState->MovementBasePos, ToState->MovementBaseQuat, FromState->GetLocation_WorldSpace(), OUT FromLocation_ToSpace);
			UBasedMovementUtils::TransformDirectionToLocal(ToState->MovementBaseQuat, FromState->GetIntent_WorldSpace(), OUT FromIntent_ToSpace);
			UBasedMovementUtils::TransformDirectionToLocal(ToState->MovementBaseQuat, FromState->GetVelocity_WorldSpace(), OUT FromVelocity_ToSpace);
			UBasedMovementUtils::TransformDirectionToLocal(ToState->MovementBaseQuat, FromState->GetAngularVelocityDegrees_WorldSpace(), OUT FromAngularVelocity_ToSpace);
			UBasedMovementUtils::TransformRotatorToLocal(ToState->MovementBaseQuat, FromState->GetOrientation_WorldSpace(), OUT FromOrientation_ToSpace);
		}
		else if (FromState->MovementBase.IsValid())	// if leaving a base
		{
			FromLocation_ToSpace	= FromState->GetLocation_WorldSpace();
			FromIntent_ToSpace		= FromState->GetIntent_WorldSpace();
			FromVelocity_ToSpace	= FromState->GetVelocity_WorldSpace();
			FromAngularVelocity_ToSpace	= FromState->GetAngularVelocityDegrees_WorldSpace();
			FromOrientation_ToSpace = FromState->GetOrientation_WorldSpace();
		}

		Location			= FMath::Lerp(FromLocation_ToSpace,		ToState->Location, Pct);
		MoveDirectionIntent = FMath::Lerp(FromIntent_ToSpace,		ToState->MoveDirectionIntent, Pct);
		Velocity			= FMath::Lerp(FromVelocity_ToSpace,		ToState->Velocity, Pct);
		AngularVelocityDegrees = FMath::Lerp(FromAngularVelocity_ToSpace,ToState->AngularVelocityDegrees, Pct);
		Orientation			= FMath::Lerp(FromOrientation_ToSpace,	ToState->Orientation, Pct);

	}
}

void FMoverDefaultSyncState::SetTransforms_WorldSpace(FVector WorldLocation, FRotator WorldOrient, FVector WorldVelocity, UPrimitiveComponent* Base,
	FName BaseBone)
{
	SetTransforms_WorldSpace(WorldLocation, WorldOrient, WorldVelocity, FVector::ZeroVector, Base, BaseBone);
}

void FMoverDefaultSyncState::SetTransforms_WorldSpace(FVector WorldLocation, FRotator WorldOrient, FVector WorldVelocity, FVector WorldAngularVelocityDegrees, UPrimitiveComponent* Base, FName BaseBone)
{
	if (SetMovementBase(Base, BaseBone))
	{
		UBasedMovementUtils::TransformLocationToLocal(  MovementBasePos,  MovementBaseQuat, WorldLocation, OUT Location);
		UBasedMovementUtils::TransformRotatorToLocal(   MovementBaseQuat, WorldOrient, OUT Orientation);
		UBasedMovementUtils::TransformDirectionToLocal( MovementBaseQuat, WorldVelocity, OUT Velocity);
		UBasedMovementUtils::TransformDirectionToLocal( MovementBaseQuat, WorldAngularVelocityDegrees, OUT AngularVelocityDegrees);
	}
	else
	{
		if (Base)
		{
			UE_LOGF(LogMover, Warning, "Failed to set base as %ls. Falling back to world space movement", *GetNameSafe(Base->GetOwner()));
		}

		Location = WorldLocation;
		Orientation = WorldOrient;
		Velocity = WorldVelocity;
		AngularVelocityDegrees = WorldAngularVelocityDegrees;
	}
}


bool FMoverDefaultSyncState::SetMovementBase(UPrimitiveComponent* Base, FName BaseBone)
{
	MovementBase = Base;
	MovementBaseBoneName = BaseBone;

	const bool bDidCaptureBaseTransform = UpdateCurrentMovementBase();
	return !Base || bDidCaptureBaseTransform;
}


bool FMoverDefaultSyncState::UpdateCurrentMovementBase()
{
	bool bDidGetBaseTransform = false;

	if (const UPrimitiveComponent* MovementBasePtr = MovementBase.Get())
	{
		bDidGetBaseTransform = UBasedMovementUtils::GetMovementBaseTransform(MovementBasePtr, MovementBaseBoneName, OUT MovementBasePos, OUT MovementBaseQuat);
	}

	if (!bDidGetBaseTransform)
	{
		MovementBase = nullptr;
		MovementBaseBoneName = NAME_None;
		MovementBasePos = FVector::ZeroVector;
		MovementBaseQuat = FQuat::Identity;
	}

	return bDidGetBaseTransform;
}

bool FMoverDefaultSyncState::IsNearlyEqual(const FMoverDefaultSyncState& Other) const
{
	const bool bHasSameBaseBaseInfo = (!MovementBase.IsValid() && !Other.MovementBase.IsValid()) ||
											(MovementBase == Other.MovementBase && 
											 MovementBaseBoneName == Other.MovementBaseBoneName &&
											 (MovementBasePos - Other.MovementBasePos).IsNearlyZero() && 
											 MovementBaseQuat.Equals(Other.MovementBaseQuat));

	return (Location-Other.Location).IsNearlyZero() &&
		(Orientation-Other.Orientation).IsNearlyZero() &&
		(Velocity-Other.Velocity).IsNearlyZero() &&(AngularVelocityDegrees-Other.AngularVelocityDegrees).IsNearlyZero() &&
		(MoveDirectionIntent-Other.MoveDirectionIntent).IsNearlyZero() &&
			bHasSameBaseBaseInfo;
}

FVector FMoverDefaultSyncState::GetLocation_WorldSpace() const
{
	if (MovementBase.IsValid())
	{
		return FTransform(MovementBaseQuat, MovementBasePos).TransformPositionNoScale(Location);
	}

	return Location; // if no base, assumed to be in world space
}

FVector FMoverDefaultSyncState::GetLocation_BaseSpace() const
{
	return Location;
}


FVector FMoverDefaultSyncState::GetIntent_WorldSpace() const
{
	if (MovementBase.IsValid())
	{
		return MovementBaseQuat.RotateVector(MoveDirectionIntent);
	}

	return MoveDirectionIntent; // if no base, assumed to be in world space
}

FVector FMoverDefaultSyncState::GetIntent_BaseSpace() const
{
	return MoveDirectionIntent;
}

FVector FMoverDefaultSyncState::GetVelocity_WorldSpace() const
{
	if (MovementBase.IsValid())
	{
		return MovementBaseQuat.RotateVector(Velocity);
	}

	return Velocity; // if no base, assumed to be in world space
}

FVector FMoverDefaultSyncState::GetVelocity_BaseSpace() const
{
	return Velocity;
}


FRotator FMoverDefaultSyncState::GetOrientation_WorldSpace() const
{
	if (MovementBase.IsValid())
	{
		return (MovementBaseQuat * FQuat(Orientation)).Rotator();
	}

	return Orientation; // if no base, assumed to be in world space
}


FRotator FMoverDefaultSyncState::GetOrientation_BaseSpace() const
{
	return Orientation;
}

FTransform FMoverDefaultSyncState::GetTransform_WorldSpace() const
{
	if (MovementBase.IsValid())
	{
		return FTransform(Orientation, Location) * FTransform(MovementBaseQuat, MovementBasePos);
	}

	return FTransform(Orientation, Location);
}

FTransform FMoverDefaultSyncState::GetTransform_BaseSpace() const
{
	return FTransform(Orientation, Location);
}

FVector FMoverDefaultSyncState::GetAngularVelocityDegrees_WorldSpace() const
{
	if (MovementBase.IsValid())
	{
		return MovementBaseQuat.RotateVector(AngularVelocityDegrees);
	}

	return AngularVelocityDegrees; // if no base, assumed to be in world space
}

FVector FMoverDefaultSyncState::GetAngularVelocityDegrees_BaseSpace() const
{
	return AngularVelocityDegrees;
}


// UMoverDataModelBlueprintLibrary ///////////////////////////////////////////////////

void UMoverDataModelBlueprintLibrary::SetDirectionalInput(FCharacterDefaultInputs& Inputs, const FVector& DirectionInput)
{
	Inputs.SetMoveInput(EMoveInputType::DirectionalIntent, DirectionInput);
}

void UMoverDataModelBlueprintLibrary::SetVelocityInput(FCharacterDefaultInputs& Inputs, const FVector& VelocityInput)
{
	Inputs.SetMoveInput(EMoveInputType::Velocity, VelocityInput);
}

FVector UMoverDataModelBlueprintLibrary::GetMoveDirectionIntentFromInputs(const FCharacterDefaultInputs& Inputs)
{
	return Inputs.GetMoveInput_WorldSpace();
}

FVector UMoverDataModelBlueprintLibrary::GetLocationFromSyncState(const FMoverDefaultSyncState& SyncState)
{
	return SyncState.GetLocation_WorldSpace();
}

FVector UMoverDataModelBlueprintLibrary::GetMoveDirectionIntentFromSyncState(const FMoverDefaultSyncState& SyncState)
{
	return SyncState.GetIntent_WorldSpace();
}

FVector UMoverDataModelBlueprintLibrary::GetVelocityFromSyncState(const FMoverDefaultSyncState& SyncState)
{
	return SyncState.GetVelocity_WorldSpace();
}

FVector UMoverDataModelBlueprintLibrary::GetAngularVelocityDegreesFromSyncState(const FMoverDefaultSyncState& SyncState)
{
	return SyncState.GetAngularVelocityDegrees_WorldSpace();
}

FRotator UMoverDataModelBlueprintLibrary::GetOrientationFromSyncState(const FMoverDefaultSyncState& SyncState)
{
	return SyncState.GetOrientation_WorldSpace();
}


//////////////////////////////////////////////////////////////////////////
// FMoverAIInputs

FMoverDataStructBase* FMoverAIInputs::Clone() const
{
	FMoverAIInputs* CopyPtr = new FMoverAIInputs(*this);
	return CopyPtr;
}

bool FMoverAIInputs::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	Super::NetSerialize(Ar, Map, bOutSuccess);

	Ar << RVOVelocityDelta;

	bOutSuccess = true;
	return true;
}

void FMoverAIInputs::ToString(FAnsiStringBuilderBase& Out) const
{
	Super::ToString(Out);

	Out.Appendf("ROVVelDelta: X=%.2f Y=%.2f Z=%.2f\n", RVOVelocityDelta.X, RVOVelocityDelta.Y, RVOVelocityDelta.Z);
}

bool FMoverAIInputs::ShouldReconcile(const FMoverDataStructBase& AuthorityState) const
{
	const FMoverAIInputs& TypedAuthority = static_cast<const FMoverAIInputs&>(AuthorityState);
	return TypedAuthority.RVOVelocityDelta != RVOVelocityDelta;
}

void FMoverAIInputs::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct)
{
	const FMoverAIInputs& TypedFrom = static_cast<const FMoverAIInputs&>(From);
	const FMoverAIInputs& TypedTo = static_cast<const FMoverAIInputs&>(To);

	RVOVelocityDelta = FMath::Lerp(TypedFrom.RVOVelocityDelta, TypedTo.RVOVelocityDelta, Pct);
}

void FMoverAIInputs::Merge(const FMoverDataStructBase& From)
{
}
