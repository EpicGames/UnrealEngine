// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoverGameplayTagLog.h"
#include "MoverLog.h"

void LogGameplayTagDriftDetect(
	const FString& ActorName,
	const TCHAR* RoleStr,
	const FMoverTimeStep& TimeStep,
	const FGameplayTag& Tag,
	bool bWasAdded,
	const TArray<TSharedPtr<FMoverSimulationEventData>>& Events,
	const FGameplayTagContainer& PreviousTags,
	const FGameplayTagContainer& NewTags)
{
	if (!MoverCVars::bEnableGameplayTagLog)
	{
		return;
	}

	UE_LOGF(LogMover, Log, "[GameplayTagLog][GT:DriftDetect] Actor=%ls Role=%ls Frame=%d TimeMs=%.3f IsResim=%d IsFirstResim=%d Tag=%ls Change=%ls Status=MissingEvent EventQueueSize=%d PreviousTags=[%ls] NewTags=[%ls]",
		*ActorName, RoleStr,
		TimeStep.ServerFrame, TimeStep.BaseSimTimeMs,
		(int32)TimeStep.bIsResimulating, (int32)TimeStep.bIsFirstResimFrame,
		*Tag.ToString(), bWasAdded ? TEXT("Added") : TEXT("Removed"),
		Events.Num(),
		*PreviousTags.ToStringSimple(), *NewTags.ToStringSimple());

	for (const TSharedPtr<FMoverSimulationEventData>& E : Events)
	{
		if (!E.IsValid())
		{
			continue;
		}

		const FMoverGameplayTagChangeEventData* TagEvent = E->CastTo<FMoverGameplayTagChangeEventData>();
		if (TagEvent)
		{
			UE_LOGF(LogMover, Log, "[GameplayTagLog][GT:DriftDetect:Event] Actor=%ls Role=%ls Frame=%d TimeMs=%.3f IsResim=%d IsRollback=%d Tag=%ls Change=%ls",
				*ActorName, RoleStr, E->Context.ServerFrame, E->Context.EventTimeMs,
				(int32)E->Context.bIsDuringResimulation, (int32)E->Context.bIsCausedByRollback,
				*TagEvent->Tag.ToString(), TagEvent->bWasAdded ? TEXT("Added") : TEXT("Removed"));
		}
		else
		{
			UE_LOGF(LogMover, Log, "[GameplayTagLog][GT:DriftDetect:Event] Actor=%ls Role=%ls Frame=%d TimeMs=%.3f IsResim=%d IsRollback=%d",
				*ActorName, RoleStr, E->Context.ServerFrame, E->Context.EventTimeMs,
				(int32)E->Context.bIsDuringResimulation, (int32)E->Context.bIsCausedByRollback);
		}
	}
}
