// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoverLog.h"

DEFINE_LOG_CATEGORY(LogMover);