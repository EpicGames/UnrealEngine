// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoverModule.h"

#include "Debug/MoverDebugComponent.h"
#include "HAL/ConsoleManager.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/Pawn.h"
#include "Engine/World.h"

#if WITH_GAMEPLAY_DEBUGGER
#include "GameplayDebugger.h"
#include "Debug/GameplayDebuggerCategory_Mover.h"
#define MOVER_CATEGORY_NAME "Mover"
#endif // WITH_GAMEPLAY_DEBUGGER

#define LOCTEXT_NAMESPACE "FMoverModule"


namespace UE::Mover
{
	int32 DisableDataCopyInPlace = 0;
	static FAutoConsoleVariableRef CVarDisableDataCopyInPlace(
		TEXT("mover.debug.DisableDataCopyInPlace"),
		DisableDataCopyInPlace,
		TEXT("Whether to allow Mover data collections with identical contained struct types to be copied in place, avoiding reallocating memory"),
		ECVF_Default);
}


void FMoverModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module

	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
	TEXT("Mover.LocalPlayer.ShowTrail"),
	TEXT("Toggles showing the players trail according to the mover component. Trail will show previous path and some information on rollbacks. NOTE: this is applied the first local player controller."),
	FConsoleCommandWithWorldAndArgsDelegate::CreateRaw(this, &FMoverModule::ShowTrail),
	ECVF_Cheat
	));
	
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
	TEXT("Mover.LocalPlayer.ShowTrajectory"),
	TEXT("Toggles showing the players trajectory according to the mover component. NOTE: this is applied the first local player controller"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateRaw(this, &FMoverModule::ShowTrajectory),
	ECVF_Cheat
	));
	
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
	TEXT("Mover.LocalPlayer.ShowCorrections"),
	TEXT("Toggles showing corrections that were applied to the actor. Green is the updated position after correction, Red was the position before correction. NOTE: this is applied the first local player controller."),
	FConsoleCommandWithWorldAndArgsDelegate::CreateRaw(this, &FMoverModule::ShowCorrections),
	ECVF_Cheat
	));
	
#if WITH_GAMEPLAY_DEBUGGER
	IGameplayDebugger& GameplayDebuggerModule = IGameplayDebugger::Get();
	GameplayDebuggerModule.RegisterCategory(MOVER_CATEGORY_NAME, IGameplayDebugger::FOnGetCategory::CreateStatic(&FGameplayDebuggerCategory_Mover::MakeInstance));
	GameplayDebuggerModule.NotifyCategoriesChanged();
#endif // WITH_GAMEPLAY_DEBUGGER
}

void FMoverModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.

#if WITH_GAMEPLAY_DEBUGGER
	if (IGameplayDebugger::IsAvailable())
	{
		IGameplayDebugger& GameplayDebuggerModule = IGameplayDebugger::Get();
		GameplayDebuggerModule.UnregisterCategory(MOVER_CATEGORY_NAME);
		GameplayDebuggerModule.NotifyCategoriesChanged();
	}
#endif // WITH_GAMEPLAY_DEBUGGER
}

void FMoverModule::ShowTrajectory(const TArray<FString>& Args, UWorld* World)
{
	if (const APlayerController* PC = World->GetFirstPlayerController())
	{
		APawn* MyPawn = PC->GetPawn();
		if (UMoverDebugComponent* MoverDebugComponent = MyPawn ? Cast<UMoverDebugComponent>(MyPawn->GetComponentByClass(UMoverDebugComponent::StaticClass())) : nullptr)
		{
			MoverDebugComponent->bShowTrajectory = !MoverDebugComponent->bShowTrajectory;
		}
		else
		{
			UMoverDebugComponent* NewMoverDebugComponent = Cast<UMoverDebugComponent>(MyPawn->AddComponentByClass(UMoverDebugComponent::StaticClass(), false, FTransform::Identity, false));
			NewMoverDebugComponent->bShowTrajectory = true;
			NewMoverDebugComponent->bShowTrail = false;
			NewMoverDebugComponent->bShowCorrections = false;
			NewMoverDebugComponent->SetHistoryTracking(1.0f, 20.0f);
		}
	}
}
void FMoverModule::ShowTrail(const TArray<FString>& Args, UWorld* World)
{
	if (const APlayerController* PC = World->GetFirstPlayerController())
	{
		APawn* MyPawn = PC->GetPawn();
		if (UMoverDebugComponent* MoverDebugComponent = MyPawn ? Cast<UMoverDebugComponent>(MyPawn->GetComponentByClass(UMoverDebugComponent::StaticClass())) : nullptr)
		{
			MoverDebugComponent->bShowTrail = !MoverDebugComponent->bShowTrail;
		}
		else
		{
			UMoverDebugComponent* NewMoverDebugComponent = Cast<UMoverDebugComponent>(MyPawn->AddComponentByClass(UMoverDebugComponent::StaticClass(), false, FTransform::Identity, false));
			NewMoverDebugComponent->bShowTrail = true;
			NewMoverDebugComponent->bShowTrajectory = false;
			NewMoverDebugComponent->bShowCorrections = false;
			NewMoverDebugComponent->SetHistoryTracking(1.0f, 20.0f);
		}
	}
}

void FMoverModule::ShowCorrections(const TArray<FString>& Args, UWorld* World)
{
	if (const APlayerController* PC = World->GetFirstPlayerController())
	{
		APawn* MyPawn = PC->GetPawn();
		if (UMoverDebugComponent* MoverDebugComponent = MyPawn ? Cast<UMoverDebugComponent>(MyPawn->GetComponentByClass(UMoverDebugComponent::StaticClass())) : nullptr)
		{
			MoverDebugComponent->bShowCorrections = !MoverDebugComponent->bShowCorrections;
		}
		else
		{
			UMoverDebugComponent* NewMoverDebugComponent = Cast<UMoverDebugComponent>(MyPawn->AddComponentByClass(UMoverDebugComponent::StaticClass(), false, FTransform::Identity, false));
			NewMoverDebugComponent->bShowTrail = false;
			NewMoverDebugComponent->bShowTrajectory = false;
			NewMoverDebugComponent->bShowCorrections = true;
			NewMoverDebugComponent->SetHistoryTracking(1.0f, 20.0f);
		}
	}
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FMoverModule, Mover)