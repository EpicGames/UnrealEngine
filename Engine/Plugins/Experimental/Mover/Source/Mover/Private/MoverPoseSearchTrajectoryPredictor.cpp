// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoverPoseSearchTrajectoryPredictor.h"
 #include "Animation/TrajectoryTypes.h"
 #include "MoverComponent.h"
 #include "MoverDataModelTypes.h"
 #include "MoverSimulationTypes.h"
 #include "MoveLibrary/MovementUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoverPoseSearchTrajectoryPredictor)
 
void UMoverTrajectoryPredictor::Predict(FTransformTrajectory& InOutTrajectory,
	int32 NumPredictionSamples, float SecondsPerPredictionSample, int NumHistorySamples)
{
	if (!MoverComponent)
	{
		UE_LOGF(LogMover, Log, "Calling Predict without a Mover Component. This is invalid and the trajectory will not be modified.");
		return;
	}

	Predict(*MoverComponent, InOutTrajectory, NumPredictionSamples, SecondsPerPredictionSample, NumHistorySamples, static_cast<float>(MoverSamplingFrameRate.AsInterval()), bDisableGravity);
}

void UMoverTrajectoryPredictor::Predict(UMoverComponent& MoverComponent, FTransformTrajectory& InOutTrajectory, int32 NumPredictionSamples, float SecondsPerPredictionSample, int32 NumHistorySamples, float MoverSamplingInterval, bool bDisableGravity)
{
	FMoverPredictTrajectoryParams PredictParams;

	// Important: the sampling frequency of the prediction does not necessarily match the output frequency on the trajectory
	const float LookAheadTime = NumPredictionSamples * SecondsPerPredictionSample;
	const int NumMoverPredictionSamplesRequired =  FMath::FloorToInt32(LookAheadTime / MoverSamplingInterval) + 2;
	
	PredictParams.NumPredictionSamples = NumMoverPredictionSamplesRequired;
	PredictParams.SecondsPerSample = MoverSamplingInterval;
	PredictParams.bUseVisualComponentRoot = true;
	PredictParams.bDisableGravity = bDisableGravity;

	// IMPORTANT! The first sample returned is actually the current state
	TArray<FTrajectorySampleInfo> MoverPredictionSamples = MoverComponent.GetPredictedTrajectory(PredictParams);

	if (InOutTrajectory.Samples.Num() != (NumHistorySamples + 1 + NumPredictionSamples))
	{
		UE_LOGF(LogMover, Warning, "UMoverTrajectoryPredictor::Predict - InOutTrajectory Samples array containing %d items does not match the requirement for %i history samples, 1 current sample, %i predicted samples", InOutTrajectory.Samples.Num(), NumHistorySamples, NumPredictionSamples);
	}
	else
	{
		// Subsample or supersample the mover prediction to the trajectory prediction

		int MoverPredictionSampleIndex = 1; // we start at index 1, since index 0 is actually the current state
		
		// Start at Current position
		float CurrentTime = InOutTrajectory.Samples[NumHistorySamples].TimeInSeconds;	// t == 0 is the last frame, so we need to account for the starting mover state being offset

		// Use actual SimTimeMs from samples for bracket calculation to account for non-uniformly spaced MoverPredictionSamples.
		auto SampleTimeToSeconds = [&](int32 Idx) -> float
		{
			return (MoverPredictionSamples[Idx].SimTimeMs - MoverPredictionSamples[0].SimTimeMs)*0.001f + CurrentTime;
		};

		float TimeInSecondsUpper = SampleTimeToSeconds(MoverPredictionSampleIndex);
		float AccumulatedSeconds = CurrentTime + SecondsPerPredictionSample; // first prediction sample should be at time for the first prediction

		for (int32 i = 0; i < NumPredictionSamples; ++i)
		{
			// Progress target if necessary
			while (AccumulatedSeconds > TimeInSecondsUpper && MoverPredictionSampleIndex < MoverPredictionSamples.Num() - 1)
			{
				MoverPredictionSampleIndex++;

				// Update to next mover prediction value
				TimeInSecondsUpper = SampleTimeToSeconds(MoverPredictionSampleIndex);
			}

			const int PoseSampleIdx = i + NumHistorySamples + 1;

			const float TimeInSecondsLower = SampleTimeToSeconds(MoverPredictionSampleIndex - 1);
			const float DeltaTimeInSeconds = TimeInSecondsUpper - TimeInSecondsLower;

			float T = 0.f;
			if (DeltaTimeInSeconds > UE_SMALL_NUMBER)
			{
				T = (AccumulatedSeconds - TimeInSecondsLower) / DeltaTimeInSeconds;
				T = FMath::Clamp(T, 0.f, 1.f);
			}

			const FTransform& MoverPredictLower = MoverPredictionSamples[MoverPredictionSampleIndex - 1].Transform;
			const FTransform& MoverPredictUpper = MoverPredictionSamples[MoverPredictionSampleIndex].Transform;

			InOutTrajectory.Samples[PoseSampleIdx].Position = FMath::Lerp(MoverPredictLower.GetLocation(), MoverPredictUpper.GetLocation(), T);
			InOutTrajectory.Samples[PoseSampleIdx].Facing = FQuat::Slerp(MoverPredictLower.GetRotation(), MoverPredictUpper.GetRotation(), T);
			InOutTrajectory.Samples[PoseSampleIdx].TimeInSeconds = AccumulatedSeconds;

			AccumulatedSeconds += SecondsPerPredictionSample;
		}
	}
}

void UMoverTrajectoryPredictor::GetGravity(FVector& OutGravityAccel)
{
	if (!MoverComponent)
	{
		UE_LOGF(LogMover, Log, "Calling GetGravity without a Mover Component. Return value will be defaulted.");
		OutGravityAccel = FVector::ZeroVector;
		return;
	}

	OutGravityAccel = MoverComponent->GetGravityAcceleration();
}


void UMoverTrajectoryPredictor::GetCurrentState(FVector& OutPosition, FQuat& OutFacing, FVector& OutVelocity)
{
	if (!MoverComponent)
	{
		UE_LOGF(LogMover, Log, "Calling GetCurrentState without a Mover Component. Return values will be defaulted.");
		OutPosition = OutVelocity = FVector::ZeroVector;
		OutFacing = FQuat::Identity;
		return;
	}

	GetCurrentState(*MoverComponent, OutPosition, OutFacing, OutVelocity);
}

void UMoverTrajectoryPredictor::GetCurrentState(UMoverComponent& MoverComponent, FVector& OutPosition, FQuat& OutFacing, FVector& OutVelocity)
{
	const USceneComponent* VisualComp = MoverComponent.GetPrimaryVisualComponent();

	if (VisualComp)
	{
		OutPosition = VisualComp->GetComponentLocation();
	}
	else
	{
		OutPosition = MoverComponent.GetUpdatedComponentTransform().GetLocation();
	}

	const bool bOrientRotationToMovement = true;

	if (bOrientRotationToMovement)
	{
		if (VisualComp)
		{
			OutFacing = VisualComp->GetComponentRotation().Quaternion();
		}
		else
		{
			OutFacing = MoverComponent.GetUpdatedComponentTransform().GetRotation();
		}
	}
	else
	{
		// JAH TODO: Needs a solve
		//OutFacing = FQuat::MakeFromRotator(FRotator(0, TrajectoryDataState.DesiredControllerYawLastUpdate, 0)) * TrajectoryDataDerived.MeshCompRelativeRotation;
	}

	OutVelocity = MoverComponent.GetVelocity();
}


void UMoverTrajectoryPredictor::GetVelocity(FVector& OutVelocity)
{
	if (!MoverComponent)
	{
		UE_LOGF(LogMover, Log, "Calling GetVelocity without a Mover Component. Return value will be defaulted.");
		OutVelocity = FVector::ZeroVector;
		return;
	}

	OutVelocity = MoverComponent->GetVelocity();
}

void UMoverTrajectoryPredictor::GetAngularVelocity(FVector& OutAngularVelocityDegrees)
{
	OutAngularVelocityDegrees = FVector::ZeroVector;

	if (!MoverComponent)
	{
		UE_LOGF(LogMover, Log, "Calling GetAngularVelocity without a Mover Component. Return value will be defaulted.");
		return;
	}

	const FMoverSyncState& SyncState = MoverComponent->GetSyncState();
	if (const FMoverDefaultSyncState* DefaultSyncState = SyncState.SyncStateCollection.FindDataByType<FMoverDefaultSyncState>())
	{
		OutAngularVelocityDegrees = DefaultSyncState->GetAngularVelocityDegrees_WorldSpace();
	}
}
