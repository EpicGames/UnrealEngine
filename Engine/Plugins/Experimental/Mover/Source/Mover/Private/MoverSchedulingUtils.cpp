// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoverSchedulingUtils.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PBDRigidsSolver.h"

namespace UE::Mover
{

FMoverSchedulingInfo MakeSchedulingInfoForScheduledNetworkTime(const TScriptInterface<IMoverBackendLiaisonInterface>& Backend, const FMoverTimeStep& TimeStep)
{
	FMoverTime ScheduledTime = ensure(Backend) ? Backend->GetScheduledNetworkTime(FMoverTime(TimeStep.ServerFrame, TimeStep.BaseSimTimeMs)) : FMoverTime();
	bool bIsFixedDt = Backend ? Backend->IsFixedDt() : false;
	return FMoverSchedulingInfo(
								/* ServerIssuanceTime  = */ FMoverTime(TimeStep.ServerFrame, TimeStep.BaseSimTimeMs),
								/* ServerExecutionTime = */ FMoverTime(ScheduledTime.FrameCount, ScheduledTime.TimeMs),
								/* bIsFixedDt          = */ bIsFixedDt);
}

FMoverSchedulingInfo MakeSchedulingInfoForNextFrame(const TScriptInterface<IMoverBackendLiaisonInterface>& Backend, const FMoverTimeStep& TimeStep)
{
	bool bIsFixedDt = ensure(Backend) ? Backend->IsFixedDt() : false;

	if (bIsFixedDt)
	{
		const int32 CurrentServerFrame = TimeStep.ServerFrame;
		const int32 ExecutionServerFrame = CurrentServerFrame + 1;

		return FMoverSchedulingInfo(
				/* ServerIssuanceTime  = */ FMoverTime(CurrentServerFrame, 0.0),
				/* ServerExecutionTime = */ FMoverTime(ExecutionServerFrame, 0.0),
				/* bIsFixedDt          = */ bIsFixedDt);
	}
	else
	{
		const double CurrentServerTimeMs = TimeStep.BaseSimTimeMs;
		const double ExecutionServerTimeMs = CurrentServerTimeMs + UE_KINDA_SMALL_NUMBER;

		return FMoverSchedulingInfo(
				/* ServerIssuanceTime  = */ FMoverTime(INDEX_NONE, CurrentServerTimeMs),
				/* ServerExecutionTime = */ FMoverTime(INDEX_NONE, ExecutionServerTimeMs),
				/* bIsFixedDt          = */ bIsFixedDt);
	}
}

FScheduledInstantMovementEffect MakeScheduledEffect(const TScriptInterface<IMoverBackendLiaisonInterface>& Backend, const FMoverTimeStep& TimeStep, TSharedPtr<FInstantMovementEffect> InstantMovementEffect)
{
	ensure(InstantMovementEffect.IsValid());

	return FScheduledInstantMovementEffect(MakeSchedulingInfoForScheduledNetworkTime(Backend, TimeStep), InstantMovementEffect);
}

FScheduledInstantMovementEffect MakeScheduledEffectForNextFrame(const TScriptInterface<IMoverBackendLiaisonInterface>& Backend, const FMoverTimeStep& TimeStep, TSharedPtr<FInstantMovementEffect> InstantMovementEffect)
{
	ensure(InstantMovementEffect.IsValid());

	return FScheduledInstantMovementEffect(MakeSchedulingInfoForNextFrame(Backend, TimeStep), InstantMovementEffect);
}

FScheduledLayeredMove MakeScheduledMove(const TScriptInterface<IMoverBackendLiaisonInterface>& Backend, const FMoverTimeStep& TimeStep, TSharedPtr<FLayeredMoveBase> Move)
{
	ensure(Move.IsValid());

	return FScheduledLayeredMove(MakeSchedulingInfoForScheduledNetworkTime(Backend, TimeStep), Move);
}

FScheduledLayeredMove MakeScheduledMoveForNextFrame(const TScriptInterface<IMoverBackendLiaisonInterface>& Backend, const FMoverTimeStep& TimeStep, TSharedPtr<FLayeredMoveBase> Move)
{
	ensure(Move.IsValid());

	return FScheduledLayeredMove(MakeSchedulingInfoForNextFrame(Backend, TimeStep), Move);
}

} // namespace UE::Mover