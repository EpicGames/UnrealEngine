// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoverSimulation.h"

#include "MoveLibrary/MoverBlackboard.h"
#include "MoveLibrary/RollbackBlackboardWrappers.h"
#include "Misc/TransactionallySafeRWLock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoverSimulation)

UMoverSimulation::UMoverSimulation()
{
	Blackboard = CreateDefaultSubobject<UMoverBlackboard>(TEXT("MoverSimulationBlackboard"));
	RollbackBlackboard = CreateDefaultSubobject<URollbackBlackboard>(TEXT("MoverSimulationRollbackBlackboard"));

	// TODO: this should be set up depending on whether the simulation is real or purely for prediction 
	RollbackBlackboardSimWrapper = FRollbackBlackboardSimWrapper(*RollbackBlackboard, /*bIsPredictive*/ false);
}

const UMoverBlackboard* UMoverSimulation::GetBlackboard() const
{
	return Blackboard;
}

UMoverBlackboard* UMoverSimulation::GetBlackboard_Mutable()
{
	return Blackboard;
}

FRollbackBlackboardSimWrapper UMoverSimulation::GetRollbackBlackboardSimWrapper() const
{
	return RollbackBlackboardSimWrapper;
}

FRollbackBlackboardSimWrapper UMoverSimulation::GetRollbackBlackboardPredictionWrapper() const
{
	return FRollbackBlackboardSimWrapper(*RollbackBlackboard, /*bIsPredictive*/ true);
}

// -----------------------------------------------------------------------------
// Simulation-level locked queue implementations
// -----------------------------------------------------------------------------

void UMoverSimulation::QueueLayeredMove(const FScheduledLayeredMove& Move)
{
	UE::TRWScopeLock Lock(SimQueueLock, SLT_Write);
	QueuedLayeredMoves.Add(Move);
}

void UMoverSimulation::QueueInstantMovementEffect(const FScheduledInstantMovementEffect& Effect)
{
	UE::TRWScopeLock Lock(SimQueueLock, SLT_Write);
	QueuedInstantMovementEffects.Add(Effect);
}

void UMoverSimulation::QueueLayeredMoveInstance(TSharedPtr<FLayeredMoveInstance> MoveInstance)
{
	UE::TRWScopeLock Lock(SimQueueLock, SLT_Write);
	QueuedLayeredMoveInstances.Add(MoveInstance);
}

TArray<FScheduledLayeredMove> UMoverSimulation::ConsumeQueuedLayeredMoves()
{
	UE::TRWScopeLock Lock(SimQueueLock, SLT_Write);
	return MoveTemp(QueuedLayeredMoves);
}

TArray<FScheduledInstantMovementEffect> UMoverSimulation::ConsumeQueuedInstantMovementEffects()
{
	UE::TRWScopeLock Lock(SimQueueLock, SLT_Write);
	return MoveTemp(QueuedInstantMovementEffects);
}

TArray<TSharedPtr<FLayeredMoveInstance>> UMoverSimulation::ConsumeQueuedLayeredMoveInstances()
{
	UE::TRWScopeLock Lock(SimQueueLock, SLT_Write);
	return MoveTemp(QueuedLayeredMoveInstances);
}