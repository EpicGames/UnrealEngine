// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoverSimulationTypes.h"
#include "MoverGameplayTagLog.h"
#include "GameFramework/GameStateBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoverSimulationTypes)

namespace DefaultModeNames
{
	const FName Walking = TEXT("Walking");
	const FName Falling = TEXT("Falling");
	const FName Flying  = TEXT("Flying");
	const FName Swimming  = TEXT("Swimming");
}

namespace CommonBlackboard
{
	const FName LastFloorResult = TEXT("LastFloor");
	const FName LastWaterResult = TEXT("LastWater");

	const FName LastFoundDynamicMovementBase = TEXT("LastFoundDynamicMovementBase");
	const FName AccumulatedBasedTransformDelta = TEXT("AccumulatedBasedTransformDelta");
	const FName LastBasedMovementAppliedEventTime = TEXT("LastBasedMovementAppliedEventTime");

	const FName TimeSinceSupported = TEXT("TimeSinceSupported");

	const FName LastModeChangeRecord = TEXT("LastModeChangeRecord");
}

UScriptStruct* FMoverSimulationEventData::GetScriptStruct() const
{
	checkf(false, TEXT("%hs is being called erroneously. This must be overridden in derived types!"), __FUNCTION__);
	return FMoverSimulationEventData::StaticStruct();
}

namespace UE::Mover
{
	void FSimulationOutputData::Reset()
	{
		SyncState.Reset();
		LastUsedInputCmd.Reset();
		AdditionalOutputData.Empty();
		Events.Empty();
		InternalGameplayTags.Reset();
		bIsValid = false;
	}

	void FSimulationOutputData::Interpolate(const FSimulationOutputData& From, const FSimulationOutputData& To, float Alpha, double SimTimeMs)
	{
		SyncState.Interpolate(&From.SyncState, &To.SyncState, Alpha);
		LastUsedInputCmd.Interpolate(&From.LastUsedInputCmd, &To.LastUsedInputCmd, Alpha);
		AdditionalOutputData.Interpolate(From.AdditionalOutputData, To.AdditionalOutputData, Alpha);
		InternalGameplayTags = From.InternalGameplayTags;
		Events.Empty();
	}

	void FSimulationOutputRecord::FData::Reset()
	{
		TimeStep = FMoverTimeStep();
		SimOutputData.Reset();
	}

	void FSimulationOutputRecord::Add(const FMoverTimeStep& InTimeStep, const FSimulationOutputData& InData)
	{
		CurrentIndex = (CurrentIndex + 1) % 2;
		Data[CurrentIndex] = { InTimeStep, InData };

		// Transfer events to local Events array
		for (TSharedPtr<FMoverSimulationEventData>& Event : Data[CurrentIndex].SimOutputData.Events)
		{
			Events.Add(Event);
		}
		Data[CurrentIndex].SimOutputData.Events.Empty();
		NextInterpolatedGeneration++;
	}

	const FSimulationOutputData& FSimulationOutputRecord::GetLatest() const
	{
		return Data[CurrentIndex].SimOutputData;
	}

	void FSimulationOutputRecord::LogTagInterpolate(uint8 PrevIndex, double AtBaseTimeMs, bool bIsInterpolated) const
	{
		if (!MoverCVars::bEnableGameplayTagLog)
		{
			return;
		}

		MOVER_TAG_LOG(LogMover, "[GT:Interpolate] Actor=%s Role=%s Frame=%d..%d TimeMs=%.3f IsInterpolated=%d PrevIsResim=%d CurrIsResim=%d | PendingEvents=%d",
			*DebugOwnerName, *DebugOwnerRole, Data[PrevIndex].TimeStep.ServerFrame, Data[CurrentIndex].TimeStep.ServerFrame, AtBaseTimeMs, (int32)bIsInterpolated, (int32)Data[PrevIndex].TimeStep.bIsResimulating, (int32)Data[CurrentIndex].TimeStep.bIsResimulating, Events.Num());

		for (const TSharedPtr<FMoverSimulationEventData>& PendingEvt : Events)
		{
			if (!PendingEvt.IsValid())
			{
				continue;
			}
			if (const FMoverGameplayTagChangeEventData* TagEvt = PendingEvt->CastTo<FMoverGameplayTagChangeEventData>())
			{
				MOVER_TAG_LOG(LogMover, "[GT:Interpolate:PendingEvent] Actor=%s Role=%s Frame=%d TimeMs=%.3f IsResim=%d IsRollback=%d Tag=%s Change=%s",
					*DebugOwnerName, *DebugOwnerRole, PendingEvt->Context.ServerFrame, PendingEvt->Context.EventTimeMs, (int32)PendingEvt->Context.bIsDuringResimulation, (int32)PendingEvt->Context.bIsCausedByRollback, *TagEvt->Tag.ToString(), TagEvt->bWasAdded ? TEXT("Added") : TEXT("Removed"));
			}
		}
	}

	void FSimulationOutputRecord::CreateInterpolatedResult(double AtBaseTimeMs, FMoverTimeStep& OutTimeStep, FSimulationOutputData& OutData)
	{
		const uint8 PrevIndex = (CurrentIndex + 1) % 2;
		const double PrevTimeMs = Data[PrevIndex].TimeStep.BaseSimTimeMs;
		const double CurrTimeMs = Data[CurrentIndex].TimeStep.BaseSimTimeMs;

		bool bIsInterpolated = false;

		if (FMath::IsNearlyEqual(PrevTimeMs, CurrTimeMs) || (AtBaseTimeMs >= CurrTimeMs))
		{
			OutData = Data[CurrentIndex].SimOutputData;
			OutTimeStep = Data[CurrentIndex].TimeStep;
		}
		else if (AtBaseTimeMs <= PrevTimeMs)
		{
			OutData = Data[PrevIndex].SimOutputData;
			OutTimeStep = Data[PrevIndex].TimeStep;
		}
		else
		{
			const float Alpha = FMath::Clamp((AtBaseTimeMs - PrevTimeMs) / (CurrTimeMs - PrevTimeMs), 0.0f, 1.0f);
			OutData.Interpolate(Data[PrevIndex].SimOutputData, Data[CurrentIndex].SimOutputData, Alpha, AtBaseTimeMs);
			OutTimeStep = Data[PrevIndex].TimeStep;
			bIsInterpolated = true;
		}

		OutTimeStep.BaseSimTimeMs = AtBaseTimeMs;

		// Mover Gameplay Tag Logging
		LogTagInterpolate(PrevIndex, AtBaseTimeMs, bIsInterpolated);

		// If we are re-entering the same interpolation window (same NextInterpolatedGeneration -- meaning
		// Add() has not been called since the last visit), start from the accumulated
		// post-event tag state rather than the original From snapshot. This prevents
		// SetSimulationOutput from seeing a stale InternalGameplayTags (the From state)
		// after events have already been consumed, which would trigger spurious drift
		// detection for tags that were correctly dispatched in a prior call.
		// Otherwise (first visit, or bIsInterpolated is false), OutData.InternalGameplayTags
		// already holds Data[PrevIndex].SimOutputData.InternalGameplayTags (the From snapshot),
		// set by FSimulationOutputData::Interpolate, and events are applied on top of it below.
		if (bIsInterpolated && LastInterpolatedGeneration == NextInterpolatedGeneration)
		{
			OutData.InternalGameplayTags = LastInterpolatedTags;
		}

		for (TSharedPtr<FMoverSimulationEventData>& Event : Events)
		{
			if (!Event.IsValid())
			{
				continue;
			}

			if (Event->Context.EventTimeMs <= AtBaseTimeMs)
			{
				if (bIsInterpolated && Event->Context.EventTimeMs > PrevTimeMs)
				{
					if (const FMoverGameplayTagChangeEventData* TagStateChangeEvent = Event->CastTo<FMoverGameplayTagChangeEventData>())
					{
						if (TagStateChangeEvent->bWasAdded)
						{
							UE_CLOGF(OutData.InternalGameplayTags.HasTagExact(TagStateChangeEvent->Tag), LogMover, Verbose,
								"Interpolated gameplay tag add for '%ls' but tag is already present in the From snapshot - possible drift between recorded tag snapshot and event log",
								*TagStateChangeEvent->Tag.ToString());

							OutData.InternalGameplayTags.AddTag(TagStateChangeEvent->Tag);
						}
						else
						{
							UE_CLOGF(!OutData.InternalGameplayTags.HasTagExact(TagStateChangeEvent->Tag), LogMover, Verbose,
								"Interpolated gameplay tag remove for '%ls' but tag is not present in the From snapshot - possible drift between recorded tag snapshot and event log",
								*TagStateChangeEvent->Tag.ToString());

							OutData.InternalGameplayTags.RemoveTag(TagStateChangeEvent->Tag);
						}
					}
				}

				if (const FMoverGameplayTagChangeEventData* TagDispatch = Event->CastTo<FMoverGameplayTagChangeEventData>())
				{
					MOVER_TAG_LOG(LogMover, "[GT:Interpolate:TagEventOutput] Actor=%s Role=%s Frame=%d TimeMs=%.3f Tag=%s Change=%s InternalTags=[%s]",
						*DebugOwnerName, *DebugOwnerRole, Event->Context.ServerFrame, Event->Context.EventTimeMs, *TagDispatch->Tag.ToString(), TagDispatch->bWasAdded ? TEXT("Added") : TEXT("Removed"), *OutData.InternalGameplayTags.ToStringSimple());
				}

				OutData.Events.Add(MoveTemp(Event));
			}
		}

		Events.RemoveAll([](const TSharedPtr<FMoverSimulationEventData> Event) {
			return !Event.IsValid();
			});

		if (bIsInterpolated)
		{
			MOVER_TAG_LOG(LogMover, "[GT:Interpolate:TagState] Actor=%s Role=%s Frame=%d..%d TimeMs=%.3f PrevTags=[%s] InternalTags=[%s]",
				*DebugOwnerName, *DebugOwnerRole, Data[PrevIndex].TimeStep.ServerFrame, Data[CurrentIndex].TimeStep.ServerFrame, AtBaseTimeMs, *LastInterpolatedTags.ToStringSimple(), *OutData.InternalGameplayTags.ToStringSimple());

			LastInterpolatedGeneration = NextInterpolatedGeneration;
			LastInterpolatedTags = OutData.InternalGameplayTags;
		}
	}

	void FSimulationOutputRecord::Clear()
	{
		CurrentIndex = 1;
		Data[0].Reset();
		Data[1].Reset();
		Events.Empty();
		LastInterpolatedGeneration = INDEX_NONE;
		LastInterpolatedTags.Reset();
	}

	void FSimulationOutputRecord::PruneEventsFromFrame(int32 FromServerFrame)
	{
		Events.RemoveAll([FromServerFrame](const TSharedPtr<FMoverSimulationEventData>& Event)
		{
			return Event.IsValid() && Event->Context.ServerFrame >= FromServerFrame;
		});
	}

} // namespace UE::Mover
