// Copyright Epic Games, Inc. All Rights Reserved.


#include "MoverTypes.h"
#include "Blueprint/BlueprintExceptionInfo.h"
#include "MoverLog.h"
#include "MoverModule.h"
#include "StructUtils/UserDefinedStruct.h"
#include "UObject/ObjectKey.h"
#include "UserDefinedStructSupport.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoverTypes)

#define LOCTEXT_NAMESPACE "MoverData"

UE_DEFINE_GAMEPLAY_TAG_COMMENT(Mover_IsOnGround, "Mover.IsOnGround", "Default Mover state flag indicating character is on the ground.");
UE_DEFINE_GAMEPLAY_TAG_COMMENT(Mover_IsInAir, "Mover.IsInAir", "Default Mover state flag indicating character is in the air.");
UE_DEFINE_GAMEPLAY_TAG_COMMENT(Mover_IsFalling, "Mover.IsFalling", "Default Mover state flag indicating character is falling.");
UE_DEFINE_GAMEPLAY_TAG_COMMENT(Mover_IsFlying, "Mover.IsFlying", "Default Mover state flag indicating character is flying.");
UE_DEFINE_GAMEPLAY_TAG_COMMENT(Mover_IsSwimming, "Mover.IsSwimming", "Default Mover state flag indicating character is swimming.");
UE_DEFINE_GAMEPLAY_TAG_COMMENT(Mover_IsCrouching, "Mover.Stance.IsCrouching", "Default Mover state flag indicating character is crouching.");
UE_DEFINE_GAMEPLAY_TAG_COMMENT(Mover_IsNavWalking, "Mover.IsNavWalking", "Default Mover state flag indicating character is NavWalking.");
UE_DEFINE_GAMEPLAY_TAG_COMMENT(Mover_AnimRootMotion, "Mover.AnimRootMotion", "Default Mover state flag indicating some form of animation root motion is active.");
UE_DEFINE_GAMEPLAY_TAG_COMMENT(Mover_SkipAnimRootMotion, "Mover.SkipAnimRootMotion", "Default Mover state flag indicating Animation Root Motion proposed movement should be skipped.");
UE_DEFINE_GAMEPLAY_TAG_COMMENT(Mover_SkipVerticalAnimRootMotion, "Mover.SkipVerticalAnimRootMotion", "Default Mover state flag indicating Animation Root Motion proposed movements should not include a vertical velocity component (along the up/down axis).");
UE_DEFINE_GAMEPLAY_TAG_COMMENT(Mover_DisableLanding, "Mover.DisableLanding", "Default Mover state flag indicating the transition to landing is not allowed.")

FMoverOnImpactParams::FMoverOnImpactParams() 
	: AttemptedMoveDelta(0) 
{
}

FMoverOnImpactParams::FMoverOnImpactParams(const FName& ModeName, const FHitResult& Hit, const FVector& Delta)
	: MovementModeName(ModeName)
	, HitResult(Hit)
	, AttemptedMoveDelta(Delta)
{
}

FMoverEventContext::FMoverEventContext()
{
}

FMoverEventContext::FMoverEventContext(const FMoverTimeStep& InEventTime)
	: EventTimeMs(InEventTime.BaseSimTimeMs)
	, ServerFrame(InEventTime.ServerFrame)
	, bIsDuringResimulation(InEventTime.bIsResimulating)
{
}

FMoverDataStructBase::FMoverDataStructBase()
{
}

FMoverDataStructBase* FMoverDataStructBase::Clone() const
{
	// If child classes don't override this, collections will not work
	checkf(false, TEXT("%hs is being called erroneously on [%s]. This must be overridden in derived types!"), __FUNCTION__, *GetScriptStruct()->GetName());
	return nullptr;
}

UScriptStruct* FMoverDataStructBase::GetScriptStruct() const
{
	checkf(false, TEXT("%hs is being called erroneously. This must be overridden in derived types!"), __FUNCTION__);
	return FMoverDataStructBase::StaticStruct();
}

bool FMoverDataStructBase::ShouldReconcile(const FMoverDataStructBase& AuthorityState) const
{
	checkf(false, TEXT("%hs is being called erroneously on [%s]. This must be overridden in derived types that comprise STATE data (sync/aux) "
					"or INPUT data for use with physics-based movement"), __FUNCTION__, *GetScriptStruct()->GetName());
	return false;
}

void FMoverDataStructBase::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct)
{
	checkf(false, TEXT("%hs is being called erroneously on [%s]. This must be overridden in derived types that comprise STATE data (sync/aux) "
					"or INPUT data for use with physics-based movement"), __FUNCTION__, *GetScriptStruct()->GetName());
}

void FMoverDataStructBase::Merge(const FMoverDataStructBase& From)
{
	checkf(false, TEXT("%hs is being called erroneously on [%s]. This must be overridden in derived types that comprise INPUT data for use with physics-based movement"),
		__FUNCTION__, *GetScriptStruct()->GetName());
}

const UScriptStruct* FMoverDataStructBase::GetDataScriptStruct() const
{ 
	return GetScriptStruct(); 
}


FMoverDataCollection::FMoverDataCollection()
{
}

bool FMoverDataCollection::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	NetSerializeDataArray(Ar, Map, DataArray);

	if (Ar.IsError())
	{
		bOutSuccess = false;
		return false;
	}

	bOutSuccess = true;
	return true;
}

struct FMoverDataDeleter
{
	FORCEINLINE void operator()(FMoverDataStructBase* Object) const
	{
		check(Object);
		UScriptStruct* ScriptStruct = Object->GetScriptStruct();
		check(ScriptStruct);
		ScriptStruct->DestroyStruct(Object);
		FMemory::Free(Object);
	}
};

bool FMoverDataCollection::SerializeDebugData(FArchive& Ar)
{
	// DISCLAIMER: This serialization is not version independent, so it might not be good enough to be used for the Chaos Visual Debugger in the long run

	// First serialize the number of structs in the collection
	int32 NumDataStructs;
	if (Ar.IsLoading())
	{
		Ar << NumDataStructs;
		DataArray.SetNumZeroed(NumDataStructs);
	}
	else
	{
		NumDataStructs = DataArray.Num();
		Ar << NumDataStructs;
	}

	if (Ar.IsLoading())
	{
		DataArray.Empty();
		for (int32 i = 0; i < NumDataStructs && !Ar.IsError(); ++i)
		{
			FString StructName;
			Ar << StructName;
			if (UScriptStruct* MoveDataStruct = Cast<UScriptStruct>(FindObject<UStruct>(nullptr, *StructName)))
			{
				FMoverDataStructBase* NewMoverData = AddDataByType(MoveDataStruct);
				MoveDataStruct->SerializeBin(Ar, NewMoverData);
			}
		}
	}
	else
	{
		for (int32 i = 0; i < DataArray.Num() && !Ar.IsError(); ++i)
		{
			FMoverDataStructBase* MoveDataStruct = DataArray[i].Get();
			if (MoveDataStruct)
			{
				// The FullName of the script struct will be something like "ScriptStruct /Script/Mover.FCharacterDefaultInputs"
				FString FullStructName = MoveDataStruct->GetScriptStruct()->GetFullName(nullptr);
				// We don't need to save the first part since we only ever save UScriptStructs (C++ structs)
				FString StructName = FullStructName.RightChop(13); // So we chop the "ScriptStruct " part (hence 13 characters)
				Ar << StructName;
				MoveDataStruct->GetScriptStruct()->SerializeBin(Ar, MoveDataStruct);
			}
		}
	}

	return true;
}

FMoverDataCollection& FMoverDataCollection::operator=(const FMoverDataCollection& Other)
{
	// Perform deep copy of this Group
	if (this != &Other)
	{
		bool bCanCopyInPlace = (UE::Mover::DisableDataCopyInPlace == 0 && DataArray.Num() == Other.DataArray.Num());
		if (bCanCopyInPlace)
		{
			// If copy in place is enabled and the arrays are the same size, copy by index
			for (int32 i = 0; i < DataArray.Num(); ++i)
			{
				if (FMoverDataStructBase* SrcData = Other.DataArray[i].Get())
				{
					FMoverDataStructBase* DestData = DataArray[i].Get();
					UScriptStruct* SourceStruct = SrcData->GetScriptStruct();

					if (DestData && SourceStruct == DestData->GetScriptStruct())
					{
						// Same type so copy in place
						SourceStruct->CopyScriptStruct(DestData, SrcData, 1);
					}
					else
					{
						// Different type so replace the shared ptr with a clone
						DataArray[i] = TSharedPtr<FMoverDataStructBase>(SrcData->Clone());
					}
				}
				else
				{
					// Found invalid source, fall back to full copy
					bCanCopyInPlace = false;
					break;
				}
			}
		}
		
		if (!bCanCopyInPlace)
		{
			// Deep copy active data blocks
			DataArray.Empty(Other.DataArray.Num());
			for (int i = 0; i < Other.DataArray.Num(); ++i)
			{
				if (Other.DataArray[i].IsValid())
				{
					FMoverDataStructBase* CopyOfSourcePtr = Other.DataArray[i]->Clone();
					DataArray.Add(TSharedPtr<FMoverDataStructBase>(CopyOfSourcePtr));
				}
				else
				{
					UE_LOGF(LogMover, Warning, "FMoverDataCollection::operator= trying to copy invalid Other DataArray element");
				}
			}
		}
	}

	return *this;
}

bool FMoverDataCollection::operator==(const FMoverDataCollection& Other) const
{
	// Deep move-by-move comparison
	if (DataArray.Num() != Other.DataArray.Num())
	{
		return false;
	}

	for (int32 i = 0; i < DataArray.Num(); ++i)
	{
		if (DataArray[i].IsValid() == Other.DataArray[i].IsValid())
		{
			if (DataArray[i].IsValid())
			{
				// TODO: Implement deep equality checks
				// 				if (!DataArray[i]->MatchesAndHasSameState(Other.DataArray[i].Get()))
				// 				{
				// 					return false; // They're valid and don't match/have same state
				// 				}
			}
		}
		else
		{
			return false; // Mismatch in validity
		}
	}

	return true;
}

bool FMoverDataCollection::operator!=(const FMoverDataCollection& Other) const
{
	return !(FMoverDataCollection::operator==(Other));
}


bool FMoverDataCollection::ShouldReconcile(const FMoverDataCollection& Other) const
{
	// Collections must have matching elements, and those elements are piece-wise tested for needing reconciliation
	if (DataArray.Num() != Other.DataArray.Num())
	{
		return true;
	}

	for (int32 i = 0; i < DataArray.Num(); ++i)
	{
		const FMoverDataStructBase* DataElement = DataArray[i].Get();
		const FMoverDataStructBase* OtherDataElement = Other.FindDataByType(DataElement->GetDataScriptStruct());

		// Reconciliation is needed if there's no matching types, or if the element pair needs reconciliation
		if (OtherDataElement == nullptr ||
			DataElement->ShouldReconcile(*OtherDataElement))
		{
			return true;
		}
	}

	return false;
}

void FMoverDataCollection::Interpolate(const FMoverDataCollection& From, const FMoverDataCollection& To, float Pct)
{
	// TODO: Consider an inline allocator to avoid dynamic memory allocations
	TSet<TObjectKey<UScriptStruct>> AddedDataTypes;

	// Piece-wise interpolation of matching data blocks
	for (const TSharedPtr<FMoverDataStructBase>& FromElement : From.DataArray)
	{
		AddedDataTypes.Add(FromElement->GetDataScriptStruct());

		if (const FMoverDataStructBase* ToElement = To.FindDataByType(FromElement->GetDataScriptStruct()))
		{
			FMoverDataStructBase* InterpElement = FindOrAddDataByType(FromElement->GetDataScriptStruct());
			InterpElement->Interpolate(*FromElement, *ToElement, Pct);
		}
		else
		{
			// If only present in From, add the block directly to this collection
			AddDataByCopy(FromElement.Get());
		}
	}

	// Add any types present only in To as well
	for (const TSharedPtr<FMoverDataStructBase>& ToElement : To.DataArray)
	{
		if (!AddedDataTypes.Contains(ToElement->GetDataScriptStruct()))
		{
			AddDataByCopy(ToElement.Get());
		}
	}
}

void FMoverDataCollection::Merge(const FMoverDataCollection& From)
{
	for (const TSharedPtr<FMoverDataStructBase>& FromElement : From.DataArray)
	{
		if (FMoverDataStructBase* ExistingElement = FindDataByType(FromElement->GetDataScriptStruct()))
		{
			ExistingElement->Merge(*FromElement);
		}
		else
		{
			// If only present in the previous block, copy it into this block
			AddDataByCopy(FromElement.Get());
		}
	}
}

void FMoverDataCollection::Decay(float DecayAmount)
{
	for (const TSharedPtr<FMoverDataStructBase>& Element : DataArray)
	{
		Element->Decay(DecayAmount);
	}
}


bool FMoverDataCollection::HasSameContents(const FMoverDataCollection& Other) const
{
	if (DataArray.Num() != Other.DataArray.Num())
	{
		return false;
	}

	for (int32 i = 0; i < DataArray.Num(); ++i)
	{
		if (DataArray[i]->GetDataScriptStruct() != Other.DataArray[i]->GetDataScriptStruct())
		{
			return false;
		}
	}

	return true;
}

void FMoverDataCollection::AddStructReferencedObjects(FReferenceCollector& Collector) const
{
	for (const TSharedPtr<FMoverDataStructBase>& Data : DataArray)
	{
		if (Data.IsValid())
		{
			Data->AddReferencedObjects(Collector);
		}
	}
}

void FMoverDataCollection::ToString(FAnsiStringBuilderBase& Out) const
{
	for (const TSharedPtr<FMoverDataStructBase>& Data : DataArray)
	{
		if (Data.IsValid())
		{
			UScriptStruct* Struct = Data->GetScriptStruct();
			Out.Appendf("\n[%s]\n", TCHAR_TO_ANSI(*Struct->GetName()));
			Data->ToString(Out);
		}
	}
}

TArray<TSharedPtr<FMoverDataStructBase>>::TConstIterator FMoverDataCollection::GetCollectionDataIterator() const
{
	return DataArray.CreateConstIterator();
}

//static 
TSharedPtr<FMoverDataStructBase> FMoverDataCollection::CreateDataByType(const UScriptStruct* DataStructType)
{
	check(DataStructType->IsChildOf(FMoverDataStructBase::StaticStruct()));

	FMoverDataStructBase* NewDataBlock = (FMoverDataStructBase*)FMemory::Malloc(DataStructType->GetCppStructOps()->GetSize());
	DataStructType->InitializeStruct(NewDataBlock);

	return TSharedPtr<FMoverDataStructBase>(NewDataBlock, FMoverDataDeleter());
}


FMoverDataStructBase* FMoverDataCollection::AddDataByType(const UScriptStruct* DataStructType)
{
	if (ensure(!FindDataByType(DataStructType)))
	{
		TSharedPtr<FMoverDataStructBase> NewDataInstance;

		if (DataStructType->IsA<UUserDefinedStruct>())
		{
			NewDataInstance = CreateDataByType(FMoverUserDefinedDataStruct::StaticStruct());
			static_cast<FMoverUserDefinedDataStruct*>(NewDataInstance.Get())->StructInstance.InitializeAs(DataStructType);
		}
		else
		{
			NewDataInstance = CreateDataByType(DataStructType);
		}

		DataArray.Add(NewDataInstance);
		return NewDataInstance.Get();
	}
	
	return nullptr;
}


void FMoverDataCollection::AddOrOverwriteData(const TSharedPtr<FMoverDataStructBase> DataInstance)
{
	RemoveDataByType(DataInstance->GetDataScriptStruct());
	DataArray.Add(DataInstance);
}


void FMoverDataCollection::AddDataByCopy(const FMoverDataStructBase* DataInstanceToCopy)
{
	check(DataInstanceToCopy);

	const UScriptStruct* TypeToMatch = DataInstanceToCopy->GetDataScriptStruct();

	if (FMoverDataStructBase* ExistingMatchingData = FindDataByType(TypeToMatch))
	{
		// Note that we've matched based on the "data" type but we're copying the top-level type (a FMoverDataStructBase subtype)
		const UScriptStruct* MoverDataTypeToCopy = DataInstanceToCopy->GetScriptStruct();
		MoverDataTypeToCopy->CopyScriptStruct(ExistingMatchingData, DataInstanceToCopy, 1);
	}
	else
	{
		DataArray.Add(TSharedPtr<FMoverDataStructBase>(DataInstanceToCopy->Clone()));
	}
}


FMoverDataStructBase* FMoverDataCollection::FindDataByType(const UScriptStruct* DataStructType) const
{
	for (const TSharedPtr<FMoverDataStructBase>& Data : DataArray)
	{
		const UStruct* CandidateStruct = Data->GetDataScriptStruct();
		while (CandidateStruct)
		{
			if (DataStructType == CandidateStruct)
			{
				return Data.Get();
			}

			CandidateStruct = CandidateStruct->GetSuperStruct();
		}
	}

	return nullptr;
}


FMoverDataStructBase* FMoverDataCollection::FindOrAddDataByType(const UScriptStruct* DataStructType)
{
	if (FMoverDataStructBase* ExistingData = FindDataByType(DataStructType))
	{
		return ExistingData;
	}

	return AddDataByType(DataStructType);
}


bool FMoverDataCollection::RemoveDataByType(const UScriptStruct* DataStructType)
{
	int32 IndexToRemove = -1;

	for (int32 i=0; i < DataArray.Num() && IndexToRemove < 0; ++i)
	{
		const UStruct* CandidateStruct = DataArray[i]->GetDataScriptStruct();
		while (CandidateStruct)
		{
			if (DataStructType == CandidateStruct)
			{
				IndexToRemove = i;
				break;
			}

			CandidateStruct = CandidateStruct->GetSuperStruct();
		}
	}

	if (IndexToRemove >= 0)
	{
		DataArray.RemoveAt(IndexToRemove);
		return true;
	}

	return false;
}

/*static*/
void FMoverDataCollection::NetSerializeDataArray(FArchive& Ar, UPackageMap* Map, TArray<TSharedPtr<FMoverDataStructBase>>& DataArray)
{
	uint8 NumDataStructsToSerialize;
	if (Ar.IsSaving())
	{
		NumDataStructsToSerialize = DataArray.Num();
	}

	Ar << NumDataStructsToSerialize;

	if (Ar.IsLoading())
	{
		DataArray.SetNumZeroed(NumDataStructsToSerialize);
	}

	for (int32 i = 0; i < NumDataStructsToSerialize && !Ar.IsError(); ++i)
	{
		TCheckedObjPtr<UScriptStruct> ScriptStruct = DataArray[i].IsValid() ? DataArray[i]->GetScriptStruct() : nullptr;
		UScriptStruct* ScriptStructLocal = ScriptStruct.Get();

		Ar << ScriptStruct;

		if (ScriptStruct.IsValid())
		{
			// Restrict replication to derived classes of FMoverDataStructBase for security reasons:
			// If FMoverDataCollection is replicated through a Server RPC, we need to prevent clients from sending us
			// arbitrary ScriptStructs due to the allocation/reliance on GetCppStructOps below which could trigger a server crash
			// for invalid structs. All provided sources are direct children of FMoverDataStructBase and we never expect to have deep hierarchies
			// so this should not be too costly
			bool bIsDerivedFromBase = false;
			UStruct* CurrentSuperStruct = ScriptStruct->GetSuperStruct();
			while (CurrentSuperStruct)
			{
				if (CurrentSuperStruct == FMoverDataStructBase::StaticStruct())
				{
					bIsDerivedFromBase = true;
					break;
				}
				CurrentSuperStruct = CurrentSuperStruct->GetSuperStruct();
			}

			if (bIsDerivedFromBase)
			{
				if (Ar.IsLoading())
				{
					if (DataArray[i].IsValid() && ScriptStructLocal == ScriptStruct.Get())
					{
						// What we have locally is the same type as we're being serialized into, so we don't need to
						// reallocate - just use existing structure
					}
					else
					{
						// For now, just reset/reallocate the data when loading.
						// Longer term if we want to generalize this and use it for property replication, we should support
						// only reallocating when necessary
						FMoverDataStructBase* NewDataBlock = (FMoverDataStructBase*)FMemory::Malloc(ScriptStruct->GetCppStructOps()->GetSize());
						ScriptStruct->InitializeStruct(NewDataBlock);

						DataArray[i] = TSharedPtr<FMoverDataStructBase>(NewDataBlock, FMoverDataDeleter());
					}
				}

				bool bArrayElementSuccess = false;
				DataArray[i]->NetSerialize(Ar, Map, bArrayElementSuccess);

				if (!bArrayElementSuccess)
				{
					UE_LOGF(LogMover, Error, "FMoverDataCollection::NetSerialize: Failed to serialize ScriptStruct %ls", *ScriptStruct->GetName());
					Ar.SetError();
					break;
				}
			}
			else
			{
				UE_LOGF(LogMover, Error, "FMoverDataCollection::NetSerialize: ScriptStruct not derived from FMoverDataStructBase attempted to serialize.");
				Ar.SetError();
				break;
			}
		}
		else if (ScriptStruct.IsError())
		{
			UE_LOGF(LogMover, Error, "FMoverDataCollection::NetSerialize: Invalid ScriptStruct serialized.");
			Ar.SetError();
			break;
		}
	}

}



void UMoverDataCollectionLibrary::K2_AddDataToCollection(FMoverDataCollection& Collection, const int32& SourceAsRawBytes)
{
	// This will never be called, the exec version below will be hit instead
	checkNoEntry();
}

// static
DEFINE_FUNCTION(UMoverDataCollectionLibrary::execK2_AddDataToCollection)
{
	P_GET_STRUCT_REF(FMoverDataCollection, TargetCollection);

	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(nullptr);

	void* SourceDataAsRawPtr = Stack.MostRecentPropertyAddress;
	FStructProperty* SourceStructProp = CastField<FStructProperty>(Stack.MostRecentProperty);

	P_FINISH;

	if (!SourceDataAsRawPtr || !SourceStructProp)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			LOCTEXT("MoverDataCollection_AddDataToCollection", "Failed to resolve the SourceAsRawBytes for AddDataToCollection")
		);

		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else
	{
		P_NATIVE_BEGIN;

		if (ensure(SourceStructProp->Struct))
		{
			// User-defined struct type support: we wrap an instance inside a FMoverUserDefinedDataStruct
			if (SourceStructProp->Struct->IsA<UUserDefinedStruct>())
			{
				FMoverUserDefinedDataStruct UserDefinedDataWrapper;
				UserDefinedDataWrapper.StructInstance.InitializeAs(SourceStructProp->Struct, (uint8*)SourceDataAsRawPtr);

				TargetCollection.AddDataByCopy(&UserDefinedDataWrapper);
			}
			else if (SourceStructProp->Struct->IsChildOf(FMoverDataStructBase::StaticStruct()))
			{
				FMoverDataStructBase* SourceDataAsBasePtr = reinterpret_cast<FMoverDataStructBase*>(SourceDataAsRawPtr);
				TargetCollection.AddDataByCopy(SourceDataAsBasePtr);
			}
			else
			{
				UE_LOGF(LogMover, Warning, "AddDataToCollection: invalid struct type submitted: %ls", *SourceStructProp->Struct->GetName());
			}
		}

		P_NATIVE_END;
	}
}


void UMoverDataCollectionLibrary::K2_GetDataFromCollection(bool& DidSucceed, const FMoverDataCollection& Collection, int32& TargetAsRawBytes)
{
	// This will never be called, the exec version below will be hit instead
	checkNoEntry();
}

// static
DEFINE_FUNCTION(UMoverDataCollectionLibrary::execK2_GetDataFromCollection)
{
	P_GET_UBOOL_REF(DidSucceed);
	P_GET_STRUCT_REF(FMoverDataCollection, TargetCollection);

	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(nullptr);

	void* TargetDataAsRawPtr = Stack.MostRecentPropertyAddress;
	FStructProperty* TargetStructProp = CastField<FStructProperty>(Stack.MostRecentProperty);

	P_FINISH;

	DidSucceed = false;

	if (!TargetDataAsRawPtr || !TargetStructProp)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			LOCTEXT("MoverDataCollection_GetDataFromCollection_UnresolvedTarget", "Failed to resolve the TargetAsRawBytes for GetDataFromCollection")
		);

		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else if (!TargetStructProp->Struct || 
				(!TargetStructProp->Struct->IsChildOf(FMoverDataStructBase::StaticStruct()) && !TargetStructProp->Struct->IsA<UUserDefinedStruct>()))
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			LOCTEXT("MoverDataCollection_GetDataFromCollection_BadType", "TargetAsRawBytes is not a valid type. Must be a child of FMoverDataStructBase or a User-Defined Struct type.")
		);

		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else
	{
		P_NATIVE_BEGIN;

		if (TargetStructProp->Struct->IsA<UUserDefinedStruct>())
		{
			if (FMoverDataStructBase* FoundDataInstance = TargetCollection.FindDataByType(TargetStructProp->Struct))
			{
				// User-defined struct instances are wrapped in a FMoverUserDefinedDataStruct, so we need to extract the instance memory from inside it
				FMoverUserDefinedDataStruct* FoundBPDataInstance = static_cast<FMoverUserDefinedDataStruct*>(FoundDataInstance);
				TargetStructProp->Struct->CopyScriptStruct(TargetDataAsRawPtr, FoundBPDataInstance->StructInstance.GetMemory());
				DidSucceed = true;
			}
		}
		else
		{
			if (FMoverDataStructBase* FoundDataInstance = TargetCollection.FindDataByType(TargetStructProp->Struct))
			{
				TargetStructProp->Struct->CopyScriptStruct(TargetDataAsRawPtr, FoundDataInstance);
				DidSucceed = true;
			}
		}

		P_NATIVE_END;
	}
}


void UMoverDataCollectionLibrary::ClearDataFromCollection(FMoverDataCollection& Collection)
{
	Collection.Empty();
}

#undef LOCTEXT_NAMESPACE
