// Copyright Epic Games, Inc. All Rights Reserved.


#include "UserDefinedStructSupport.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Engine/Engine.h"
#include "Engine/NetConnection.h"
#include "Engine/NetDriver.h"
#include "Engine/PackageMapClient.h"
#include "Net/RepLayout.h"
#include "MoverLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UserDefinedStructSupport)

#define LOCTEXT_NAMESPACE "MoverUDSInstances"


// TODO: Consider different rules for interpolation/merging/reconciliation checks. 
// This could be accomplished via cvars / Mover settings / per-type metadata , etc.

bool FMoverUserDefinedDataStruct::ShouldReconcile(const FMoverDataStructBase& AuthorityState) const
{
	const FMoverUserDefinedDataStruct& TypedAuthority = static_cast<const FMoverUserDefinedDataStruct&>(AuthorityState);

	check(TypedAuthority.StructInstance.GetScriptStruct() == this->StructInstance.GetScriptStruct());

	return !StructInstance.Identical(&TypedAuthority.StructInstance, EPropertyPortFlags::PPF_DeepComparison);
}

void FMoverUserDefinedDataStruct::Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float LerpFactor)
{
	const FMoverUserDefinedDataStruct& PrimarySource = static_cast<const FMoverUserDefinedDataStruct&>((LerpFactor < 0.5f) ? From : To);

	// copy all properties from the heaviest-weighted source rather than interpolate
	StructInstance = PrimarySource.StructInstance;
}

void FMoverUserDefinedDataStruct::Merge(const FMoverDataStructBase& From)
{
	const FMoverUserDefinedDataStruct& TypedFrom = static_cast<const FMoverUserDefinedDataStruct&>(From);

	check(TypedFrom.StructInstance.GetScriptStruct() == this->StructInstance.GetScriptStruct());

	// Merging is typically only done for inputs. Let's make the assumption that boolean inputs should be OR'd so we never miss any digital inputs.

	if (const UScriptStruct* UdsScriptStruct = TypedFrom.StructInstance.GetScriptStruct())
	{
		uint8* ThisInstanceMemory = StructInstance.GetMutableMemory();
		const uint8* FromInstanceMemory = TypedFrom.StructInstance.GetMemory();

		for (TFieldIterator<FBoolProperty> BoolProperty(UdsScriptStruct); BoolProperty; ++BoolProperty)
		{
			bool bMergedBool = BoolProperty->GetPropertyValue(ThisInstanceMemory);

			if (!bMergedBool)
			{
				bMergedBool |= BoolProperty->GetPropertyValue(FromInstanceMemory);

				if (bMergedBool)
				{
					BoolProperty->SetPropertyValue(ThisInstanceMemory, bMergedBool);
				}
			}
		}
	}
}

FMoverDataStructBase* FMoverUserDefinedDataStruct::Clone() const
{
	FMoverUserDefinedDataStruct* CopyPtr = new FMoverUserDefinedDataStruct(*this);
	return CopyPtr;
}

bool FMoverUserDefinedDataStruct::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	bool bSuperSuccess, bStructSuccess;

	Super::NetSerialize(Ar, Map, bSuperSuccess);
	bStructSuccess = NetSerializeUserDefinedStruct(StructInstance, Ar, Map);

	bOutSuccess = bSuperSuccess && bStructSuccess;

	return true;
}


void FMoverUserDefinedDataStruct::ToString(FAnsiStringBuilderBase& Out) const
{
	Super::ToString(Out);

	// TODO: add property-wise concatenated string output
}

const UScriptStruct* FMoverUserDefinedDataStruct::GetDataScriptStruct() const
{
	return StructInstance.GetScriptStruct();
}


bool FMoverUserDefinedDataStruct::NetSerializeUserDefinedStruct(FInstancedStruct& UDSStructInstance, FArchive& Ar, UPackageMap* Map)
{
	UScriptStruct* NonConstScriptStruct = nullptr;	// This is not modified directly, but needed for some non-const function calls

	if (Ar.IsLoading())
	{
		Ar << NonConstScriptStruct;

		// Initialize only if the type changes.
		if (UDSStructInstance.GetScriptStruct() != NonConstScriptStruct)
		{
			UDSStructInstance.InitializeAs(NonConstScriptStruct);
		}

		if (!UDSStructInstance.IsValid())
		{
			NonConstScriptStruct = nullptr;
			UE_LOGF(LogMover, Error, "FMoverUserDefinedDataStruct::NetSerializeUserDefinedStruct: Bad script struct serialized when loading, cannot recover.");
			Ar.SetError();
		}
	}
	else if (Ar.IsSaving())
	{
		NonConstScriptStruct = const_cast<UScriptStruct*>(UDSStructInstance.GetScriptStruct());
		check(::IsValid(NonConstScriptStruct));
		Ar << NonConstScriptStruct;
	}

	// Check ScriptStruct here, as loading might have failed. 
	if (NonConstScriptStruct)
	{
		check((NonConstScriptStruct->StructFlags & STRUCT_NetSerializeNative) == 0);	// We should never have a native netserialize function with User-Defined Structs

		// Note that we are doing this method of net serialization because FInstancedStruct's NetSerialize
		// does not work with Iris yet. We're also able to assume our struct does not have a native NetSerialize
		// because it's a User-Defined Struct.
		UNetDriver* NetDriver = nullptr;

		if (UPackageMapClient* MapClient = Cast<UPackageMapClient>(Map))	// Non-Iris cases provide a pkg map that lets us get the net driver
		{
			if (UNetConnection* NetConnection = MapClient->GetConnection())
			{
				NetDriver = NetConnection->GetDriver();
			}
		}
		else if (UWorld* World = GEngine->GetCurrentPlayWorld())	// Fall back case when Iris is enabled
		{
			NetDriver = World->GetNetDriver();
		}

		check(::IsValid(NetDriver));

		const TSharedPtr<FRepLayout> RepLayout = NetDriver->GetStructRepLayout(NonConstScriptStruct);
		check(RepLayout.IsValid());

		bool bHasUnmapped = false;
		RepLayout->SerializePropertiesForStruct(NonConstScriptStruct, static_cast<FBitArchive&>(Ar), Map, UDSStructInstance.GetMutableMemory(), bHasUnmapped);
		return true;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
