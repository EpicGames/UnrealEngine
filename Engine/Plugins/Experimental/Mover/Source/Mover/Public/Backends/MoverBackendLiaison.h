// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "MoverSimulationTypes.h"
#include "MoverBackendLiaison.generated.h"

class FDataValidationContext;
class UMoverComponent;

/**
 * MoverBackendLiaisonInterface: any object or system wanting to be the driver of Mover actors must implement this. The intent is to act as a
 * middleman between the Mover actor and the system that drives it, such as the Network Prediction plugin.
 * In practice, objects implementing this interface should be some kind of UActorComponent. The Mover actor instantiates its backend liaison
 * when initialized, then relies on the liaison to call various functions as the simulation progresses. See @MoverComponent.
 */
UINTERFACE(MinimalAPI)
class UMoverBackendLiaisonInterface : public UInterface
{
	GENERATED_BODY()
};

class IMoverBackendLiaisonInterface
{
	GENERATED_BODY()

public:
	virtual double GetCurrentSimTimeMs() = 0;
	virtual int32 GetCurrentSimFrame() = 0;

	// Whether this backend will simulate movement asynchronously
	virtual bool IsAsync() const { return false; }
	// Whether the underlying simulation advances time in fixed increments
	virtual bool IsFixedDt() const { return false; }
	// During a resim, whether this particular mover should resimulate
	virtual bool ShouldResim() const { return true; }

	// How much delay to apply to scheduled events. This is important for networked events, and should be greater than the RTT to ensure the event will be executed on all end points at the same frame.
	virtual float GetEventSchedulingMinDelaySeconds() const {return 0.3f;}
	// Util function that returns a time in the future far enough for all end points to receive a networked message before that time elapses. This is only useful in the context of networked events.
	virtual FMoverTime GetScheduledNetworkTime(const FMoverTime& Time) const
	{
		ensureMsgf(!IsFixedDt(), TEXT("In Fixed Dt it is recommended to also override GetScheduledNetworkTime to return a correct scheduled frame count time"));
		return FMoverTime(INDEX_NONE, Time.TimeMs + GetEventSchedulingMinDelaySeconds() * 1000);
	}

	// Pending State: the simulation state currently being authored
	virtual bool ReadPendingSyncState(OUT FMoverSyncState& OutSyncState) { return false; }
	virtual bool WritePendingSyncState(const FMoverSyncState& SyncStateToWrite) { return false; }
	
	// Presentation State: the most recent presentation state, possibly the result of interpolation or smoothing. Writing to it does not affect the official simulation record.
	virtual bool ReadPresentationSyncState(OUT FMoverSyncState& OutSyncState) { return false; }
	virtual bool WritePresentationSyncState(const FMoverSyncState& SyncStateToWrite) { return false; }

	// Previous Presentation State: the state that our optional smoothing process is moving away from, towards a more recent state. Writing to it does not affect the official simulation record.
	virtual bool ReadPrevPresentationSyncState(OUT FMoverSyncState& OutSyncState) { return false; }
	virtual bool WritePrevPresentationSyncState(const FMoverSyncState& SyncStateToWrite) { return false; }

	/** Called on the game thread when a rollback occurs, after simulation state has been restored.
	 *  NewBaseTimeStep is the frame we are about to re-simulate from.
	 *  PreRollbackTimeStep is the last forward-simulated frame before the rollback -- together they define
	 *  the window of time that was invalidated (mirrors the FMover_OnPostSimRollback delegate).
	 *  NewSyncState is the sync state at NewBaseTimeStep, i.e., the state the simulation has been restored to. */
	virtual void OnSimulationRollback(const FMoverSyncState& NewSyncState, const FMoverTimeStep& NewBaseTimeStep, const FMoverTimeStep& PreRollbackTimeStep) {}

#if WITH_EDITOR
	virtual EDataValidationResult ValidateData(FDataValidationContext& Context, const UMoverComponent& ValidationMoverComp) const { return EDataValidationResult::Valid; }
#endif // WITH_EDITOR

};