// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/ActorComponent.h"
#include "Backends/MoverBackendLiaison.h"
#include "NetworkPredictionComponent.h"
#include "NetworkPredictionSimulation.h"
#include "NetworkPredictionTickState.h"
#include "MovementMode.h"
#include "MoverTypes.h"

#include "MoverNetworkPredictionLiaison.generated.h"

#define UE_API MOVER_API

class UMoverComponent;
class UKinematicActorSimulation;


using KinematicMoverStateTypes = TNetworkPredictionStateTypes<FMoverInputCmdContext, FMoverSyncState, FMoverAuxStateContext>;

/**
 * MoverNetworkPredictionLiaisonComponent: this component acts as a middleman between an actor's Mover component and the Network Prediction plugin.
 * This class is set on a Mover component as the "back end".
 */
UCLASS(MinimalAPI)
class UMoverNetworkPredictionLiaisonComponent : public UNetworkPredictionComponent, public IMoverBackendLiaisonInterface
{
	GENERATED_BODY()

public:
	// Begin NP Driver interface
	// Get latest local input prior to simulation step. Called by Network Prediction system on owner's instance (autonomous or authority).
	UE_API void ProduceInput(const int32 DeltaTimeMS, FMoverInputCmdContext* Cmd);

	// Query whether a rollback / resim should occur based on how different the predicted and authority sync states were. Called by Network Prediction system.
	UE_API bool ShouldReconcile(const FMoverSyncState* PredictedSyncState, const FMoverSyncState* AuthoritySyncState) const;

	// Query whether a rollback / resim should occur based on how different the predicted and authority aux states were. Called by Network Prediction system.
	UE_API bool ShouldReconcile(const FMoverAuxStateContext* PredictedAuxState, const FMoverAuxStateContext* AuthorityAuxState) const;

	// Restore a previous frame prior to resimulating. Called by Network Prediction system.
	UE_API void RestoreFrame(const FMoverSyncState* SyncState, const FMoverAuxStateContext* AuxState);

	// Take output for simulation. Called by Network Prediction system.
	UE_API void FinalizeFrame(const FMoverSyncState* SyncState, const FMoverAuxStateContext* AuxState);

	// Take output for smoothing. Called by Network Prediction system.
	UE_API void FinalizeSmoothingFrame(const FMoverSyncState* SyncState, const FMoverAuxStateContext* AuxState);

	// Seed initial values based on component's state. Called by Network Prediction system.
	UE_API void InitializeSimulationState(FMoverSyncState* OutSync, FMoverAuxStateContext* OutAux);

	// Primary movement simulation update. Given an starting state and timestep, produce a new state. Called by Network Prediction system.
	UE_API void SimulationTick(const FNetSimTimeStep& TimeStep, const TNetSimInput<KinematicMoverStateTypes>& SimInput, const TNetSimOutput<KinematicMoverStateTypes>& SimOutput);
	// End NP Driver interface

	// IMoverBackendLiaisonInterface
	UE_API virtual double GetCurrentSimTimeMs() override;
	UE_API virtual int32 GetCurrentSimFrame() override;
	UE_API virtual bool ReadPendingSyncState(OUT FMoverSyncState& OutSyncState) override;
	UE_API virtual bool WritePendingSyncState(const FMoverSyncState& SyncStateToWrite) override;
	UE_API virtual bool ReadPresentationSyncState(OUT FMoverSyncState& OutSyncState) override;
	UE_API virtual bool WritePresentationSyncState(const FMoverSyncState& SyncStateToWrite) override;
	UE_API virtual bool ReadPrevPresentationSyncState(FMoverSyncState& OutSyncState) override;
	UE_API virtual bool WritePrevPresentationSyncState(const FMoverSyncState& SyncStateToWrite) override;
#if WITH_EDITOR
	UE_API virtual EDataValidationResult ValidateData(FDataValidationContext& Context, const UMoverComponent& ValidationMoverComp) const override;
#endif
	// End IMoverBackendLiaisonInterface

	UE_API virtual void BeginPlay() override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type Reason) override;

	// UObject interface
	UE_API void InitializeComponent() override;
	UE_API void UninitializeComponent() override;
	UE_API void OnRegister() override;
	UE_API void RegisterComponentTickFunctions(bool bRegister) override;
	// End UObject interface

	// UNetworkPredictionComponent interface
	UE_API virtual void InitializeNetworkPredictionProxy() override;
	// End UNetworkPredictionComponent interface


public:
	UE_API UMoverNetworkPredictionLiaisonComponent();



protected:
	friend class UMoverComponent;
	void ResetSimulation();
	void SetupSimulation();

	TObjectPtr<UMoverComponent> MoverComp;	// the component that we're in charge of driving
	FMoverSyncState* StartingOutSync;
	FMoverAuxStateContext* StartingOutAux;

	// Type of simulation to instantiate as the system spins up
	TSubclassOf<UKinematicActorSimulation> SimulationClass;

	UPROPERTY(Transient)
	TObjectPtr<UKinematicActorSimulation> Simulation;
};

#undef UE_API
