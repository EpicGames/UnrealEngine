// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "Backends/MoverBackendLiaison.h"
#include "MoverStandaloneLiaison.generated.h"

#define UE_API MOVER_API

class UMoverSimulation;
class UKinematicActorSimulation;
class UMoverComponent;
class UMoverStandaloneLiaisonComponent;
class AController;





// Tick task for producing input before the next movement simulation step
USTRUCT()
struct FMoverStandaloneProduceInputTickFunction : public FTickFunction
{
	GENERATED_BODY()

	/** Standalone liaison that is the target of this tick **/
	TWeakObjectPtr<UMoverStandaloneLiaisonComponent> Target;

	virtual void ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
	virtual FString DiagnosticMessage() override;
	virtual FName DiagnosticContext(bool bDetailed) override;
};

template<>
struct TStructOpsTypeTraits<FMoverStandaloneProduceInputTickFunction> : public TStructOpsTypeTraitsBase2<FMoverStandaloneProduceInputTickFunction>
{
	enum
	{
		WithCopy = false
	};
};


// Tick task for advancing the movement simulation step, after input has been produced
USTRUCT()
struct FMoverStandaloneSimulateMovementTickFunction : public FTickFunction
{
	GENERATED_BODY()

	/** Standalone liaison that is the target of this tick **/
	TWeakObjectPtr<UMoverStandaloneLiaisonComponent> Target;

	virtual void ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
	virtual FString DiagnosticMessage() override;
	virtual FName DiagnosticContext(bool bDetailed) override;
};

template<>
struct TStructOpsTypeTraits<FMoverStandaloneSimulateMovementTickFunction> : public TStructOpsTypeTraitsBase2<FMoverStandaloneSimulateMovementTickFunction>
{
	enum
	{
		WithCopy = false
	};
};


// Tick task for applying the new simulation state to the actor/components, after movement has been simulated
USTRUCT()
struct FMoverStandaloneApplyStateTickFunction : public FTickFunction
{
	GENERATED_BODY()

	/** Standalone liaison that is the target of this tick **/
	TWeakObjectPtr<UMoverStandaloneLiaisonComponent> Target;

	virtual void ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
	virtual FString DiagnosticMessage() override;
	virtual FName DiagnosticContext(bool bDetailed) override;
};

template<>
struct TStructOpsTypeTraits<FMoverStandaloneApplyStateTickFunction> : public TStructOpsTypeTraitsBase2<FMoverStandaloneApplyStateTickFunction>
{
	enum
	{
		WithCopy = false
	};
};




/**
 * MoverStandaloneLiaison: this component acts as a backend driver for an actor's Mover component, for use in Standalone (non-networked) games.
 * This class is set on a Mover component as the "back end".
 */
UCLASS(MinimalAPI)
class UMoverStandaloneLiaisonComponent : public UActorComponent, public IMoverBackendLiaisonInterface
{
	GENERATED_BODY()

public:
	UE_API UMoverStandaloneLiaisonComponent(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	// IMoverBackendLiaisonInterface
	UE_API virtual double GetCurrentSimTimeMs() override;
	UE_API virtual int32 GetCurrentSimFrame() override;
	UE_API virtual bool ReadPendingSyncState(OUT FMoverSyncState& OutSyncState) override;
	UE_API virtual bool WritePendingSyncState(const FMoverSyncState& SyncStateToWrite) override;
	// End IMoverBackendLiaisonInterface

	// Begin UActorComponent interface
	UE_API virtual void RegisterComponentTickFunctions(bool bRegister) override;
	UE_API virtual void InitializeComponent() override;
	UE_API virtual void UninitializeComponent() override;
	UE_API virtual void BeginPlay() override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type Reason) override;
	// End UActorComponent interface

	UE_API FTickFunction* FindTickFunction(EMoverTickPhase MoverTickPhase);

	/**
	 * Adds a tick dependency between another component and one of mover's tick functions.
	 * @param OtherComponent		The component to add a dependency with.
	 * @param TickOrder				What order OtherComponent's should have relative to TickPhase i.e. OtherComponent Before SimulateMovement.
	 * @param TickPhase				The mover tick phase we want to a dependency with.
	 */
	UFUNCTION(BlueprintCallable, Category = "Ticking", meta = (Keywords = "Tick Dependency"))
	UE_API void AddTickDependency(UActorComponent* OtherComponent, EMoverTickDependencyOrder TickOrder, EMoverTickPhase TickPhase);
	
	/** Sets whether this instance's produce input can run on worker threads or not. See @bUseAsyncProduceInput and @SetEnableProduceInput */
	UFUNCTION(BlueprintCallable, Category = "Ticking")
	UE_API void SetUseAsyncProduceInput(bool bUseAsyncInputProduction);

	UFUNCTION(BlueprintCallable, Category = "Ticking")
	UE_API bool GetUseAsyncProduceInput() const;

	/** Sets whether this instance's produce-input tick will run at all. It may be useful to disable on actors that don't rely on Mover input to move.  */
	UFUNCTION(BlueprintCallable, Category = "Ticking")
	UE_API void SetEnableProduceInput(bool bEnableInputProduction);
	
	/** Whether this instance will have its produce-input tick called. */
	UFUNCTION(BlueprintCallable, Category = "Ticking")
	UE_API bool GetEnableProduceInput() const;
	
	/** Sets whether this instance's movement simulation tick can run on worker threads or not. See @bUseAsyncMovementSimulationTick */
	UFUNCTION(BlueprintCallable, Category = "Ticking")
	UE_API void SetUseAsyncMovementSimulationTick(bool bUseAsyncMovementSim);

	UFUNCTION(BlueprintCallable, Category = "Ticking")
	UE_API bool GetUseAsyncMovementSimulationTick() const;

protected:
	UE_API void TickInputProduction(float DeltaSeconds);
	UE_API void TickMovementSimulation(float DeltaSeconds);
	UE_API void TickApplySimulationState(float DeltaSeconds);

	UE_API void UpdateSimulationTime();

	// Called when controller changes, used to manage ticking dependencies
	UFUNCTION()
	UE_API virtual void OnControllerChanged(APawn* Pawn, AController* OldController, AController* NewController);

protected:
	/**
	 * Sets whether produce input can run on worker threads or not, also gated by global option.
	 * Changes at runtime will take affect next frame. Has no effect on simulation ticking or applying results.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "Async Movement")
	bool bUseAsyncProduceInput = false;
	
	/**
	 * Sets whether the movement simulation tick can run on worker threads or not, also gated by global option.
	 * Changes at runtime will take affect next frame. Has no effect on input production or applying results.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "Async Movement")
	bool bUseAsyncMovementSimulationTick = false;

	// Type of simulation to instantiate as the system spins up
	TSubclassOf<UKinematicActorSimulation> SimulationClass;

	UPROPERTY(Transient)
	TObjectPtr<UKinematicActorSimulation> Simulation;

	TObjectPtr<UMoverComponent> MoverComp;	// the component that we're in charge of driving

	double CurrentSimTimeMs;
	int32 CurrentSimFrame;

	FMoverInputCmdContext LastProducedInputCmd;

	FMoverSyncState CachedLastSyncState;
	FMoverAuxStateContext CachedLastAuxState;
	bool bIsCachedStateDirty = false;	// If true, we need to propagate the state to our MoverComponent during ApplyState

	FRWLock StateDataLock;	// used when reading/writing to our cached state (sync/aux) data

	bool bIsInApplySimulationState = false;	// transient flag indicating ApplySimulationState is active

	FMoverStandaloneProduceInputTickFunction ProduceInputTickFunction;
	FMoverStandaloneSimulateMovementTickFunction SimulateMovementTickFunction;
	FMoverStandaloneApplyStateTickFunction ApplyStateTickFunction;

	friend struct FMoverStandaloneProduceInputTickFunction;
private:
	// Internal-use-only tick data structs, for efficiency since they typically have the same contents from frame to frame
	FMoverTickStartData WorkingStartData;
	FMoverTickEndData WorkingEndData;

	friend struct FMoverStandaloneSimulateMovementTickFunction;
	friend struct FMoverStandaloneApplyStateTickFunction;
};

#undef UE_API
