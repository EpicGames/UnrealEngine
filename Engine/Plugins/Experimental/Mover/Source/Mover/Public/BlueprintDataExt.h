// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoverTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "BlueprintDataExt.generated.h"


#define UE_API MOVER_API

/** Data block containing mappings between names and commonly-used types, so that Blueprint-only devs can include custom data
 * in their project's sync state or input cmds without needing native code changes.
 * EXPERIMENTAL: this will be removed in favor of generic user-defined struct support. If this is for Blueprint usage,
 * consider using FMoverUserDefinedDataStruct instead. If native, consider deriving your own FMoverDataStructBase type.
 */ 
USTRUCT(BlueprintType, Experimental)
struct FMoverDictionaryData : public FMoverDataStructBase
{
	GENERATED_BODY()

public:

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	TMap<FName, bool> BoolValues;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	TMap<FName, int32> IntValues;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	TMap<FName, double> FloatValues;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	TMap<FName, FVector> VectorValues;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	TMap<FName, FRotator> RotatorValues;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	TMap<FName, FName> NameValues;


	virtual FMoverDataStructBase* Clone() const override
	{
		return new FMoverDictionaryData(*this);
	}

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return StaticStruct();
	}

	virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;
	virtual void ToString(FAnsiStringBuilderBase& Out) const override;
	virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Alpha) override;
	virtual void Merge(const FMoverDataStructBase& From) override;
	virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override;

};


#undef UE_API
