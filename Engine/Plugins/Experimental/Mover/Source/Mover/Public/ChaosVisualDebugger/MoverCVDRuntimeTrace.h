// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"

#include "ChaosVisualDebugger/ChaosVDOptionalDataChannel.h"

struct FMoverCVDSimDataWrapper;
struct FMoverInputCmdContext;
struct FMoverSyncState;
struct FMoverDataCollection;
class UMoverComponent;


namespace UE::MoverUtils
{

typedef TArray<TPair<FName, const FMoverDataCollection*>> NamedDataCollections;

#if WITH_CHAOS_VISUAL_DEBUGGER

/** Utility functions used to trace Mover data into the Chaos Visual Debugger */
class FMoverCVDRuntimeTrace
{
public:
	MOVER_API static void TraceMoverData(UMoverComponent* MoverComponent, const FMoverInputCmdContext* InputCmd, const FMoverSyncState* SyncState, const NamedDataCollections* LocalSimDataCollections = nullptr);
	MOVER_API static void TraceMoverData(uint32 SolverID, uint32 ParticleID, const FMoverInputCmdContext* InputCmd, const FMoverSyncState* SyncState, const NamedDataCollections* LocalSimDataCollections = nullptr);

	MOVER_API static void UnwrapSimData(const FMoverCVDSimDataWrapper& InSimDataWrapper, TSharedPtr<FMoverInputCmdContext>& OutInputCmd, TSharedPtr<FMoverSyncState>& OutSyncState, TArray<TPair<FName, TSharedPtr<FMoverDataCollection>>>& OutLocalSimDataSections);
	MOVER_API static void WrapSimData(uint32 SolverID, uint32 ParticleID, const FMoverInputCmdContext& InInputCmd, const FMoverSyncState& InSyncState, const NamedDataCollections* LocalSimDataCollections, FMoverCVDSimDataWrapper& OutSimDataWrapper);

private:
	static void TraceMoverDataPrivate(uint32 SolverID, uint32 ParticleID, const FMoverInputCmdContext* InputCmd, const FMoverSyncState* SyncState, const NamedDataCollections* LocalSimDataCollections = nullptr);
};

// This is all mover data that is networked, either input command (client to server) or sync state (server to client)
CVD_DECLARE_OPTIONAL_DATA_CHANNEL_EXTERN(MoverNetworkedData, MOVER_API)
// This is additional mover data, local to each end point's simulation
CVD_DECLARE_OPTIONAL_DATA_CHANNEL_EXTERN(MoverLocalSimData, MOVER_API)

#else

// Noop implementation in case this is compiled without Chaos Visual Debugger support (e.g. shipping)
class FMoverCVDRuntimeTrace
{
public:
	static void TraceMoverData(UMoverComponent* MoverComponent, const FMoverInputCmdContext* InputCmd, const FMoverSyncState* SyncState, const NamedDataCollections* FMoverLocalSimState = nullptr) {}
	static void TraceMoverData(uint32 SolverID, uint32 ParticleID, const FMoverInputCmdContext* InputCmd, const FMoverSyncState* SyncState, const NamedDataCollections* FMoverLocalSimState = nullptr) {}
	static void UnwrapSimData(const FMoverCVDSimDataWrapper& InSimDataWrapper, TSharedPtr<FMoverInputCmdContext>& OutInputCmd, TSharedPtr<FMoverSyncState>& OutSyncState, TArray<TPair<FName, TSharedPtr<FMoverDataCollection>>>& OutLocalSimDataSections) {}
	static void WrapSimData(uint32 SolverID, uint32 ParticleID, const FMoverInputCmdContext& InInputCmd, const FMoverSyncState& InSyncState, const NamedDataCollections* LocalSimDataCollections, FMoverCVDSimDataWrapper& OutSimDataWrapper) {}
};

#endif // WITH_CHAOS_VISUAL_DEBUGGER

} // namespace UE::MoverUtils