// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "Containers/CircularBuffer.h"
#include "Containers/RingBuffer.h"
#include "CoreMinimal.h"

#include "MoverDebugComponent.generated.h"

#define UE_API MOVER_API

struct FMoverAuxStateContext;
struct FMoverSyncState;
struct FMoverTimeStep;

/** Component to help display debug information for actors using the Mover Component. Is expected to be attached to the actor that has the mover component.
 *  Currently supports trajectory functionality and trailing functionality. Is also used in the gameplay debugger under the mover category. */
UCLASS(MinimalAPI, BlueprintType, meta=(BlueprintSpawnableComponent))
class UMoverDebugComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	// Begin UActorComponent interface
	UE_API UMoverDebugComponent();

	UE_API virtual void InitializeComponent() override;

	UE_API virtual void BeginPlay() override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	// End UActorComponent interface

	// Begin Trajectory Debug
	/** Number of seconds to lookahead and show the current trajectory */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MoverDebug)
	float LookaheadSeconds = 1.0f;

	/** Number of times trajectory will be sampled each second */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MoverDebug)
	int32 LookaheadSamplesPerSecond = 20;
	// End Trajectory Debug

	// Begin MoverDebugger component
	/** Whether this component should show the trajectory of the movement component of the Actor it's attached too */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MoverDebug)
	bool bShowTrajectory = true;

	/** Whether this component should show the trail of the movement component of the Actor it's attached too */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MoverDebug)
	bool bShowTrail = true;

	/** Whether this component should show the corrections and rollbacks applied to the Actor it's attached too */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MoverDebug)
	bool bShowCorrections = true;

	// Draw network corrections. Uses 
	UE_API void DrawCorrections();
	
	// Draw the trail behind the actor
	UE_API void DrawTrail();

	// Draw the current trajectory of the actor
	UE_API void DrawTrajectory();
	// End MoverDebugger component
	
	// Begin Motion history tracking
	/** Change history tracking feature settings. Anything <= 0 for SecondsToTrack will disable the feature. */
    UFUNCTION(BlueprintCallable, Category = Mover)
    UE_API void SetHistoryTracking(float SecondsToTrack, float SamplesPerSecond);

	/** Get a read-only sampling of where the actor has recently been, ordered by ascending age. Will be empty unless history tracking is enabled. @see SetHistoryTracking */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category=Mover)
	UE_API TArray<FTrajectorySampleInfo> GetPastTrajectory() const;
	// End Motion history tracking

protected:

	// Begin Motion history tracking
	// Max age of tracked samples. Feature is disabled for values <= 0. @see SetHistoryTracking
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Mover)
	float HistoryTrackingSeconds = 0.f;

	// Target sampling rate for history tracking. Note that sampling rate is variable and may be higher during times of rapid change. @see SetHistoryTracking
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Mover)
	int32 HistorySamplesPerSecond = 15;

	UE_API void InitHistoryTracking();
	UE_API void UpdateHistoryTrackingForFrame(const FMoverTimeStep& TimeStep, const FMoverSyncState* SyncState, const FMoverAuxStateContext* AuxState);

	UFUNCTION()
	UE_API void OnHistoryTrackingRollback(const FMoverTimeStep& NewTimeStep, const FMoverTimeStep& InvalidatedTimeStep);

	bool bIsTrackingHistory = false;	// Master flag for whether trajectory history is being tracked
	TRingBuffer<FTrajectorySampleInfo> HistorySamples;	// Running buffer of most recent trajectory
	// End Motion history tracking
	 
private:
	// Begin Trail simulation
	UFUNCTION()
	void OnMovementSimTick(const FMoverTimeStep& TimeStep);

	UFUNCTION()
	void OnMovementSimRollback(const FMoverTimeStep& NewTimeStep, const FMoverTimeStep& InvalidatedTimeStep);
	
	struct FTrailSample
	{
		FVector Location = FVector::ZeroVector;
		int32 SimTimeMS = -1;
		int32 SimFrame = -1;
		float GameTimeSecs = -1.f;
	};
	
	TUniquePtr<TCircularBuffer<FTrailSample>> SimulatedSamples;
	TUniquePtr<TCircularBuffer<FTrailSample>> RolledBackSamples;

	int32 NumSimulatedSamplesToBuffer = 300;
	int32 NumRolledBackSamplesToBuffer = 100;

	float OldestSampleToRenderByGameSecs = 2.f;

	int32 FrameOfLastSample = -1;
	int32 HighestRolledBackFrame = -1;
	bool bHasValidRollbackSamples = false;
	// End Trail simulation

	// Begin ShowCorrections
	TArray<FVector> ClientLocations;
	TArray<FVector> CorrectedLocations;
	// End ShowCorrections
};

#undef UE_API
