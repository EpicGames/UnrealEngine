// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoverComponent.h"
#include "DefaultMovementSet/LayeredMoves/MontageStateProvider.h"
#include "MovementModifiers/StanceModifier.h"
#include "CharacterMoverComponent.generated.h"

#define UE_API MOVER_API

/**
 * Fires when a stance is changed, if stance handling is enabled (see @SetHandleStanceChanges)
 * Note: If a stance was just Activated it will fire with an invalid OldStance
 *		 If a stance was just Deactivated it will fire with an invalid NewStance
 */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FMover_OnStanceChanged, EStanceMode, OldStance, EStanceMode, NewStance);



/** 
* Character Mover Component: this is a specialization of the core Mover Component that is set up with a 
* classic character in mind. Defaults and extended functionality, such as jumping and simple montage replication, 
* are intended to support features similar to UE's ACharacter actor type.
*/
UCLASS(MinimalAPI, BlueprintType, Blueprintable, meta = (BlueprintSpawnableComponent))
class UCharacterMoverComponent : public UMoverComponent
{
	GENERATED_BODY()
	
public:
	UE_API UCharacterMoverComponent();
	
	UE_API virtual void BeginPlay() override;


	// Returns whether this component is tasked with handling jump input or not
	UFUNCTION(BlueprintGetter)
	UE_API bool GetHandleJump() const;

	// If true, this component will handle default character inputs for jumping
	UFUNCTION(BlueprintSetter)
	UE_API void SetHandleJump(bool bInHandleJump);

	// Returns whether this component is tasked with handling character stance changes, including crouching
	UFUNCTION(BlueprintGetter)
	UE_API bool GetHandleStanceChanges() const;

	// If true, this component will process stancing changes and crouching inputs
	UFUNCTION(BlueprintSetter)
	UE_API void SetHandleStanceChanges(bool bInHandleStanceChanges);

	/** Returns true if currently crouching */ 
	UFUNCTION(BlueprintCallable, Category = Mover)
	UE_API virtual bool IsCrouching() const;

	/** Returns true if currently flying (moving through a non-fluid volume without resting on the ground) */
	UFUNCTION(BlueprintPure, Category = Mover)
	UE_API virtual bool IsFlying() const;
	
	// Is this actor in a falling state? Note that this includes upwards motion induced by jumping.
	UFUNCTION(BlueprintPure, Category = Mover)
	UE_API virtual bool IsFalling() const;

	// Is this actor in a airborne state? (e.g. Flying, Falling)
	UFUNCTION(BlueprintPure, Category = Mover)
	UE_API virtual bool IsAirborne() const;

	// Is this actor in a grounded state? (e.g. Walking)
	UFUNCTION(BlueprintPure, Category = Mover)
	UE_API virtual bool IsOnGround() const;

	// Is this actor in a Swimming state? (e.g. Swimming)
	UFUNCTION(BlueprintPure, Category = Mover)
	UE_API virtual bool IsSwimming() const;
	
	// Is this actor sliding on an unwalkable slope
	UFUNCTION(BlueprintPure, Category = Mover)
	UE_API virtual bool IsSlopeSliding() const;

	// Can this Actor jump?
	UFUNCTION(BlueprintPure, Category = Mover)
	UE_API virtual bool CanActorJump() const;

	// Perform jump on actor
	UFUNCTION(BlueprintCallable, Category = Mover)
	UE_API virtual bool Jump();
	
	// Whether this actor can currently crouch or not 
	UFUNCTION(BlueprintCallable, Category = Mover)
	UE_API virtual bool CanCrouch();
	
	// Perform crouch on actor
	UFUNCTION(BlueprintCallable, Category = Mover)
	UE_API virtual void Crouch();

	// Perform uncrouch on actor
	UFUNCTION(BlueprintCallable, Category = Mover)
	UE_API virtual void UnCrouch();

	// Get Crouch Intent
	UFUNCTION(BlueprintCallable, Category = Mover)
	bool GetCrouchIntent() const { return bWantsToCrouch; };

	// Broadcast when this actor changes stances.
	UPROPERTY(BlueprintAssignable, Category = Mover)
	FMover_OnStanceChanged OnStanceChanged;
	
protected:
	UFUNCTION()
	UE_API virtual void OnMoverPreSimulationTick(const FMoverTimeStep& TimeStep, const FMoverInputCmdContext& InputCmd);

	UFUNCTION()
	UE_API virtual void OnMoverPostFinalize(const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState);

	UE_API virtual void OnHandlerSettingChanged();

	UE_API virtual void UpdateSyncedMontageState(const FMoverTimeStep& TimeStep, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState);

	// ID used to keep track of the modifier responsible for crouching
	FMovementModifierHandle StanceModifierHandle;

	/** If true, try to crouch (or keep crouching) on next update. If false, try to stop crouching on next update. */
	UPROPERTY(BlueprintReadOnly, Transient, Category = "Mover|Crouch")
	uint8 bWantsToCrouch : 1 = 0;

	// Whether this component should directly handle jumping or not 
	UPROPERTY(EditAnywhere, BlueprintGetter = GetHandleJump, BlueprintSetter = SetHandleJump, Category = "Mover|Character")
	uint8 bHandleJump : 1 = 1;

	// Whether this component should directly handle stance changes, including crouching input
	UPROPERTY(EditAnywhere, BlueprintGetter = GetHandleStanceChanges, BlueprintSetter = SetHandleStanceChanges, Category = "Mover|Character")
	uint8 bHandleStanceChanges : 1 = 1;

	/** Current state of replicated montage playback from an active movement mechanism (layered move, etc.) */
	UPROPERTY(Transient)
	FMoverAnimMontageState SyncedMontageState;
};

#undef UE_API
