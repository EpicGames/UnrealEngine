// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoverSimulationTypes.h"
#include "MoveLibrary/FloorQueryUtils.h"

#include "CharacterMoverSimulationTypes.generated.h"

USTRUCT()
struct FLandedEventData : public FMoverSimulationEventData
{
	GENERATED_BODY()

	FLandedEventData(const FMoverTimeStep& InEventTime, const FHitResult& InHitResult, const FName InNewModeName)
		: FMoverSimulationEventData(InEventTime)
		, HitResult(InHitResult)
		, NewModeName(InNewModeName)
	{
	}
	FLandedEventData() {}

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return FLandedEventData::StaticStruct();
	}

	FHitResult HitResult;
	FName NewModeName = NAME_None;
};

USTRUCT()
struct FJumpedEventData : public FMoverSimulationEventData
{
	GENERATED_BODY()

	FJumpedEventData(const FMoverTimeStep& InEventTime, float InJumpStartHeight)
		: FMoverSimulationEventData(InEventTime)
		, JumpStartHeight(InJumpStartHeight)
	{
	}
	FJumpedEventData() {}

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return FJumpedEventData::StaticStruct();
	}

	float JumpStartHeight = 0.0f;
};

USTRUCT(BlueprintType)
struct FFloorResultData : public FMoverDataStructBase
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadWrite, Category = Mover)
	FFloorCheckResult FloorResult;

	FFloorResultData() = default;
	virtual ~FFloorResultData() = default;

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return StaticStruct();
	}

	virtual FMoverDataStructBase* Clone() const override
	{
		return new FFloorResultData(*this);
	}

	MOVER_API virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;
	MOVER_API virtual void ToString(FAnsiStringBuilderBase& Out) const override;
	MOVER_API virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override;
	MOVER_API virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override;
	MOVER_API virtual void Merge(const FMoverDataStructBase& From) override;
	MOVER_API virtual void Decay(float DecayAmount) override;
};

template<>
struct TStructOpsTypeTraits< FFloorResultData > : public TStructOpsTypeTraitsBase2< FFloorResultData >
{
	enum
	{
		WithCopy = true
	};
};