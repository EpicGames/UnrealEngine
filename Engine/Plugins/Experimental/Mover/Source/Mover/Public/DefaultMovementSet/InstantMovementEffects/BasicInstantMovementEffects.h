// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "InstantMovementEffect.h"
#include "BasicInstantMovementEffects.generated.h"

#define UE_API MOVER_API

/** Teleport: instantly moves an actor to a new location and rotation
 * For async-compatible teleportation, use AsyncTeleportEffect instead.
 */
USTRUCT(BlueprintType, Category = "Mover|Instant Movement Effect", DisplayName = "Teleport Instant Movement Effect")
struct FTeleportEffect : public FInstantMovementEffect
{
	GENERATED_BODY()

	UE_API FTeleportEffect();
	virtual ~FTeleportEffect() = default;

	// Location to teleport to, in world space
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FVector TargetLocation;

	// Whether this teleport effect should keep the actor's current rotation or use a specified one (TargetRotation)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	bool bUseActorRotation;
	
	// Actor rotation is set to this value on teleport if bUseActorRotation is false
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover, meta=(EditCondition="bUseActorRotation==false", EditConditionHides))
	FRotator TargetRotation;
	
	UE_API virtual bool ApplyMovementEffect(FApplyMovementEffectParams& ApplyEffectParams, FMoverSyncState& OutputState) override;
	UE_API virtual bool ApplyMovementEffect_Async(FApplyMovementEffectParams_Async& ApplyEffectParams, FMoverSyncState& OutputState) override;

	UE_API virtual FInstantMovementEffect* Clone() const override;

	UE_API virtual void NetSerialize(FArchive& Ar) override;

	UE_API virtual UScriptStruct* GetScriptStruct() const override;

	UE_API virtual FString ToSimpleString() const override;

	UE_API virtual void AddReferencedObjects(class FReferenceCollector& Collector) override;
};

/** Async Teleport: instantly moves an actor to a new location and rotation (compatible with async movement simulation) */
USTRUCT(BlueprintType, Category = "Mover|Instant Movement Effect", DisplayName = "Async Teleport Instant Movement Effect")
struct FAsyncTeleportEffect : public FTeleportEffect
{
	GENERATED_BODY()

	UE_API virtual bool ApplyMovementEffect(FApplyMovementEffectParams& ApplyEffectParams, FMoverSyncState& OutputState) override;

	UE_API virtual FInstantMovementEffect* Clone() const override;

	UE_API virtual UScriptStruct* GetScriptStruct() const override;

	UE_API virtual FString ToSimpleString() const override;
};

/** Jump Impulse: introduces an instantaneous upwards change in velocity. This overrides the existing 'up' component of the actor's current velocity
  * Note: this only applies the impulse for one tick!
  */
USTRUCT(BlueprintType, Category = "Mover|Instant Movement Effect", DisplayName = "Jump Impulse Instant Movement Effect")
struct FJumpImpulseEffect : public FInstantMovementEffect
{
	GENERATED_BODY()

	UE_API FJumpImpulseEffect();

	virtual ~FJumpImpulseEffect() = default;

	// Units per second, in whatever direction the target actor considers 'up'
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover, meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s"))
	float UpwardsSpeed;

	UE_API virtual bool ApplyMovementEffect(FApplyMovementEffectParams& ApplyEffectParams, FMoverSyncState& OutputState) override;

	UE_API virtual FInstantMovementEffect* Clone() const override;

	UE_API virtual void NetSerialize(FArchive& Ar) override;

	UE_API virtual UScriptStruct* GetScriptStruct() const override;

	UE_API virtual FString ToSimpleString() const override;

	UE_API virtual void AddReferencedObjects(class FReferenceCollector& Collector) override;
};

/** Apply Velocity: provides an impulse velocity to the actor after (optionally) forcing them into a particular movement mode
  * Note: this only applies the impulse for one tick!
  */
USTRUCT(BlueprintType, Category = "Mover|Instant Movement Effect", DisplayName = "Apply Velocity Instant Movement Effect")
struct FApplyVelocityEffect : public FInstantMovementEffect
{
	GENERATED_BODY()

	UE_API FApplyVelocityEffect();
	virtual ~FApplyVelocityEffect() = default;

	// Velocity to apply to the actor.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover, meta=(ForceUnits="cm/s"))
	FVector VelocityToApply;

	// If true VelocityToApply will be added to current velocity on this actor. If false velocity will be set directly to VelocityToApply
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	bool bAdditiveVelocity;
	
	// Optional movement mode name to force the actor into before applying the impulse velocity.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FName ForceMovementMode;

	UE_API virtual bool ApplyMovementEffect(FApplyMovementEffectParams& ApplyEffectParams, FMoverSyncState& OutputState) override;
	
	UE_API virtual FInstantMovementEffect* Clone() const override;

	UE_API virtual void NetSerialize(FArchive& Ar) override;

	UE_API virtual UScriptStruct* GetScriptStruct() const override;

	UE_API virtual FString ToSimpleString() const override;

	UE_API virtual void AddReferencedObjects(class FReferenceCollector& Collector) override;
};

#undef UE_API
