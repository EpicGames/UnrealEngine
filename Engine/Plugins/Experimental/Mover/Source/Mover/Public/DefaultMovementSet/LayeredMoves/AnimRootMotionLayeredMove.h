// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "LayeredMove.h"
#include "DefaultMovementSet/LayeredMoves/AnimRootMotionWarpingTypes.h"
#include "DefaultMovementSet/LayeredMoves/MontageStateProvider.h"
#include "MoverTypes.h"
#include "NativeGameplayTags.h"

#include "AnimRootMotionLayeredMove.generated.h"

#define UE_API MOVER_API

class UAnimMontage;

UE_API UE_DECLARE_GAMEPLAY_TAG_EXTERN(Mover_AnimRootMotion_Montage);

#if !UE_BUILD_SHIPPING
extern UE_API FAutoConsoleVariable CVarLogAnimRootMotionSteps;
#endif

/** Anim Root Motion Move: handles root motion from a montage played on the primary visual component (skeletal mesh).
 * In this method, root motion is extracted independently from anim playback. The move will end itself if the animation
 * is interrupted on the mesh.
 *
 * This variant runs on the game thread only. Use FLayeredMove_AnimRootMotion_SimDriven for simulation-driven
 * (worker thread) root motion with motion warping support.
 */
USTRUCT(BlueprintType)
struct FLayeredMove_AnimRootMotion : public FLayeredMove_MontageStateProvider
{
	GENERATED_BODY()

	UE_API FLayeredMove_AnimRootMotion();
	virtual ~FLayeredMove_AnimRootMotion() = default;

	UPROPERTY(BlueprintReadWrite, Category = Mover)
	FMoverAnimMontageState MontageState;

	// Generate a movement
	UE_API virtual bool GenerateMove(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove) override;

	UE_API bool HasGameplayTag(FGameplayTag TagToFind, bool bExactMatch) const override;

	UE_API virtual FLayeredMoveBase* Clone() const override;

	UE_API virtual void NetSerialize(FArchive& Ar) override;

	UE_API virtual UScriptStruct* GetScriptStruct() const override;

	UE_API virtual FString ToSimpleString() const override;

	UE_API virtual void AddReferencedObjects(class FReferenceCollector& Collector) override;

	// FLayeredMove_MontageStateProvider
	UE_API virtual FMoverAnimMontageState GetMontageState() const override;
};

template<>
struct TStructOpsTypeTraits< FLayeredMove_AnimRootMotion > : public TStructOpsTypeTraitsBase2< FLayeredMove_AnimRootMotion >
{
	enum
	{
		WithCopy = true
	};
};


/** Async-capable variant of FLayeredMove_AnimRootMotion. Runs GenerateMove_Async on the simulation worker
 * thread, computing root motion from montage data without touching game-thread objects. Supports motion
 * warping via warp targets snapshotted to the blackboard each frame.
 *
 * Use this variant when the Mover component is running in async physics mode and you need the simulation
 * to be authoritative over root motion position and blend-out timing.
 */
USTRUCT(BlueprintType)
struct FLayeredMove_AnimRootMotion_SimDriven : public FLayeredMove_AnimRootMotion
{
	GENERATED_BODY()

	virtual ~FLayeredMove_AnimRootMotion_SimDriven() = default;

	// Simulation time snapshot taken when this move starts on a worker thread. FrameCount is valid
	// only when started from a fixed-timestep step (ServerFrame != INDEX_NONE); otherwise it
	// remains INDEX_NONE and ElapsedSecondsTo falls back to ms arithmetic automatically.
	UPROPERTY(VisibleInstanceOnly, Transient, Category = Mover)
	FMoverTime StartMoverTime;

	// Set true in async the first time blend-out is triggered. Sticky: never cleared once set.
	// Read on the game thread to call Montage_Stop exactly once.
	UPROPERTY(VisibleInstanceOnly, Transient, Category = Mover)
	bool bShouldBlendOut = false;

	// Set true in async when the montage position has reached the end of a non-looping montage.
	// Read on the game thread to finalize the montage instance cleanly.
	UPROPERTY(VisibleInstanceOnly, Transient, Category = Mover)
	bool bIsFinished = false;

	// Server frame at the end of the substep when blend-out was first triggered. INDEX_NONE if not yet.
	UPROPERTY(VisibleInstanceOnly, Transient, Category = Mover)
	int32 BlendOutServerFrame = INDEX_NONE;

	// Server frame at the end of the substep when the montage was first detected as finished. INDEX_NONE if not yet.
	UPROPERTY(VisibleInstanceOnly, Transient, Category = Mover)
	int32 FinishServerFrame = INDEX_NONE;

	virtual bool SupportsAsync() const override { return true; }

	virtual void OnStart_Async(UMoverBlackboard* SimBlackboard, const FMoverTime& SimTime) override;

	UE_API virtual bool GenerateMove_Async(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove) override;

	UE_API virtual FLayeredMoveBase* Clone() const override;

	UE_API virtual void NetSerialize(FArchive& Ar) override;

	UE_API virtual UScriptStruct* GetScriptStruct() const override;

	UE_API virtual FString ToSimpleString() const override;

	// FLayeredMove_MontageStateProvider
	UE_API virtual void AppendMontageOutputEntry(TArray<FMoverSimDrivenMontageEntry>& OutEntries, const FMoverTimeStep& TimeStep) const override;
};

template<>
struct TStructOpsTypeTraits< FLayeredMove_AnimRootMotion_SimDriven > : public TStructOpsTypeTraitsBase2< FLayeredMove_AnimRootMotion_SimDriven >
{
	enum
	{
		WithCopy = true
	};
};

#undef UE_API
