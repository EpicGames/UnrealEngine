// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoverDataModelTypes.h"

#include "AnimRootMotionWarpingTypes.generated.h"

// Blackboard keys written by ChaosMover's state machine each substep for use by async layered moves
// (e.g. AnimRootMotion). Declared here alongside the types they carry.
namespace AnimRootMotionBlackboard
{
	// Relative transform of the primary visual component with respect to the actor root.
	// Written each substep so async layered moves can convert mesh-local root motion into world
	// space without touching game-thread-only objects.
	extern MOVER_API const FName LastPrimaryVisualComponentRelativeTransform;

	// GT-resolved motion warp targets snapshotted from UMotionWarpingComponent each frame.
	// Written each substep so GenerateMove_Async can apply skew-warp math without accessing
	// game-thread-only UObjects.
	// Value type: TArray<FMoverResolvedWarpTarget>
	extern MOVER_API const FName LastResolvedMotionWarpTargets;
}

// A GT-resolved motion warp target, safe to read on a worker thread.
// Populated from UMotionWarpingComponent::WarpTargets each frame by the ChaosMover backend.
USTRUCT()
struct FMoverResolvedWarpTarget
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = Mover)
	FName Name;

	UPROPERTY(VisibleAnywhere, Category = Mover)
	FVector Location = FVector::ZeroVector;

	UPROPERTY(VisibleAnywhere, Category = Mover)
	FQuat Rotation = FQuat::Identity;
};

// Replicated motion warp targets, populated from UMotionWarpingComponent on the locally-controlled
// machine and forwarded to all participants via the input command collection so every endpoint uses
// consistent targets during root motion warping on the worker thread.
USTRUCT()
struct FMoverMotionWarpingInputs : public FMoverDataStructBase
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = Mover)
	TArray<FMoverResolvedWarpTarget> WarpTargets;

	virtual FMoverDataStructBase* Clone() const override;
	virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;
	virtual UScriptStruct* GetScriptStruct() const override;
	virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override;
	virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override;
	virtual void Merge(const FMoverDataStructBase& From) override;
};
