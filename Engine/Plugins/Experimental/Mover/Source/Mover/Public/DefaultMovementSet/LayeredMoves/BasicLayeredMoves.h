// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "LayeredMove.h"
#include "LayeredMoveBase.h"
#include "BasicLayeredMoves.generated.h"

#define UE_API MOVER_API

class UCurveVector;
class UCurveFloat;

UENUM(BlueprintType, meta=(Bitflags, UseEnumValuesAsMaskValuesInEditor="true"))
enum class ELayeredMove_ConstantVelocitySettingsFlags : uint8
{
	NoFlags						= 0,

	// Velocity direction is determined when this move starts, relative to the target actor. This flag is incompatible with other Velocity**** flags
	VelocityStartRelative		= (1 << 0),
	// Velocity direction is always relative to the target actor, even if its orientation changes. This flag is incompatible with other Velocity**** flags
	VelocityAlwaysRelative		= (1 << 1),

};

/** Linear Velocity: A method of inducing a straight-line velocity on an actor over time  */
USTRUCT(BlueprintType)
struct FLayeredMove_LinearVelocity : public FLayeredMoveBase
{
	GENERATED_BODY()

	UE_API FLayeredMove_LinearVelocity();

	virtual ~FLayeredMove_LinearVelocity() = default;

	// Units per second, could be worldspace vs relative depending on Flags
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FVector Velocity;

	// Optional curve float for controlling the magnitude of the velocity applied to the actor over the duration of the move
	// Warning: This should not be set in async mode as it is not thread safe
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	TObjectPtr<UCurveFloat> MagnitudeOverTime;
	
	// @see ELayeredMove_ConstantVelocitySettingsFlags
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover, meta=(Bitmask, BitmaskEnum="/Script/Mover.ELayeredMove_ConstantVelocitySettingsFlags"))
	uint8 SettingsFlags;

	virtual bool SupportsAsync() const override { return true; }

	// Generate a movement
	UE_API virtual bool GenerateMove(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove) override;
	// Async safe version of movement generation (does not use MagnitudeOverTime)
	UE_API virtual bool GenerateMove_Async(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove) override;

	UE_API virtual FLayeredMoveBase* Clone() const override;

	UE_API virtual void NetSerialize(FArchive& Ar) override;

	UE_API virtual UScriptStruct* GetScriptStruct() const override;

	UE_API virtual FString ToSimpleString() const override;

	UE_API virtual void AddReferencedObjects(class FReferenceCollector& Collector) override;
};

template<>
struct TStructOpsTypeTraits< FLayeredMove_LinearVelocity > : public TStructOpsTypeTraitsBase2< FLayeredMove_LinearVelocity >
{
	enum
	{
		//WithNetSerializer = true,
		WithCopy = true
	};
};


USTRUCT(Blueprintable)
struct FLinearVelocityMoveActivationParams : public FLayeredMoveActivationParams
{
	GENERATED_BODY()

	// Units per second, world-space or relative to the actor depending on SettingsFlags.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FVector Velocity = FVector::ZeroVector;

	// @see ELayeredMove_ConstantVelocitySettingsFlags
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover, meta=(Bitmask, BitmaskEnum="/Script/Mover.ELayeredMove_ConstantVelocitySettingsFlags"))
	uint8 SettingsFlags = 0;
};

USTRUCT(Blueprintable)
struct FLinearVelocityMoveData : public FLayeredMoveInstancedData
{
	GENERATED_BODY()

	virtual FLayeredMoveInstancedData* Clone() const override { return new FLinearVelocityMoveData(*this); }
	virtual UScriptStruct* GetScriptStruct() const override { return StaticStruct(); }

	// Units per second, world-space or relative to the actor depending on SettingsFlags.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FVector Velocity = FVector::ZeroVector;

	// @see ELayeredMove_ConstantVelocitySettingsFlags
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover, meta=(Bitmask, BitmaskEnum="/Script/Mover.ELayeredMove_ConstantVelocitySettingsFlags"))
	uint8 SettingsFlags = 0;

	virtual void ActivateFromContext(const FLayeredMoveActivationParams* ActivationParams) override;
	virtual void NetSerialize(FArchive& Ar) override;
};

template<>
struct TStructOpsTypeTraits<FLinearVelocityMoveData> : public TStructOpsTypeTraitsBase2<FLinearVelocityMoveData>
{
	enum
	{
		WithCopy = true
	};
};

/**
 * Linear Velocity (instanced): Induces a straight-line velocity on an actor over time.
 * Async-compatible. Curve-based magnitude modulation is not supported; use FLayeredMove_LinearVelocity for that.
 */
UCLASS()
class ULinearVelocityMoveLogic : public ULayeredMoveLogic
{
	GENERATED_BODY()

public:
	UE_API ULinearVelocityMoveLogic();

	virtual bool SupportsAsync() const override { return true; }

protected:
	UE_API virtual bool GenerateMove_Implementation(const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, const FMoverTickStartData& StartState, FProposedMove& OutProposedMove) override;
	UE_API virtual bool GenerateMove_Async_Implementation(const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, const FMoverTickStartData& StartState, FProposedMove& OutProposedMove) override;
};


/** Jump Impulse Over Duration: introduces an instantaneous upwards change in velocity over a duration(ms).
  * This overrides the existing 'up' component of the actor's current velocity
  * Note: For a one time Jump Impulse consider using a FJumpImpulseEffect Instant Effect
  */
USTRUCT(BlueprintType)
struct FLayeredMove_JumpImpulseOverDuration : public FLayeredMoveBase
{
	GENERATED_BODY()

	UE_API FLayeredMove_JumpImpulseOverDuration();

	virtual ~FLayeredMove_JumpImpulseOverDuration() = default;

	// Units per second, in whatever direction the target actor considers 'up'
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	float UpwardsSpeed;

	// Generate a movement 
	UE_API virtual bool GenerateMove(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove) override;

	UE_API virtual FLayeredMoveBase* Clone() const override;

	UE_API virtual void NetSerialize(FArchive& Ar) override;

	UE_API virtual UScriptStruct* GetScriptStruct() const override;

	UE_API virtual FString ToSimpleString() const override;

	UE_API virtual void AddReferencedObjects(class FReferenceCollector& Collector) override;
};

template<>
struct TStructOpsTypeTraits< FLayeredMove_JumpImpulseOverDuration > : public TStructOpsTypeTraitsBase2< FLayeredMove_JumpImpulseOverDuration >
{
	enum
	{
		//WithNetSerializer = true,
		WithCopy = true
	};
};

/** JumpTo: Moves this actor in a more jump-like manner - specifying height and distance of jump rather than just upwards speed
  * Note: this layered move is only intended for Mover actors using Z as it's up direction
  */
USTRUCT(BlueprintType)
struct FLayeredMove_JumpTo : public FLayeredMoveBase
{
	GENERATED_BODY()

	UE_API FLayeredMove_JumpTo();
	virtual ~FLayeredMove_JumpTo() = default;
	
	// Distance this jump impulse is supposed to reach
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	float JumpDistance;

	// Height this jump impulse is supposed to reach
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	float JumpHeight;

	// If true this jump will use the actor's rotation for jump direction instead of the Jump Rotation property.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	bool bUseActorRotation;
	
	// Direction to jump in. Only used if bUseActorRotation is false
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FRotator JumpRotation;
	
	// Optional CurveVector used to offset the actor from the path
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	TObjectPtr<UCurveVector> PathOffsetCurve;
	
	// Optional CurveFloat to modify jump impulse over the duration of the impulse
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	TObjectPtr<UCurveFloat> TimeMappingCurve;
	
	// Generate a movement 
	UE_API virtual bool GenerateMove(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove) override;
	
	UE_API virtual FLayeredMoveBase* Clone() const override;
	
	UE_API virtual void NetSerialize(FArchive& Ar) override;
	
	UE_API virtual UScriptStruct* GetScriptStruct() const override;
	
	UE_API virtual FString ToSimpleString() const override;

	UE_API virtual void AddReferencedObjects(class FReferenceCollector& Collector) override;
protected:
	// helper function to apply movement vector offset from the PathOffsetCurve
	UE_API FVector GetPathOffset(const float MoveFraction) const;

	// Helper function to apply TimeMappingCurve to the layered move
	UE_API float EvaluateFloatCurveAtFraction(const UCurveFloat& Curve, const float Fraction) const;

	// Helper function for calculating position in jump given move fraction of total time for move
	UE_API FVector GetRelativeLocation(float MoveFraction, const FRotator& Rotator) const;
};

template<>
struct TStructOpsTypeTraits< FLayeredMove_JumpTo > : public TStructOpsTypeTraitsBase2< FLayeredMove_JumpTo >
{
	enum
	{
		//WithNetSerializer = true,
		WithCopy = true
	};
};

/** MoveTo: Move Actor from the starting location to the target location over a duration of time.*/
USTRUCT(BlueprintType)
struct FLayeredMove_MoveTo : public FLayeredMoveBase
{
	GENERATED_BODY()

	UE_API FLayeredMove_MoveTo();
	virtual ~FLayeredMove_MoveTo() = default;

	// Location to Start the MoveTo move from
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FVector StartLocation;
	
	// Location to move towards
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FVector TargetLocation;

	// if true, will restrict speed to where the actor is expected to be (in regard to start, end and duration)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	bool bRestrictSpeedToExpected;
	
	// Optional CurveVector used to offset the actor from the path
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	TObjectPtr<UCurveVector> PathOffsetCurve;

	// Optional CurveFloat to apply to how fast the actor moves as they get closer to the target location
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	TObjectPtr<UCurveFloat> TimeMappingCurve;
	
	// Generate a movement 
	UE_API virtual bool GenerateMove(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove) override;

	UE_API virtual FLayeredMoveBase* Clone() const override;

	UE_API virtual void NetSerialize(FArchive& Ar) override;

	UE_API virtual UScriptStruct* GetScriptStruct() const override;

	UE_API virtual FString ToSimpleString() const override;

	UE_API virtual void AddReferencedObjects(class FReferenceCollector& Collector) override;
	
protected:
	// helper function to apply movement vector offset from the PathOffsetCurve
	UE_API FVector GetPathOffsetInWorldSpace(const float MoveFraction) const;

	// Helper function to apply TimeMappingCurve to the layered move
	UE_API float EvaluateFloatCurveAtFraction(const UCurveFloat& Curve, const float Fraction) const;
};

template<>
struct TStructOpsTypeTraits< FLayeredMove_MoveTo > : public TStructOpsTypeTraitsBase2< FLayeredMove_MoveTo >
{
	enum
	{
		//WithNetSerializer = true,
		WithCopy = true
	};
};


/** MoveToDynamic: Move Actor from the starting location to the target location over a duration of time with a moving target location
 *  You can either update TargetLocation manually to update the location this actor is moving towards or set an actor for this actor to move towards.
 */
USTRUCT(BlueprintType)
struct FLayeredMove_MoveToDynamic : public FLayeredMove_MoveTo
{
	GENERATED_BODY()

	UE_API FLayeredMove_MoveToDynamic();
	virtual ~FLayeredMove_MoveToDynamic() = default;
	
	// Optional actor to move to. Note: this overrides the TargetLocation Property
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	TObjectPtr<AActor> LocationActor;

	// Generate a movement 
	UE_API virtual bool GenerateMove(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove) override;

	UE_API virtual FLayeredMoveBase* Clone() const override;

	UE_API virtual void NetSerialize(FArchive& Ar) override;

	UE_API virtual UScriptStruct* GetScriptStruct() const override;

	UE_API virtual FString ToSimpleString() const override;

	UE_API virtual void AddReferencedObjects(class FReferenceCollector& Collector) override;
};

template<>
struct TStructOpsTypeTraits< FLayeredMove_MoveToDynamic > : public TStructOpsTypeTraitsBase2< FLayeredMove_MoveToDynamic >
{
	enum
	{
		//WithNetSerializer = true,
		WithCopy = true
	};
};

/** RadialImpulse applies a velocity pulling or pushing away from a given world location to the target actor */
USTRUCT(BlueprintType)
struct FLayeredMove_RadialImpulse : public FLayeredMoveBase
{
	GENERATED_BODY()

	UE_API FLayeredMove_RadialImpulse();
	virtual ~FLayeredMove_RadialImpulse() = default;

	// Location to pull or push actor from
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FVector Location;

	// Optional Actor to pull or push this actor from. Note: this overrides the Location Property
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	TObjectPtr<AActor> LocationActor;

	// Radius from the location to apply radial velocity
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	float Radius;

	// Magnitude of velocity applied to actors in range
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	float Magnitude;

	// If true the velocity applied will push actor away from location, otherwise this layered move wil pull the actor towards it
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	bool bIsPush;

	// Whether to apply vertical velocity (based off mover components up direction) to the actor affected by this layered move
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	bool bNoVerticalVelocity;

	// Optional Curvefloat to dictate falloff of velocity as you get further from the velocity location
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	TObjectPtr<UCurveFloat> DistanceFalloff;

	// Optional Curvefloat to dictate magnitude of velocity applied over time
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	TObjectPtr<UCurveFloat> MagnitudeOverTime;

	// If true velocity added to the actor will be in the direction of FixedWorldDirection, otherwise it will be calculated from the position of the actor and velocity location.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	bool bUseFixedWorldDirection;

	// Direction to apply velocity from the radial impulse in. This is only taken into account if UseFixedWorldDirection is true.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FRotator FixedWorldDirection;
	
	// Generate a movement 
	UE_API virtual bool GenerateMove(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove) override;

	UE_API virtual FLayeredMoveBase* Clone() const override;

	UE_API virtual void NetSerialize(FArchive& Ar) override;

	UE_API virtual UScriptStruct* GetScriptStruct() const override;

	UE_API virtual FString ToSimpleString() const override;

	UE_API virtual void AddReferencedObjects(class FReferenceCollector& Collector) override;
};

template<>
struct TStructOpsTypeTraits< FLayeredMove_RadialImpulse > : public TStructOpsTypeTraitsBase2< FLayeredMove_RadialImpulse >
{
	enum
	{
		//WithNetSerializer = true,
		WithCopy = true
	};
};

#undef UE_API
