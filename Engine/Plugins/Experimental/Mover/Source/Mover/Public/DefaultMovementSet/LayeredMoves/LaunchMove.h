// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LayeredMove.h"
#include "LayeredMoveBase.h"
#include "LaunchMove.generated.h"

#define UE_API MOVER_API

USTRUCT(Blueprintable)
struct FLaunchMoveActivationParams : public FLayeredMoveActivationParams
{
	GENERATED_BODY()

	/** Velocity to apply to the updated component. Could be additive or overriding depending on MixMode setting. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover, meta=(ForceUnits="cm/s"))
	FVector LaunchVelocity = FVector::ZeroVector;

	// Optional movement mode name to force the actor into before applying the impulse velocity.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FName ForceMovementMode = NAME_None;
	
};

USTRUCT(Blueprintable)
struct FLaunchMoveData : public FLayeredMoveInstancedData
{
	GENERATED_BODY()

	//@todo DanH: This is boilerplate begging for a macro
	virtual FLayeredMoveInstancedData* Clone() const override { return new FLaunchMoveData(*this); }
	virtual UScriptStruct* GetScriptStruct() const override { return StaticStruct(); }
	
	/** Velocity to apply to the updated component. Could be additive or overriding depending on MixMode setting. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover, meta=(ForceUnits="cm/s"))
	FVector LaunchVelocity = FVector::ZeroVector;

	/** Optional movement mode name to force the actor into before applying the impulse velocity. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FName ForceMovementMode;
	
	virtual void ActivateFromContext(const FLayeredMoveActivationParams* ActivationParams) override;
	
	virtual void NetSerialize(FArchive& Ar) override;
};

// TODO: Create data for this? Is it not needed?!
UCLASS()
class ULaunchMoveLogic : public ULayeredMoveLogic
{
	GENERATED_BODY()

public:
	UE_API ULaunchMoveLogic();

	/** Velocity to apply to the updated component. Could be additive or overriding depending on MixMode setting. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover, meta=(ForceUnits="cm/s"))
	FVector LaunchVelocity = FVector::ZeroVector;

	//@todo DanH: Should forcing a mode be an option at the root ULayeredMoveLogic?
	/** Optional movement mode name to force the actor into before applying the impulse velocity. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FName ForceMovementMode;

	virtual bool SupportsAsync() const override { return true; }

protected:
	UE_API virtual bool GenerateMove_Implementation(const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, const FMoverTickStartData& StartState, FProposedMove& OutProposedMove) override;
	UE_API virtual bool GenerateMove_Async_Implementation(const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, const FMoverTickStartData& StartState, FProposedMove& OutProposedMove) override;
};

/** Launch Move: provides an impulse velocity to the actor after (optionally) forcing them into a particular movement mode */
USTRUCT(BlueprintType)
struct FLayeredMove_Launch : public FLayeredMoveBase
{
	GENERATED_BODY()

	UE_API FLayeredMove_Launch();
	virtual ~FLayeredMove_Launch() = default;

	// Velocity to apply to the actor. Could be additive or overriding depending on MixMode setting.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover, meta=(ForceUnits="cm/s"))
	FVector LaunchVelocity = FVector::ZeroVector;

	// Optional movement mode name to force the actor into before applying the impulse velocity.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FName ForceMovementMode = NAME_None;

	virtual bool SupportsAsync() const override { return true; }

	// Generate a movement
	UE_API virtual bool GenerateMove(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove) override;
	UE_API virtual bool GenerateMove_Async(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove) override;

	UE_API virtual FLayeredMoveBase* Clone() const override;

	UE_API virtual void NetSerialize(FArchive& Ar) override;

	UE_API virtual UScriptStruct* GetScriptStruct() const override;

	UE_API virtual FString ToSimpleString() const override;

	UE_API virtual void AddReferencedObjects(class FReferenceCollector& Collector) override;
};


template<>
struct TStructOpsTypeTraits< FLayeredMove_Launch > : public TStructOpsTypeTraitsBase2< FLayeredMove_Launch >
{
	enum
	{
		WithCopy = true
	};
};

#undef UE_API
