// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "LayeredMove.h"
#include "DefaultMovementSet/MoverMontageSimulationTypes.h"
#include "MontageStateProvider.generated.h"

#define UE_API MOVER_API

class UAnimMontage;


/** Data about montages that is replicated to simulated clients */
USTRUCT(BlueprintType)
struct FMoverAnimMontageState
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category = Mover)
	TObjectPtr<UAnimMontage> Montage;

	// Asset name of Montage, kept in sync with the Montage pointer for CVD display
	// (UObject references cannot cross the CVD trace boundary).
	// Not net serialized -- derived locally from the Montage pointer on each machine.
	// Only populated in non-shipping builds.
	UPROPERTY(VisibleAnywhere, Category = Mover, meta = (DisplayName = "MontageName"))
	FName Debug_MontageName;

	// Montage position when started (in unscaled seconds).
	UPROPERTY(BlueprintReadWrite, Category = Mover)
	float StartingMontagePosition = 0.0f;

	// Rate at which this montage is intended to play
	UPROPERTY(BlueprintReadWrite, Category = Mover)
	float PlayRate = 1.0f;

	// Current position (during playback only)
	UPROPERTY(BlueprintReadOnly, Category = Mover)
	float CurrentPosition = 0.0f;

	// Duration of the blend-out in seconds, copied from the montage asset's default blend-out at
	// move-creation time. Used by the PT to compute the blend-out trigger position.
	UPROPERTY(BlueprintReadOnly, Category = Mover)
	float BlendOutTimeSeconds = 0.f;

	// True when the simulation should automatically trigger blend-out based on playback position
	// approaching the end of the montage. Set to false for montages that require an external stop
	// signal (e.g. internally looping montages whose terminal section is reached via JumpToSection
	// on the GT); for those, blend-out falls back to DurationMs expiry.
	UPROPERTY(BlueprintReadOnly, Category = Mover)
	bool bEnableAutoBlendOut = true;

	void Reset();
	void NetSerialize(FArchive& Ar);
};



/** Note this will become obsolete once layered move logic is represented by a uobject, allowing use of interface classes.  */
USTRUCT(BlueprintType)
struct FLayeredMove_MontageStateProvider : public FLayeredMoveBase
{
	GENERATED_BODY()

	// Push simulation-authoritative montage state into the simulation output so the game thread can drive FAnimMontageInstance.
	// Derived classes append one FMoverSimDrivenMontageEntry per active montage they own.
	// TimeStep is provided so implementations can populate timing fields (FrameSimTimeMs, etc.) for precise interpolation.
	// Default implementation is a no-op: base class has no montage state to contribute.
	virtual void AppendMontageOutputEntry(TArray<FMoverSimDrivenMontageEntry>& OutEntries, const FMoverTimeStep& TimeStep) const {}

	virtual FMoverAnimMontageState GetMontageState() const
	{
		checkNoEntry();
		return FMoverAnimMontageState();
	}
};

#undef UE_API
