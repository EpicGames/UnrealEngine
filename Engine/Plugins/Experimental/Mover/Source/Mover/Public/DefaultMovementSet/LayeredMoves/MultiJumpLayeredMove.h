// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LayeredMove.h"
#include "MultiJumpLayeredMove.generated.h"

#define UE_API MOVER_API

struct FMoverInputCmdContext;
struct FMoverDefaultSyncState;
class UCommonLegacyMovementSettings;

/** MultiJump: layered move for handling multiple jumps without touching the ground i.e. a double jump.
  * Note: This layered move ends automatically when the actor hits a valid floor so duration is expected to be less than 0. */
USTRUCT(BlueprintType)
struct FLayeredMove_MultiJump : public FLayeredMoveBase
{
	GENERATED_BODY()

	UE_API FLayeredMove_MultiJump();
	virtual ~FLayeredMove_MultiJump() = default;

	/** Maximum Jumps an actor could perform while in the air */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Jumping", meta = (ClampMin = "1", UIMin = "1"))
	int32 MaximumInAirJumps;

	// Units per second, in whatever direction the target actor considers 'up'
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Jumping", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s"))
	float UpwardsSpeed;

	// returns true if input context has state that the actor wants to jump
	UE_API virtual bool WantsToJump(const FMoverInputCmdContext& InputCmd);
	
	// Generate a movement 
	UE_API virtual bool GenerateMove(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove) override;

	UE_API virtual FLayeredMoveBase* Clone() const override;

	UE_API virtual void NetSerialize(FArchive& Ar) override;

	UE_API virtual UScriptStruct* GetScriptStruct() const override;

	UE_API virtual FString ToSimpleString() const override;

	UE_API virtual void AddReferencedObjects(class FReferenceCollector& Collector) override;

protected:
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Jumping")
	int32 JumpsInAirRemaining;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Jumping")
	double TimeOfLastJumpMS;

	UE_API bool PerformJump(const FMoverDefaultSyncState* SyncState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, FProposedMove& OutProposedMove);
};

template<>
struct TStructOpsTypeTraits< FLayeredMove_MultiJump > : public TStructOpsTypeTraitsBase2< FLayeredMove_MultiJump >
{
	enum
	{
		//WithNetSerializer = true,
		WithCopy = true
	};
};

#undef UE_API
