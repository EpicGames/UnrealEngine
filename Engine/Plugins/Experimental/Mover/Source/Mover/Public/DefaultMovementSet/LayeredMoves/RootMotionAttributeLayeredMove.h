// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LayeredMove.h"
#include "NativeGameplayTags.h"
#include "RootMotionModifier.h"
#include "RootMotionAttributeLayeredMove.generated.h"

#define UE_API MOVER_API

MOVER_API UE_DECLARE_GAMEPLAY_TAG_EXTERN(Mover_AnimRootMotion_MeshAttribute);

/** 
 * Root Motion Attribute Move: handles root motion from a mesh's custom attribute, ignoring scaling.
 * Currently only supports Independent ticking mode, and allows controlled movement while jumping/falling or when a SkipAnimRootMotion tag is active.
 */
USTRUCT(BlueprintType)
struct FLayeredMove_RootMotionAttribute : public FLayeredMoveBase
{
	GENERATED_BODY()

	UE_API FLayeredMove_RootMotionAttribute();
	virtual ~FLayeredMove_RootMotionAttribute() = default;

	// If true, any root motion rotations will be projected onto the movement plane (in worldspace), relative to the "up" direction. Otherwise, they'll be taken as-is.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	bool bConstrainWorldRotToMovementPlane = true;

protected:
	// These member variables are NOT replicated. They are used if we rollback and resimulate when the root motion attribute is no longer in sync.
	bool bDidAttrHaveRootMotionForResim = false;
	FTransform LocalRootMotionForResim;
	FMotionWarpingUpdateContext WarpingContextForResim;

	// Generate a movement 
	UE_API virtual bool GenerateMove(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove) override;

	UE_API virtual bool HasGameplayTag(FGameplayTag TagToFind, bool bExactMatch) const override;

	UE_API virtual FLayeredMoveBase* Clone() const override;

	UE_API virtual void NetSerialize(FArchive& Ar) override;

	UE_API virtual UScriptStruct* GetScriptStruct() const override;

	UE_API virtual FString ToSimpleString() const override;

	UE_API virtual void AddReferencedObjects(class FReferenceCollector& Collector) override;
};

template<>
struct TStructOpsTypeTraits< FLayeredMove_RootMotionAttribute > : public TStructOpsTypeTraitsBase2< FLayeredMove_RootMotionAttribute >
{
	enum
	{
		WithCopy = true
	};
};

#undef UE_API
