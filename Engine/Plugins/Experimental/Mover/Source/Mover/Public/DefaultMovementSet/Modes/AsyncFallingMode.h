// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovementMode.h"
#include "FallingMode.h"
#include "AsyncFallingMode.generated.h"

#define UE_API MOVER_API

class UCommonLegacyMovementSettings;
struct FFloorCheckResult;

/**
 * AsyncFallingMode: a default movement mode for moving through the air and jumping, typically influenced by gravity and air control
 * This mode simulates movement without actually modifying any scene component(s).
 */
UCLASS(MinimalAPI, Blueprintable, BlueprintType, Experimental)
class UAsyncFallingMode : public UBaseMovementMode
{
	GENERATED_BODY()


public:
	UE_API UAsyncFallingMode(const FObjectInitializer& ObjectInitializer);

	UE_API virtual void OnRegistered(const FName ModeName, const FMoverSimContext& SimContext) override;
    UE_API virtual void OnUnregistered(const FMoverSimContext& SimContext) override;
	UE_API virtual void GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const override;
	UE_API virtual void SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState) override;

	// Broadcast when this actor lands on a valid surface.
	UPROPERTY(BlueprintAssignable, Category = Mover)
	FMover_OnLanded OnLanded;
	
	/**
	 * If true, actor will land and lose all speed in the vertical direction upon landing. If false, actor's vertical speed will be redirected based on the surface normal it hit.
	 * Note: Actor's horizontal speed will not be affected if true. If false, horizontal speed may be increased on landing.
	 */
	UPROPERTY(Category = Mover, EditAnywhere, BlueprintReadWrite)
	bool bCancelVerticalSpeedOnLanding;
	
	/**
	 * When falling, amount of movement control available to the actor.
	 * 0 = no control, 1 = full control
	 */
	UPROPERTY(Category=Mover, EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", ClampMax="1.0"))
	float AirControlPercentage;
	
	/**
 	 * Deceleration to apply to air movement when falling slower than terminal velocity.
 	 * Note: This is NOT applied to vertical velocity, only movement plane velocity
 	 */
	UPROPERTY(Category=Mover, EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", ForceUnits = "cm/s^2"))
	float FallingDeceleration;

	/**
     * Deceleration to apply to air movement when falling faster than terminal velocity
	 * Note: This is NOT applied to vertical velocity, only movement plane velocity
     */
    UPROPERTY(Category=Mover, EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", ForceUnits = "cm/s^2"))
    float OverTerminalSpeedFallingDeceleration;
	
	/**
	 * If the actor's movement plane velocity is greater than this speed falling will start applying OverTerminalSpeedFallingDeceleration instead of FallingDeceleration
	 * The expected behavior is to set OverTerminalSpeedFallingDeceleration higher than FallingDeceleration so the actor will slow down faster
	 * when going over TerminalMovementPlaneSpeed.
	 */
	UPROPERTY(Category=Mover, EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", ForceUnits = "cm/s"))
	float TerminalMovementPlaneSpeed;

	/** When exceeding maximum vertical speed, should it be enforced via a hard clamp? If false, VerticalFallingDeceleration will be used for a smoother transition to the terminal speed limit. */
	UPROPERTY(Category = Mover, EditAnywhere, BlueprintReadWrite)
	bool bShouldClampTerminalVerticalSpeed;

	/** Deceleration to apply to vertical velocity when it's greater than TerminalVerticalSpeed. Only used if bShouldClampTerminalVerticalSpeed is false. */
	UPROPERTY(Category=Mover, EditAnywhere, BlueprintReadWrite, meta=(EditCondition="!bShouldClampTerminalVerticalSpeed", ClampMin="0", ForceUnits = "cm/s^2"))
	float VerticalFallingDeceleration;
	
	/**
	 * If the actors vertical velocity is greater than this speed VerticalFallingDeceleration will be applied to vertical velocity
	 */
	UPROPERTY(Category=Mover, EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", ForceUnits = "cm/s"))
	float TerminalVerticalSpeed;
	
protected:
	/**
	 * Is called at the end of the tick in falling mode. Handles checking any landings that should occur and switching to specific modes
	 * (i.e. landing on a walkable surface would switch to the walking movement mode) 
	 */
	UFUNCTION(BlueprintCallable, Category=Mover)
	UE_API virtual void ProcessLanded(const FFloorCheckResult& FloorResult, FVector Location, FVector& Velocity, FRelativeBaseInfo& BaseInfo, FMoverTickEndData& TickEndData) const;

	UE_API void CaptureFinalState(const FMoverDefaultSyncState* StartSyncState, const FVector FinalLocation, const FRotator FinalRotation, const FFloorCheckResult& FloorResult, float DeltaSeconds, float DeltaSecondsUsed, const FVector& AngularVelocityDegrees, FMoverDefaultSyncState& OutputSyncState, FMoverTickEndData& TickEndData, FMovementRecord& Record) const;

	TWeakObjectPtr<const UCommonLegacyMovementSettings> CommonLegacySettings;
};

#undef UE_API
