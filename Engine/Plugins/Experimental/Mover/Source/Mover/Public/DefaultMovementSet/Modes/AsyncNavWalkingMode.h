// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovementMode.h"
#include "MoveLibrary/BasedMovementUtils.h"
#include "AI/Navigation/NavigationTypes.h"
#include "NavWalkingMode.h"
#include "AsyncNavWalkingMode.generated.h"

#define UE_API MOVER_API

class UNavMoverComponent;
class INavigationDataInterface;
class UCommonLegacyMovementSettings;


/**
 * AsyncNavWalkingMode: a default movement mode for traversing surfaces and movement bases by using an active navmesh when moving the actor rather than collision checks.
 * Note: This movement mode requires a NavMoverComponent be on the actor to function properly. This mode also contains some randomization to avoid navmesh look ups
 *	happening at the same time (which is fine for AI characters running on the server) but may cause issues if used on autonomous proxies.
 * This mode simulates movement without actually modifying any scene component(s).
 */
UCLASS(MinimalAPI, Blueprintable, BlueprintType, Experimental)
class UAsyncNavWalkingMode : public UBaseMovementMode
{
	GENERATED_BODY()

public:
	UE_API UAsyncNavWalkingMode();

	UE_API virtual void Activate(const FMoverEventContext& Context, FName PrevModeName, const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, FMoverSyncState* OutSyncState, FMoverAuxStateContext* OutAuxState) override;
	UE_API virtual void SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState) override;
	UE_API virtual void GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const override;

	/**
	 * Whether or not the actor should sweep for collision geometry while walking.
	 */
	UPROPERTY(Category = "NavMesh Movement", EditAnywhere, BlueprintReadWrite)
	bool bSweepWhileNavWalking;

	/** Whether to raycast to underlying geometry to better conform navmesh-walking actors */
	UPROPERTY(Category = "NavMesh Movement", EditAnywhere, BlueprintReadWrite)
	bool bProjectNavMeshWalking;

	/**
	 * Scale of the total capsule height to use for projection from navmesh to underlying geometry in the upward direction.
	 * In other words, start the trace at [CapsuleHeight * NavMeshProjectionHeightScaleUp] above nav mesh.
	 */
	UPROPERTY(Category = "NavMesh Movement", EditAnywhere, BlueprintReadWrite, meta = (editcondition = "bProjectNavMeshWalking", ClampMin = "0", UIMin = "0"))
	float NavMeshProjectionHeightScaleUp;

	/**
	 * Scale of the total capsule height to use for projection from navmesh to underlying geometry in the downward direction.
	 * In other words, trace down to [CapsuleHeight * NavMeshProjectionHeightScaleDown] below nav mesh.
	 */
	UPROPERTY(Category = "NavMesh Movement", EditAnywhere, BlueprintReadWrite, meta = (editcondition = "bProjectNavMeshWalking", ClampMin = "0", UIMin = "0"))
	float NavMeshProjectionHeightScaleDown;

	/** How often we should raycast to project from navmesh to underlying geometry */
	UPROPERTY(Category = "NavMesh Movement", EditAnywhere, BlueprintReadWrite, meta = (editcondition = "bProjectNavMeshWalking", ForceUnits = "s"))
	float NavMeshProjectionInterval;

	/** Speed at which to interpolate agent navmesh offset between traces. 0: Instant (no interp) > 0: Interp speed") */
	UPROPERTY(Category = "NavMesh Movement", EditAnywhere, BlueprintReadWrite, meta = (editcondition = "bProjectNavMeshWalking", ClampMin = "0", UIMin = "0"))
	float NavMeshProjectionInterpSpeed;
	
	/** What to do when we are projecting nav movement onto geometry from the nav mesh */
	UPROPERTY(Category = "NavMesh Movement", EditAnywhere, BlueprintReadWrite, meta=(editcondition = "bProjectNavMeshWalking"))
	EProjectNavMeshToGeometryBehavior ProjectNavMeshToGeometryBehavior = EProjectNavMeshToGeometryBehavior::UseTopGeometry;
	
	/** What should we do if we stray off the nav mesh? */
	UPROPERTY(Category = "NavMesh Movement", EditAnywhere, BlueprintReadWrite)
	EOffNavMeshBehavior BehaviorOffNavMesh = EOffNavMeshBehavior::SwitchToWalking;

	/** If attempting to stray off the nav mesh, should we slide along the edge instead? See @BehaviorOffNavMesh for cases where a sliding move can't be determined. */
	UPROPERTY(Category = "NavMesh Movement", EditAnywhere, BlueprintReadWrite)
	bool bSlideAlongNavMeshEdge = false;
	

	UPROPERTY(Transient)
	float NavMeshProjectionTimer;

	/** last known location projected on navmesh */
	FNavLocation CachedNavLocation;

	/**
	 * Project a location to navmesh to find adjusted height.
	 * @param TestLocation			Location to project
	 * @param OutNavFloorLocation	Location on navmesh
	 * @param NavData				NavigationDataInterface to search TestLocation for
	 * @return True if projection was performed (successfully or not)
	 */
	UE_API virtual bool FindNavFloor(const FVector& TestLocation, FNavLocation& OutNavFloorLocation, const INavigationDataInterface* NavData) const;

	// Returns the active turn generator. Note: you will need to cast the return value to the generator you expect to get, it can also be none
	UFUNCTION(BlueprintPure, Category = Mover)
	UE_API UObject* GetTurnGenerator();

	// Sets the active turn generator to use the class provided. Note: To set it back to the default implementation pass in none
	UFUNCTION(BlueprintCallable, Category = Mover)
	UE_API void SetTurnGeneratorClass(UPARAM(meta = (MustImplement = "/Script/Mover.TurnGeneratorInterface", AllowAbstract = "false")) TSubclassOf<UObject> TurnGeneratorClass);

protected:
	/** Associated Movement component that will actually move the actor */
	TWeakObjectPtr<UNavMoverComponent> NavMoverComponent;

	// Note: This isn't guaranteed to be valid at all times. It will be most of the time but re-entering this mode to call Activate() will search for NavData again and update it accordingly.
	TWeakInterfacePtr<const INavigationDataInterface> NavDataInterface;
	
	/** Use both WorldStatic and WorldDynamic channels for NavWalking geometry conforming */
	UPROPERTY(Category = "NavMesh Movement", EditAnywhere, BlueprintReadOnly, AdvancedDisplay)
	uint8 bProjectNavMeshOnBothWorldChannels : 1;

	/** Optional modular object for generating rotation towards desired orientation. If not specified, linear interpolation will be used. */
	UPROPERTY(EditAnywhere, Instanced, Category = Mover, meta = (ObjectMustImplement = "/Script/Mover.TurnGeneratorInterface"))
	TObjectPtr<UObject> TurnGenerator;
	
	/** Get Navigation data for the actor. Returns null if there is no associated nav data. */
	UE_API const INavigationDataInterface* GetNavData() const;

	/** Performs trace for ProjectLocationFromNavMesh */
	UE_API virtual void FindBestNavMeshLocation(const FVector& TraceStart, const FVector& TraceEnd, const FVector& CurrentFeetLocation, const FVector& TargetNavLocation, FHitResult& OutHitResult) const;

	/**
	 * Attempts to better align navmesh walking actors with underlying geometry (sometimes
	 * navmesh can differ quite significantly from geometry).
	 * Updates CachedProjectedNavMeshHitResult, access this for more info about hits.
	 */
	UE_API virtual FVector ProjectLocationFromNavMesh(float DeltaSeconds, const FVector& CurrentFeetLocation, const FVector& TargetNavLocation, float UpOffset, float DownOffset);


	UE_API virtual void OnRegistered(const FName ModeName, const FMoverSimContext& SimContext) override;
	UE_API virtual void OnUnregistered(const FMoverSimContext& SimContext) override;

	UE_API void CaptureOutputState(const FMoverDefaultSyncState& StartSyncState, const FVector& FinalLocation, const FRotator& FinalRotation, const FMovementRecord& Record, const FVector& AngularVelocityDegrees, FMoverDefaultSyncState& OutputSyncState, FMoverTickEndData& OutputState) const;

	TWeakObjectPtr<const UCommonLegacyMovementSettings> CommonLegacySettings;
 
};

#undef UE_API
