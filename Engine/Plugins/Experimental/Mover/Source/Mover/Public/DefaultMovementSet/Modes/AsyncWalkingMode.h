// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovementMode.h"
#include "MoveLibrary/ModularMovement.h"
#include "WalkingMode.h"
#include "AsyncWalkingMode.generated.h"

#define UE_API MOVER_API

class UCommonLegacyMovementSettings;
struct FFloorCheckResult;
struct FRelativeBaseInfo;
struct FMovementRecord;


/**
 * AsyncWalkingMode: a default movement mode for traversing surfaces and movement bases (walking, running, sneaking, etc.)
 * This mode simulates movement without actually modifying any scene component(s).
 */
UCLASS(MinimalAPI, Blueprintable, BlueprintType, Experimental)
class UAsyncWalkingMode : public UBaseMovementMode
{
	GENERATED_BODY()

public:
	UE_API UAsyncWalkingMode(const FObjectInitializer& ObjectInitializer);
	
	UE_API virtual void Activate(const FMoverEventContext& Context, FName PrevModeName, const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, FMoverSyncState* OutSyncState, FMoverAuxStateContext* OutAuxState) override;
	UE_API virtual void GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const override;
	UE_API virtual void SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState) override;

	// Returns the active turn generator. Note: you will need to cast the return value to the generator you expect to get, it can also be none
	UFUNCTION(BlueprintPure, Category=Mover)
	UE_API UObject* GetTurnGenerator();

	// Sets the active turn generator to use the class provided. Note: To set it back to the default implementation pass in none
	UFUNCTION(BlueprintCallable, Category=Mover)
	UE_API void SetTurnGeneratorClass(UPARAM(meta=(MustImplement="/Script/Mover.TurnGeneratorInterface", AllowAbstract="false")) TSubclassOf<UObject> TurnGeneratorClass);

protected:

	/** Choice of behavior for floor checks while not moving.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	EStaticFloorCheckPolicy FloorCheckPolicy = EStaticFloorCheckPolicy::OnDynamicBaseOnly;

	/** Optional modular object for generating rotation towards desired orientation. If not specified, linear interpolation will be used. */
	UPROPERTY(EditAnywhere, Instanced, Category=Mover, meta=(ObjectMustImplement="/Script/Mover.TurnGeneratorInterface"))
	TObjectPtr<UObject> TurnGenerator;

	UE_API virtual void OnRegistered(const FName ModeName, const FMoverSimContext& SimContext) override;
	UE_API virtual void OnUnregistered(const FMoverSimContext& SimContext) override;

	UE_API void CaptureFinalState(const FMoverDefaultSyncState* StartingState, const FVector FinalLocation, const FRotator FinalRotation, bool bDidAttemptMovement, const FFloorCheckResult& FloorResult, const FMovementRecord& Record, const FVector& AngularVelocityDegrees, FMoverDefaultSyncState& OutputSyncState) const;

	UE_API FRelativeBaseInfo UpdateFloorAndBaseInfo(const FFloorCheckResult& FloorResult, FVector CurrentLocation) const;

	TWeakObjectPtr<const UCommonLegacyMovementSettings> CommonLegacySettings;
};

#undef UE_API
