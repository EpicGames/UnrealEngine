// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovementMode.h"
#include "FlyingMode.generated.h"

#define UE_API MOVER_API

class UCommonLegacyMovementSettings;


/**
 * FlyingMode: a default movement mode for moving through the air freely, but still interacting with blocking geometry. The
 * moving actor will remain upright vs the movement plane.
 */
UCLASS(MinimalAPI, Blueprintable, BlueprintType)
class UFlyingMode : public UBaseMovementMode
{
	GENERATED_UCLASS_BODY()


public:
	/**
	 * If true, the actor will 'float' above any walkable surfaces to maintain the same height as ground-based modes. 
	 * This can prevent pops when transitioning to ground-based movement, at the cost of performing floor checks while flying.
	 */
	UPROPERTY(Category = Mover, EditAnywhere, BlueprintReadWrite)
	bool bRespectDistanceOverWalkableSurfaces = false;

	UE_API virtual void GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const override;

	UE_API virtual void SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState) override;

protected:
	UE_API virtual void OnRegistered(const FName ModeName, const FMoverSimContext& SimContext) override;
	UE_API virtual void OnUnregistered(const FMoverSimContext& SimContext) override;

	UE_API void CaptureFinalState(USceneComponent* UpdatedComponent, FMovementRecord& Record, const FMoverDefaultSyncState& StartSyncState, const FVector& AngularVelocityDegrees, FMoverDefaultSyncState& OutputSyncState, const float DeltaSeconds) const;

	TObjectPtr<const UCommonLegacyMovementSettings> CommonLegacySettings;
};

#undef UE_API
