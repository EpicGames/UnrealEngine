// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "MovementMode.h"
#include "MoveLibrary/BasedMovementUtils.h"
#include "AI/Navigation/NavigationTypes.h"
#include "NavWalkingMode.generated.h"

#define UE_API MOVER_API

class UNavMoverComponent;
class UMoverComponent;
class INavigationDataInterface;
class UCommonLegacyMovementSettings;


/** Options for what to do when we find ourselves off the nav mesh */
UENUM(BlueprintType)
enum class EOffNavMeshBehavior : uint8
{
	/** We change to normal walking mode and re-attempt the move. Typically this is more expensive than nav walking, and we'll later need to switch back to nav walking. */
	SwitchToWalking = 0        UMETA(DisplayName = "Switch to Walking Mode"),

	/** We proceed to move as directed, but height may diverge from the floor until we return to a valid nav mesh. */
	MoveWithoutNavMesh = 1     UMETA(DisplayName = "Move Without Nav Mesh"),

	/** We do not move. Movement will continue once instructed to move over nav mesh again. */
	DoNotMove = 2			   UMETA(DisplayName = "Do Not Move or Rotate"),

	/** We do not move, but will allow rotation in place. Movement will continue once instructed to move over nav mesh again. */
	RotateOnly = 3			   UMETA(DisplayName = "Rotate Only"),
};

/** Options for what to do when we are projecting nav movement onto geometry from the nav mesh */
UENUM(BlueprintType)
enum class EProjectNavMeshToGeometryBehavior : uint8
{
	/** Projects movement onto the closest geometry to the target location projected onto the navmesh. */
	UseClosestGeometry = 0        UMETA(DisplayName = "Use closest Geometry"),

	/** Projects movement onto the top most geometry in projection range of the navmesh */
	UseTopGeometry = 1     UMETA(DisplayName = "Use Top Geometry"),
};

namespace NavWalkingUtils
{
	void CheckClientSideNavigationSettings(const UMoverComponent* MoverComponent);
}

/**
 * NavWalkingMode: a default movement mode for traversing surfaces and movement bases by using an active navmesh when moving the actor rather than collision checks.
 * Note: This movement mode requires a NavMoverComponent be on the actor to function properly. This mode also contains some randomization to avoid navmesh look ups
 *	happening at the same time (which is fine for AI characters running on the server) but may cause issues if used on autonomous proxies.
 */
UCLASS(MinimalAPI, Blueprintable, BlueprintType)
class UNavWalkingMode : public UBaseMovementMode
{
	GENERATED_BODY()

public:
	UE_API UNavWalkingMode();
	
	UE_API virtual void Activate(const FMoverEventContext& Context, FName PrevModeName, const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, FMoverSyncState* OutSyncState, FMoverAuxStateContext* OutAuxState) override;
	UE_API virtual void Deactivate(const FMoverEventContext& Context, FName NextModeName, const FMoverSimContext& SimContext) override;
	UE_API virtual void SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState) override;
	UE_API virtual void GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const override;

	/**
	 * Whether or not the actor should sweep for collision geometry while walking.
	 */
	UPROPERTY(Category="NavMesh Movement", EditAnywhere, BlueprintReadWrite)
	bool bSweepWhileNavWalking;
	
	/** Whether to raycast to underlying geometry to better conform navmesh-walking actors */
	UPROPERTY(Category="NavMesh Movement", EditAnywhere, BlueprintReadOnly)
	bool bProjectNavMeshWalking;

	/**
	 * Scale of the total capsule height to use for projection from navmesh to underlying geometry in the upward direction.
	 * In other words, start the trace at [CapsuleHeight * NavMeshProjectionHeightScaleUp] above nav mesh.
	 */
	UPROPERTY(Category="NavMesh Movement", EditAnywhere, BlueprintReadWrite, meta=(editcondition = "bProjectNavMeshWalking", ClampMin="0", UIMin="0"))
	float NavMeshProjectionHeightScaleUp;

	/**
	 * Scale of the total capsule height to use for projection from navmesh to underlying geometry in the downward direction.
	 * In other words, trace down to [CapsuleHeight * NavMeshProjectionHeightScaleDown] below nav mesh.
	 */
	UPROPERTY(Category="NavMesh Movement", EditAnywhere, BlueprintReadWrite, meta=(editcondition = "bProjectNavMeshWalking", ClampMin="0", UIMin="0"))
	float NavMeshProjectionHeightScaleDown;

	/** How often we should raycast to project from navmesh to underlying geometry */
	UPROPERTY(Category="NavMesh Movement", EditAnywhere, BlueprintReadWrite, meta=(editcondition = "bProjectNavMeshWalking", ForceUnits="s"))
	float NavMeshProjectionInterval;

	/** Speed at which to interpolate agent navmesh offset between traces. 0: Instant (no interp) > 0: Interp speed") */
	UPROPERTY(Category="NavMesh Movement", EditAnywhere, BlueprintReadWrite, meta=(editcondition = "bProjectNavMeshWalking", ClampMin="0", UIMin="0"))
	float NavMeshProjectionInterpSpeed;

	/** What to do when we are projecting nav movement onto geometry from the nav mesh */
	UPROPERTY(Category = "NavMesh Movement", EditAnywhere, BlueprintReadWrite, meta=(editcondition = "bProjectNavMeshWalking"))
	EProjectNavMeshToGeometryBehavior ProjectNavMeshToGeometryBehavior = EProjectNavMeshToGeometryBehavior::UseTopGeometry;
	
	/** What should we do if we stray off the nav mesh? */
	UPROPERTY(Category = "NavMesh Movement", EditAnywhere, BlueprintReadWrite)
	EOffNavMeshBehavior BehaviorOffNavMesh = EOffNavMeshBehavior::SwitchToWalking;

	/** If attempting to stray off the nav mesh, should we attempt to slide along the edge instead? See @BehaviorOffNavMesh for cases where a sliding move can't be determined. */
	UPROPERTY(Category = "NavMesh Movement", EditAnywhere, BlueprintReadWrite)
	bool bSlideAlongNavMeshEdge = false;
	
	UPROPERTY(Transient)
	float NavMeshProjectionTimer;
	
	/** last known location projected on navmesh */
	FNavLocation CachedNavLocation;
	
	/**
	 * Project a location to navmesh to find adjusted height.
	 * @param TestLocation			Location to project
	 * @param OutNavFloorLocation	Location on navmesh
	 * @param NavData				NavigationDataInterface to search TestLocation for
	 * @return True if projection was performed (successfully or not)
	 */
	UE_API virtual bool FindNavFloor(const FVector& TestLocation, FNavLocation& OutNavFloorLocation, const INavigationDataInterface* NavData) const;

	// Returns the active turn generator. Note: you will need to cast the return value to the generator you expect to get, it can also be none
	UFUNCTION(BlueprintPure, Category=Mover)
	UE_API UObject* GetTurnGenerator();

	// Sets the active turn generator to use the class provided. Note: To set it back to the default implementation pass in none
	UFUNCTION(BlueprintCallable, Category=Mover)
	UE_API void SetTurnGeneratorClass(UPARAM(meta=(MustImplement="/Script/Mover.TurnGeneratorInterface", AllowAbstract="false")) TSubclassOf<UObject> TurnGeneratorClass);
	
protected:
	/** Associated Movement component that will actually move the actor */ 
	UPROPERTY(BlueprintReadOnly, Category="Nav Movement")
	TObjectPtr<UNavMoverComponent> NavMoverComponent;

	// Note: This isn't guaranteed to be valid at all times. It will be most of the time but re-entering this mode to call Activate() will search for NavData again and update it accordingly.
	TWeakInterfacePtr<const INavigationDataInterface> NavDataInterface;
	
	/** Use both WorldStatic and WorldDynamic channels for NavWalking geometry conforming */
	UPROPERTY(Category = "NavMesh Movement", EditAnywhere, BlueprintReadOnly, AdvancedDisplay)
	uint8 bProjectNavMeshOnBothWorldChannels : 1;

	/** Optional modular object for generating rotation towards desired orientation. If not specified, linear interpolation will be used. */
	UPROPERTY(EditAnywhere, Instanced, Category=Mover, meta=(ObjectMustImplement="/Script/Mover.TurnGeneratorInterface"))
	TObjectPtr<UObject> TurnGenerator;

	/** Switch collision settings for NavWalking mode (ignore world collisions) */
	UE_API virtual void SetCollisionForNavWalking(bool bEnable);
	
	/** Get Navigation data for the actor. Returns null if there is no associated nav data. */
	UE_API const INavigationDataInterface* GetNavData() const;

	/** Performs trace for ProjectLocationFromNavMesh */
	UE_API virtual void FindBestNavMeshLocation(const FVector& TraceStart, const FVector& TraceEnd, const FVector& CurrentFeetLocation, const FVector& TargetNavLocation, FHitResult& OutHitResult) const;

	/** 
	 * Attempts to better align navmesh walking actors with underlying geometry (sometimes 
	 * navmesh can differ quite significantly from geometry).
	 * Updates CachedProjectedNavMeshHitResult, access this for more info about hits.
	 */
	UE_API virtual FVector ProjectLocationFromNavMesh(float DeltaSeconds, const FVector& CurrentFeetLocation, const FVector& TargetNavLocation, float UpOffset, float DownOffset);

	
	UE_API virtual void OnRegistered(const FName ModeName, const FMoverSimContext& SimContext) override;
	UE_API virtual void OnUnregistered(const FMoverSimContext& SimContext) override;

	UE_API void CaptureFinalState(USceneComponent* UpdatedComponent, const FMovementRecord& Record, const FVector& AngularVelocityDegrees, FMoverDefaultSyncState& OutputSyncState) const;

	TObjectPtr<const UCommonLegacyMovementSettings> CommonLegacySettings;

	// Collision responses saved at the activation of this mode, to be restored as upon deactivation if the CDO can't be used
	ECollisionResponse CollideVsWorldStatic = ECollisionResponse::ECR_Block;
	ECollisionResponse CollideVsWorldDynamic = ECollisionResponse::ECR_Block;
};

#undef UE_API
