// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DefaultMovementSet/Modes/FlyingMode.h"
#include "SimpleFlyingMode.generated.h"

#define UE_API MOVER_API

/**
 * Basic flying mode that will move exactly where inputs request. Easy to derive from to create custom behaviors
 */
UCLASS(BlueprintType, Abstract)
class USimpleFlyingMode : public UFlyingMode
{
	GENERATED_BODY()

public:

	UE_API virtual void GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const override;

	// Override this to make a simple flying mode
	UFUNCTION(BlueprintNativeEvent, meta = (DisplayName = "Generate Simple Flying Move"))
	UE_API void GenerateFlyingMove(UPARAM(ref) FMoverTickStartData& StartState, float DeltaSeconds, const FVector& DesiredVelocity,
		const FQuat& DesiredFacing, const FQuat& CurrentFacing, UPARAM(ref) FVector& InOutAngularVelocityDegrees, UPARAM(ref) FVector& InOutVelocity);

	// If this value is greater or equal to 0, this will override the max speed read from the CommonLegacyMovementSettings
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mover|Flying Settings", meta = (ForceUnits = "cm/s"))
	float MaxSpeedOverride = -1.0f;
};

#undef UE_API
