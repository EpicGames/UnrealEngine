// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SimpleWalkingMode.h"
#include "SimpleSpringWalkingMode.generated.h"

#define UE_API MOVER_API

/**
 * A walking mode that uses a critically damped spring for translation and rotation.
 * The strength of the critically damped spring is set via smoothing times (separate for translation and rotation)
 */
UCLASS(BlueprintType)
class USimpleSpringWalkingMode : public USimpleWalkingMode
{
	GENERATED_BODY()

public:
	UE_API virtual void SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState) override;
	UE_API virtual void GenerateWalkMove_Implementation(FMoverTickStartData& StartState, float DeltaSeconds, const FMoverSimContext& SimContext, const FVector& DesiredVelocity,
									 const FQuat& DesiredFacing, const FQuat& CurrentFacing, FVector& InOutAngularVelocityDegrees, FVector& InOutVelocity) override;

protected:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mover|Spring Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "s"))
	float VelocitySmoothingTime = 0.2f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mover|Spring Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "s"))
	float FacingSmoothingTime = 0.2f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mover|Spring Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s"))
	// Below this speed we set velocity to 0
	float VelocityDeadzoneThreshold = 0.1f;
};

#undef UE_API
