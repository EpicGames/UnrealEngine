// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DefaultMovementSet/Modes/WalkingMode.h"
#include "SimpleWalkingMode.generated.h"

#define UE_API MOVER_API

/**
 * Basic walking mode that implements the ground based walking
 */
UCLASS(BlueprintType, Abstract)
class USimpleWalkingMode : public UWalkingMode
{
	GENERATED_BODY()

public:
	UE_API virtual void OnRegistered(const FName ModeName, const FMoverSimContext& SimContext) override;
	UE_API virtual void GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const override;

	// Override this to make a simple walking mode
	UFUNCTION(BlueprintNativeEvent, meta = (DisplayName = "Generate Simple Walk Move"))
	UE_API void GenerateWalkMove(UPARAM(ref) FMoverTickStartData& StartState, float DeltaSeconds, const FMoverSimContext& SimContext, const FVector& DesiredVelocity,
	                                     const FQuat& DesiredFacing, const FQuat& CurrentFacing, UPARAM(ref) FVector& InOutAngularVelocityDegrees, UPARAM(ref) FVector& InOutVelocity);

	// If this value is greater or equal to 0, this will override the max speed read from the common legacy shared walk settings
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mover|Walking Settings", meta = (ForceUnits = "cm/s"))
	float MaxSpeedOverride = -1.0f;

protected:
	static const FName DidGenerateMoveEntry;	// boolean blackboard entry name, set whenever GenerateWalkMove finishes - valid through next frame
};

#undef UE_API
