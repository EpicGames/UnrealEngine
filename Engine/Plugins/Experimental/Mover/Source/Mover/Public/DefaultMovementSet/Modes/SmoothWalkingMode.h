// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SimpleWalkingMode.h"
#include "DefaultMovementSet/Modes/WalkingMode.h"
#include "SmoothWalkingMode.generated.h"

#define UE_API MOVER_API

/**
 * A walking mode that provides a simplified version of the default walking mode model with additional options for smoothing.
 */
UCLASS(BlueprintType, Experimental)
class USmoothWalkingMode : public USimpleWalkingMode
{
	GENERATED_BODY()

public:
	UE_API virtual void SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState) override;
	UE_API virtual void GenerateWalkMove_Implementation(FMoverTickStartData& StartState, float DeltaSeconds, const FMoverSimContext& SimContext, const FVector& DesiredVelocity,
									 const FQuat& DesiredFacing, const FQuat& CurrentFacing, FVector& InOutAngularVelocityDegrees, FVector& InOutVelocity) override;


protected: // Velocity Controls

	// Base acceleration to apply when the desired velocity magnitude is greater than the current velocity.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mover|Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s^2"))
	float Acceleration = 1500.0f;

	// Base deceleration to apply when the desired velocity magnitude is less than the current velocity
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mover|Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s^2"))
	float Deceleration = 1500.0f;

	/**
	 * Value between 0 and 1 that controls how the acceleration is applied. When set to 1 this will effectively add the acceleration in the direction 
	 * of the desired velocity on top of the current velocity then clip the result. This emulates the behavior of the default walking mode. When set 
	 * to 0 it will apply an acceleration which changes the current velocity directly towards the desired velocity. This makes directional changes in 
	 * velocity faster and more regular, always using a fixed acceleration, but can cause the overall velocity magnitude to dip during turning.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mover|Advanced Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1"))
	float DirectionalAccelerationFactor = 1.0f;

	/**
	 * Applies an additional force that rotates the velocity to make it face the desired direction of travel. This allows quick turns which don't 
	 * lose speed but adds additional acceleration to the system. Valid values range roughly between 0 and 100, where 10 would be considered already 
	 * quite a strong turning force and 100 will produce near instant turns (before smoothing).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mover|Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0"))
	float TurningStrength = 10.0f;

	// Controls how much smoothing is applied to the velocity changes of the movement when accelerating. Set to zero to apply no smoothing.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mover|Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "s"))
	float AccelerationSmoothingTime = 0.1f;

	// Controls how much smoothing is applied to the velocity changes of the movement when decelerating. Set to zero to apply no smoothing.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mover|Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "s"))
	float DecelerationSmoothingTime = 0.1f;

	/**
	 * When the velocity is smoothed it will naturally lag behind the unsmoothed target. This factor controls how much this is compensated for by
	 * tracking a future target that is the appropriate amount of time in the future when accelerating. Setting this to 0 will produce smoother, more 
	 * S-shaped velocity profiles but may reduce the feeling of responsiveness. Setting this to 1 increases the feeling of responsiveness but makes 
	 * the initial change in velocity less smooth and reduces the effective "lead-in".
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mover|Advanced Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1"))
	float AccelerationSmoothingCompensation = 0.0f;

	// This parameter acts the same as AccelerationSmoothingCompensation but is applied during deceleration instead
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mover|Advanced Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1"))
	float DecelerationSmoothingCompensation = 0.0f;

	// Controls the point at which the velocity will "snap" to the desired velocity once it is close enough 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mover|Advanced Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s"))
	float VelocityDeadzoneThreshold = 0.01f;

	// Controls the point at which the acceleration will "snap" to zero when the desired velocity is reached
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mover|Advanced Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s^2"))
	float AccelerationDeadzoneThreshold = 0.001f;

	// Controls how quickly the built-up internal velocity will be modified when the character's movement is influenced by the outside factors such 
	// as collisions. Setting this to a small value will mean that collisions and other outside influences on the character's velocity "reset" 
	// the built-up internal velocity quickly. Larger values will mean that the character loses less momentum and takes less time to re-accumulate 
	// internal velocity in situations such as glancing collisions and other short external impulses such as pushes.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mover|Advanced Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "s"))
	float OutsideInfluenceSmoothingTime = 0.05f;

protected: // Facing Controls

	// Controls how much smoothing is applied to the facing direction. Set to zero to apply no smoothing.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mover|Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "s"))
	float FacingSmoothingTime = 0.25f;

	// Smooths facing using a double spring rather than a single spring. This produces a more S-shaped profile with a shorter "lead-out".
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mover|Advanced Smooth Walking Settings")
	bool bSmoothFacingWithDoubleSpring = true;

	// Controls the point at which the facing will "snap" to the desired facing once it is close enough 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mover|Advanced Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "deg"))
	float FacingDeadzoneThreshold = 0.1f;

	// Controls the point at which the angular velocity will "snap" to zero when the desired facing is reached
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mover|Advanced Smooth Walking Settings", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "deg/s"))
	float AngularVelocityDeadzoneThreshold = 0.01f;

};

#undef UE_API
