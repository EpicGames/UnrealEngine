// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovementMode.h"

#include "SwimmingMode.generated.h"

#define UE_API MOVER_API

class UCommonLegacyMovementSettings;
struct FFloorCheckResult;
struct FRelativeBaseInfo;
struct FMovementRecord;

// Controls for the Swimming Movement
USTRUCT(BlueprintType)
struct FSwimmingControlSettings
{
	GENERATED_BODY()

	FSwimmingControlSettings()
	{
	}

	// At or below this depth, cancel and disallow crouching.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float CancelCrouchImmersionDepth = 28.50f;

	// At or below this depth, start swimming in DBNO
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float DBNOSwimImmersionDepth = 50.00f;

	// Max acceleration.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float MaxAcceleration = 350.00f;

	// Max acceleration while sprinting.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float MaxAccelerationSprinting = 490.00f;

	// Braking deceleration (decel when letting go of input)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float BrakingDeceleration = 400.00f;

	// Max speed when not sprinting and moving normally (before water velocity is applied)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float MaxNormalSpeed = 0.50f;

	// Max sprint speed (before water velocity is applied)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float MaxSprintSpeed = 1.00f;

	// Min speed required (relative to water) to maintain sprint while jumping, otherwise will stop sprint (and change anims).
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float MinSprintJumpSpeed = -0.03f;

	// If accel deviates from velocity by this angle while sprint jumping in air, stop sprint (and change anims).
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float SprintJumpAirAccelAngleLimit = 0.25f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float SprintRetriggerDelay = 210.00f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float SprintDelayAfterFiring = 385.00f;

	// Max speed when targeting (before water velocity is applied)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float MaxTargetingSpeed = 300.00f;

	// Directional multiplier when moving mostly backwards
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float BackwardsSpeedMultiplier = 50.00f;

	// Directional multiplier is applied when dot product of velocity and player facing direction is < this.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float BackwardsSpeedCosAngle = 0.10f;

	// Speed multiplier when moving off angle (velocity and acceleration divergent)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float AngledSpeedMultiplier = 0.70f;

	// Angled multiplier is applied when dot product of velocity and acceleration is < this.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float AngledSpeedCosAngle = 250.00f;

	// Friction, ie how floaty or snappy is changing direction.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float Friction = 0.50f;

	// Friction, ie how floaty or snappy is changing direction.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float FrictionSprinting = -0.30f;

	// If Dot(Velocity, Acceleration) < this value, apply FrictionDirectionChangeMultiplier to friction value used. Allows lower friction when changing direction hard, which slows velocity change.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float FrictionDirectionChangeDot = 0.40f;

	// Friction multiplier (usually < 1) when changing direction.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float FrictionDirectionChangeMultiplier = 0.75f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float MaxSpeedUp = 500.00f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float MaxSpeedDown = 1000.00f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float MaxHorizontalEntrySpeed = 1200.00f;

	// Multiplier to water force acceleration in direction of current.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float WaterForceMultiplier = 2.00f;

	// Multiplier applied on the top of WaterForceMultiplier, to water force acceleration in direction of current. Used only for inherited objects.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float WaterForceSecondMultiplier = 1.00f;

	// Max water force, after WaterVelocity * (WaterForceMultiplier * WaterForceSecondMultiplier).
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float MaxWaterForce = 400.00f;

	// At or above this depth, use max velocity. Interps down to WaterVelocityMinMultiplier at wading depth (where player can start swimming)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float WaterVelocityDepthForMax = 175.00f;

	// Min velocity multiplier applied when depth equals min swimming depth (where they transition from wading to swimming). Interps between this and 1.0 at WaterVelocityDepthForMax.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float WaterVelocityMinMultiplier = 0.50f;

	// Max time step allowed, to prevent huge forces on hitches.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float WaterSimMaxTimeStep = 0.10f;

	// Simulation sub-step time allowed for higher quality movement (local player and server).
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float WaterSimSubStepTime = 0.05f;

	// Bobbing: 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float BobbingMaxForce = 3800.00f;

	// Bobbing: Slow down strongly when within this tolerance of the ideal immersion depth. Normally we apply drag only when going away from the ideal depth, this allows some slowdown when approaching it.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float BobbingIdealDepthTolerance = 7.50f;

	// Bobbing: friction/drag opposed to downward velocity, linear multiplier per second
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float BobbingFrictionDown = 5.00f;

	// Bobbing: friction/drag opposed to downward velocity, squared with velocity per second. Ramps up faster with higher speeds, less effect at low speeds.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float BobbingExpDragDown = 0.05f;

	// Bobbing: friction/drag opposed to downward velocity, linear multiplier per second. Only used when fully submerged (replaces other value).
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float BobbingFrictionDownSubmerged = 7.50f;

	// Bobbing: friction/drag opposed to upward velocity, squared with velocity per second. Ramps up faster with higher speeds, less effect at low speeds.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float BobbingExpDragDownSubmerged = 0.10f;

	// Bobbing: friction/drag opposed to upward velocity, linear multiplier per second
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float BobbingFrictionUp = 7.00f;

	// Bobbing: friction/drag opposed to upward velocity, squared with velocity per second. Ramps up faster with higher speeds, less effect at low speeds.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float BobbingExpDragUp = 0.20f;

	// Bobbing: friction multiplier, multiplies the fluid friction value.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float BobbingFrictionMultiplier = 1.f;
	
	// Bobbing: multiplier for the exponential drag.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float BobbingExpDragMultiplier = 1.f;

	// Bobbing: multiplier when in sprint boost to keep from popping up and out as much.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float BoostDragMultiplier = 1.85f;

	// Multiplies ground's jump speed
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SwimmingControl)
	float JumpMultiplier = 1.5f;
};

/**
 * SwimmingMode: a default movement mode for traversing water volumes
 */
UCLASS(MinimalAPI, Blueprintable, BlueprintType)
class USwimmingMode : public UBaseMovementMode
{
	GENERATED_UCLASS_BODY()

public:

	UE_API virtual void GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const override;

	UE_API virtual void SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState) override;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Water Control")
	FSwimmingControlSettings SurfaceSwimmingWaterControlSettings;

protected:
	UE_API virtual void OnRegistered(const FName ModeName, const FMoverSimContext& SimContext) override;
	UE_API virtual void OnUnregistered(const FMoverSimContext& SimContext) override;

	UE_API virtual bool AttemptJump(const FSimulationTickParams& Params, float UpwardsSpeed, FMoverTickEndData& Output);

	UE_API void CaptureFinalState(USceneComponent* UpdatedComponent, const FFloorCheckResult& FloorResult, const FMovementRecord& Record, FMoverDefaultSyncState& OutputSyncState) const;

	UE_API FRelativeBaseInfo UpdateFloorAndBaseInfo(const FFloorCheckResult& FloorResult) const;

	TObjectPtr<const UCommonLegacyMovementSettings> CommonLegacySettings;

	float OriginalCapsuleHalfHeight = 75.f;
};

#undef UE_API
