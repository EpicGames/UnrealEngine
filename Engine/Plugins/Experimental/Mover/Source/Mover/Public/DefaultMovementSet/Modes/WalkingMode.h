// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovementMode.h"
#include "MoveLibrary/ModularMovement.h"
#include "WalkingMode.generated.h"

#define UE_API MOVER_API

class UCommonLegacyMovementSettings;
struct FFloorCheckResult;
struct FRelativeBaseInfo;
struct FMovementRecord;

// Behavior policy for performing floor checks in walking mode when no movement is occurring. 
UENUM(BlueprintType)
enum class EStaticFloorCheckPolicy : uint8
{
	// Always perform floor checks, even when not moving. You may want this if static bases may disappear from underneath.
	Always = 0				UMETA(DisplayName = "Always"),

	// Only perform floor checks when not moving IF we're on a dynamic movement base
	OnDynamicBaseOnly = 1	UMETA(DisplayName = "OnDynamicBaseOnly"),
};



/**
 * WalkingMode: a default movement mode for traversing surfaces and movement bases (walking, running, sneaking, etc.)
 */
UCLASS(MinimalAPI, Blueprintable, BlueprintType)
class UWalkingMode : public UBaseMovementMode
{
	GENERATED_UCLASS_BODY()

public:
	
	UE_API virtual void Activate(const FMoverEventContext& Context, FName PrevModeName, const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, FMoverSyncState* OutSyncState, FMoverAuxStateContext* OutAuxState) override;
	UE_API virtual void GenerateMove_Implementation(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const override;
	UE_API virtual void SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState) override;

	// Returns the active turn generator. Note: you will need to cast the return value to the generator you expect to get, it can also be none
	UFUNCTION(BlueprintPure, Category=Mover)
	UE_API UObject* GetTurnGenerator();

	// Sets the active turn generator to use the class provided. Note: To set it back to the default implementation pass in none
	UFUNCTION(BlueprintCallable, Category=Mover)
	UE_API void SetTurnGeneratorClass(UPARAM(meta=(MustImplement="/Script/Mover.TurnGeneratorInterface", AllowAbstract="false")) TSubclassOf<UObject> TurnGeneratorClass);

protected:

	/** Choice of behavior for floor checks while not moving.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	EStaticFloorCheckPolicy FloorCheckPolicy = EStaticFloorCheckPolicy::OnDynamicBaseOnly;

	/** Optional modular object for generating rotation towards desired orientation. If not specified, linear interpolation will be used. */
	UPROPERTY(EditAnywhere, Instanced, Category=Mover, meta=(ObjectMustImplement="/Script/Mover.TurnGeneratorInterface"))
	TObjectPtr<UObject> TurnGenerator;

	UE_API virtual void OnRegistered(const FName ModeName, const FMoverSimContext& SimContext) override;
	UE_API virtual void OnUnregistered(const FMoverSimContext& SimContext) override;

	UE_API void CaptureFinalState(const FMoverDefaultSyncState* StartingState, USceneComponent* UpdatedComponent, bool bDidAttemptMovement, const FFloorCheckResult& FloorResult, const FMovementRecord& Record, const FMoverTimeStep& TimeStep, const FVector& AngularVelocityDegrees, FMoverDefaultSyncState& OutputSyncState) const;

	UE_API FRelativeBaseInfo UpdateFloorAndBaseInfo(const FFloorCheckResult& FloorResult, FVector CurrentLocation) const;

	TObjectPtr<const UCommonLegacyMovementSettings> CommonLegacySettings;
};

#undef UE_API
