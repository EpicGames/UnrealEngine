// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovementModifier.h"
#include "StanceModifier.generated.h"

#define UE_API MOVER_API

class UCapsuleComponent;
class UCharacterMoverComponent;

UENUM(BlueprintType)
enum class EStanceMode : uint8
{
	// Invalid default stance
	Invalid = 0,
	// Actor goes into crouch
	Crouch,
	// Actor goes into prone - not currently implemented
	Prone,
};

/**
 * Stances: Applies settings to the actor to make them go into different stances like crouch or prone(not implemented), affects actor maxacceleration and capsule height
 * Note: This modifier currently uses the CDO of the actor to reset values to "standing" values.
 *		 This modifier also assumes the actor is using a capsule as it's updated component for now
 */
USTRUCT(BlueprintType)
struct FStanceModifier : public FMovementModifierBase
{
	GENERATED_BODY()

public:
	UE_API FStanceModifier();
	virtual ~FStanceModifier() override = default;

	EStanceMode ActiveStance;
	
	UE_API virtual bool HasGameplayTag(FGameplayTag TagToFind, bool bExactMatch) const override;
	UE_API virtual void GetGameplayTags(FGameplayTagContainer& InOutTags) const override;
	
	/** Fired when this modifier is activated. */
	UE_API virtual void OnStart(UMoverComponent* MoverComp, const FMoverTimeStep& TimeStep, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState) override;
	
	/** Fired when this modifier is deactivated. */
	UE_API virtual void OnEnd(UMoverComponent* MoverComp, const FMoverTimeStep& TimeStep, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState) override;
	
	/** Fired just before a Substep */
	UE_API virtual void OnPreMovement(UMoverComponent* MoverComp, const FMoverTimeStep& TimeStep) override;

	/** Fired after a Substep */
	UE_API virtual void OnPostMovement(UMoverComponent* MoverComp, const FMoverTimeStep& TimeStep, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState) override;
	
	// @return newly allocated copy of this FMovementModifier. Must be overridden by child classes
	UE_API virtual FMovementModifierBase* Clone() const override;

	UE_API virtual void NetSerialize(FArchive& Ar) override;

	UE_API virtual UScriptStruct* GetScriptStruct() const override;

	UE_API virtual FString ToSimpleString() const override;

	UE_API virtual void AddReferencedObjects(class FReferenceCollector& Collector) override;

	UE_API virtual bool CanExpand(const UCharacterMoverComponent* MoverComp) const;
	
	// Whether expanding should be from the base of the capsule or not
	UE_API virtual bool ShouldExpandingMaintainBase(const UCharacterMoverComponent* MoverComp) const;

protected:
	// Modifies the updated component casted to a capsule component
	UE_API virtual void AdjustCapsule(UMoverComponent* MoverComp, float OldHalfHeight, float NewHalfHeight, float NewEyeHeight);

	// Applies any movement settings like acceleration or max speed changes
	UE_API void ApplyMovementSettings(UMoverComponent* MoverComp);
	
	// Reverts any movement settings like acceleration or max speed changes
	UE_API void RevertMovementSettings(UMoverComponent* MoverComp);
};

template<>
struct TStructOpsTypeTraits< FStanceModifier > : public TStructOpsTypeTraitsBase2< FStanceModifier >
{
	enum
	{
		WithCopy = true
	};
};

#undef UE_API
