// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoverSimulationTypes.h"

#include "MoverMontageSimulationTypes.generated.h"

#define UE_API MOVER_API

class UAnimMontage;


// Per-move montage state snapshot produced by the simulation each substep.
// Not serialized over the network: this is an in-process simulation->game thread scratch struct.
USTRUCT()
struct FMoverSimDrivenMontageEntry
{
	GENERATED_BODY()

	// Montage asset; key for game thread tracking.
	UPROPERTY(VisibleAnywhere, Category = Mover)
	TObjectPtr<UAnimMontage> Montage = nullptr;

	// Asset name of Montage, kept in sync with the Montage pointer for CVD display
	// (UObject references cannot cross the CVD trace boundary). Debug-only: not net serialized.
	UPROPERTY(VisibleAnywhere, Category = Mover, meta = (DisplayName = "MontageName"))
	FName Debug_MontageName;

	// Simulation-authoritative position at the END of this substep (seconds, unscaled by PlayRate)
	UPROPERTY(VisibleAnywhere, Category = Mover)
	float CurrentPosition = 0.f;

	// PlayRate at which the montage is playing
	UPROPERTY(VisibleAnywhere, Category = Mover)
	float PlayRate = 1.f;

	// Simulation time (ms) at the end of the substep that produced this entry.
	// FMoverSimDrivenMontageData::Interpolate uses this to reconstruct the display-time simulation time
	// and deliver lifecycle events precisely rather than taking them blindly from the To frame.
	UPROPERTY(VisibleAnywhere, Category = Mover)
	float FrameSimTimeMs = 0.f;

	// Simulation time (ms) when the montage move was first activated. Informational for CVD.
	UPROPERTY(VisibleAnywhere, Category = Mover)
	float StartSimTimeMs = 0.f;

	// Simulation time (ms) when blend-out was first triggered. -1 means not yet triggered.
	// Interpolate fires bShouldBlendOut exactly when the interpolated display time crosses this threshold.
	UPROPERTY(VisibleAnywhere, Category = Mover)
	float BlendOutSimTimeMs = -1.f;

	// Simulation time (ms) when the montage was first detected as finished. -1 means not yet.
	UPROPERTY(VisibleAnywhere, Category = Mover)
	float FinishSimTimeMs = -1.f;

	// Simulation has computed that blend-out should begin. Game thread calls Montage_Stop(BlendOutTimeSeconds) once.
	UPROPERTY(VisibleAnywhere, Category = Mover)
	bool bShouldBlendOut = false;

	// Blend-out duration (seconds) to pass to Montage_Stop when bShouldBlendOut first becomes true
	UPROPERTY(VisibleAnywhere, Category = Mover)
	float BlendOutTimeSeconds = 0.f;

	// Simulation has finished the move entirely. Game thread calls Montage_Stop(0) or lets existing blend-out finish.
	UPROPERTY(VisibleAnywhere, Category = Mover)
	bool bIsFinished = false;
};

// Simulation output for montage-driving layered moves. Stored in AdditionalOutputData so it flows through
// the FSimulationOutputData interpolation path (CurrentPosition is lerped between simulation frames).
USTRUCT()
struct FMoverSimDrivenMontageData : public FMoverDataStructBase
{
	GENERATED_BODY()

	// One entry per active montage-driving layered move on the simulation this substep
	UPROPERTY(VisibleAnywhere, Category = Mover)
	TArray<FMoverSimDrivenMontageEntry> MontageStates;

	virtual UScriptStruct* GetScriptStruct() const override { return StaticStruct(); }
	virtual FMoverDataStructBase* Clone() const override { return new FMoverSimDrivenMontageData(*this); }

	// Lerps CurrentPosition between simulation frames for smooth game thread animation.
	// Lifecycle flags (bShouldBlendOut, bIsFinished) are recomputed from the interpolated simulation time
	// crossing BlendOutSimTimeMs/FinishSimTimeMs, so they fire at the precise display moment rather than
	// snapping to the To-frame value.
	UE_API virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override;
};

template<>
struct TStructOpsTypeTraits<FMoverSimDrivenMontageData> : public TStructOpsTypeTraitsBase2<FMoverSimDrivenMontageData>
{
	enum { WithCopy = true };
};

#undef UE_API
