// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "Components/ActorComponent.h"
#include "MoverComponent.h"
#include "AI/RVOAvoidanceInterface.h"
#include "AI/Navigation/NavigationAvoidanceTypes.h"
#include "GameFramework/NavMovementInterface.h"
#include "UObject/WeakInterfacePtr.h"

#include "NavMoverComponent.generated.h"

#define UE_API MOVER_API

/**
 * NavMoverComponent: Responsible for implementing INavMoveInterface with UMoverComponent so pathfinding and other forms of navigation movement work.
 * This component also caches the input given to it that is then consumed by the mover system.
 * This component supports a basic version of the RVO avoidance interface to make it usable with Detour Crowd Avoidance. It does not support force based 
 * RVO avoidance, but this or child component would be a good place to update/evaluate Avoidance information and apply effects to the Mover Simulation.
 * Note: This component relies on the parent actor having a MoverComponent as well. By default, this component only has a reference to MoverComponent meaning
 * we use other ways (such as gameplay tags for the active movement mode) to check for state rather than calling specific functions on the active MoverComponent.
 */
UCLASS(MinimalAPI, BlueprintType, meta = (BlueprintSpawnableComponent))
class UNavMoverComponent : public UActorComponent, public INavMovementInterface, public IRVOAvoidanceInterface
{
	GENERATED_BODY()

public:
	UE_API UNavMoverComponent();

	/** Properties that define how the component can move. */
	UPROPERTY(EditAnywhere, Category="Nav Movement", meta = (DisplayName = "Movement Capabilities", Keywords = "Nav Agent"))
	FNavAgentProperties NavAgentProps;

	/** Expresses runtime state of character's movement. Put all temporal changes to movement properties here */
	UPROPERTY()
	FMovementProperties MovementState;
	
	/** No default value, for now it's assumed to be valid if GetAvoidanceManager() returns non-NULL. */
	UPROPERTY(Category="Avoidance", VisibleAnywhere, BlueprintReadOnly, AdvancedDisplay)
	int32 AvoidanceUID;
	
	/** Moving actor's group mask */
	UPROPERTY(Category="Avoidance", EditAnywhere, BlueprintReadOnly, AdvancedDisplay)
	FNavAvoidanceMask AvoidanceGroup;
	
	UFUNCTION(BlueprintCallable, Category = "Nav Movement|Avoidance")
	UE_API void SetAvoidanceGroupMask(const FNavAvoidanceMask& GroupMask);
	
	/** Will avoid other agents if they are in one of specified groups */
	UPROPERTY(Category="Avoidance", EditAnywhere, BlueprintReadOnly, AdvancedDisplay)
	FNavAvoidanceMask GroupsToAvoid;
	
	UFUNCTION(BlueprintCallable, Category = "Nav Movement|Avoidance")
	UE_API void SetGroupsToAvoidMask(const FNavAvoidanceMask& GroupMask);
	
	/** Will NOT avoid other agents if they are in one of specified groups, higher priority than GroupsToAvoid */
	UPROPERTY(Category="Avoidance", EditAnywhere, BlueprintReadOnly, AdvancedDisplay)
	FNavAvoidanceMask GroupsToIgnore;
	
	UFUNCTION(BlueprintCallable, Category = "Nav Movement|Avoidance")
	UE_API void SetGroupsToIgnoreMask(const FNavAvoidanceMask& GroupMask);
	
	/** De facto default value 0.5 (due to that being the default in the avoidance registration function), indicates RVO behavior. */
	UPROPERTY(Category="Avoidance", EditAnywhere, BlueprintReadOnly)
	float AvoidanceWeight;
	
	UPROPERTY(Category="Avoidance", EditAnywhere, BlueprintReadOnly, meta=(ForceUnits=cm))
	float AvoidanceConsiderationRadius;
	
protected:
	/** associated properties for nav movement */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Nav Movement")
	FNavMovementProperties NavMovementProperties;

	/** Keeps track of the last game frame we consumed nav movement. */
	uint64 GameFrameNavMovementConsumed = 0;

	/** Keeps track of the last game frame we requested nav movement. */
	uint64 GameFrameNavMovementRequested = 0;
	
	// Used to store movement input intent from requested moves
	FVector CachedNavMoveInputIntent = FVector::ZeroVector;
	// Used to store movement input velocity from requested moves
	FVector CachedNavMoveInputVelocity = FVector::ZeroVector;
	
	/** object implementing IPathFollowingAgentInterface. Is private to control access to it.
	 *	@see SetPathFollowingAgent, GetPathFollowingAgent */
	TWeakInterfacePtr<IPathFollowingAgentInterface> PathFollowingComp;

	/** Associated Movement component that will actually move the actor */ 
	UPROPERTY()
	TWeakObjectPtr<UMoverComponent> MoverComponent;
	
public:
	UE_API virtual void InitializeComponent() override;
	UE_API virtual void BeginPlay() override;
	
	/** Get the owner of the object consuming nav movement */
	virtual UObject* GetOwnerAsObject() const override { return GetOwner(); }
	
	/** Get the component this movement component is updating */
	virtual TObjectPtr<UObject> GetUpdatedObject() const override { return MoverComponent.IsValid() ? MoverComponent->GetUpdatedComponent() : nullptr; }

	/** Get axis-aligned cylinder around this actor, used for simple collision checks in nav movement */
	UE_API virtual void GetSimpleCollisionCylinder(float& CollisionRadius, float& CollisionHalfHeight) const override;

	/** Returns collision extents vector for this object, based on GetSimpleCollisionCylinder. */
	UE_API virtual FVector GetSimpleCollisionCylinderExtent() const override;
	
	/** Get forward vector of the object being driven by nav movement */
	UE_API virtual FVector GetForwardVector() const override;
	
	/** Get the current velocity of the movement component */
	UFUNCTION(BlueprintCallable, Category="AI|Components|NavMovement")
	UE_API virtual FVector GetVelocityForNavMovement() const override;

	/** Get the max speed of the movement component */
	UFUNCTION(BlueprintCallable, Category="AI|Components|NavMovement")
	UE_API virtual float GetMaxSpeedForNavMovement() const override;

	// Overridden to also call StopActiveMovement().
	UE_API virtual void StopMovementImmediately() override;

	// Writes internal cached requested velocities to the MoveInput passed in. Returns true if it had move input to write.
	UFUNCTION(BlueprintCallable, Category="AI|Components|NavMovement")
	UE_API bool ConsumeNavMovementData(FVector& OutMoveInputIntent, FVector& OutMoveInputVelocity);
	
	/** Returns location of controlled actor - meaning center of collision bounding box */
	UE_API virtual FVector GetLocation() const override;
	/** Returns location of controlled actor's "feet": the center bottom of its collision bounding box at its current location */
	UE_API virtual FVector GetFeetLocation() const override;
	/** Returns location of controlled actor's "feet": the center bottom of its collision bounding box, as if it was at a specific location */
	UE_API virtual FVector GetFeetLocationAt(FVector ComponentLocation) const;
	/** Returns based location of controlled actor */
	UE_API virtual FBasedPosition GetFeetLocationBased() const override;

	/** Set nav agent properties from an object */
	UE_API virtual void UpdateNavAgent(const UObject& ObjectToUpdateFrom) override;
	
	/** path following: request new velocity */
	UE_API virtual void RequestDirectMove(const FVector& MoveVelocity, bool bForceMaxSpeed) override;

	/** path following: request new move input (normal vector = full strength) */
	UE_API virtual void RequestPathMove(const FVector& MoveInput) override;

	/** check if current move target can be reached right now if positions are matching
	 *  (e.g. performing scripted move and can't stop) */
	UE_API virtual bool CanStopPathFollowing() const override;

	/** Get Nav movement props struct this component uses */
	virtual FNavMovementProperties* GetNavMovementProperties() override { return &NavMovementProperties; }
	/** Returns the NavMovementProps(const) */
	virtual const FNavMovementProperties& GetNavMovementProperties() const override{ return NavMovementProperties; } 
	
	UE_API virtual void SetPathFollowingAgent(IPathFollowingAgentInterface* InPathFollowingAgent) override;
	UE_API virtual IPathFollowingAgentInterface* GetPathFollowingAgent() override;
	UE_API virtual const IPathFollowingAgentInterface* GetPathFollowingAgent() const override;

	/** Returns the NavAgentProps(const) */
	UE_API virtual const FNavAgentProperties& GetNavAgentPropertiesRef() const override;
	/** Returns the NavAgentProps */
	UE_API virtual FNavAgentProperties& GetNavAgentPropertiesRef() override;

	/** Resets runtime movement state to character's movement capabilities */
	UE_API virtual void ResetMoveState() override;

	/** Returns true if path following can start */
	UE_API virtual bool CanStartPathFollowing() const override;

	/** Returns true if currently crouching */
	UE_API virtual bool IsCrouching() const override;
	
	/** Returns true if currently falling (not flying, in a non-fluid volume, and not on the ground) */ 
	UE_API virtual bool IsFalling() const override;

	/** Returns true if currently moving on the ground (e.g. walking or driving) */
	UE_API virtual bool IsMovingOnGround() const override;
	
	/** Returns true if currently swimming (moving through a fluid volume) */
	UFUNCTION(BlueprintCallable, Category="AI|Components|NavMovement")
	UE_API virtual bool IsSwimming() const override;

	/** Returns true if currently flying (moving through a non-fluid volume without resting on the ground) */
	UE_API virtual bool IsFlying() const override;
	
	/** BEGIN IRVOAvoidanceInterface */
	UE_API virtual void SetRVOAvoidanceUID(int32 UID) override;
	UE_API virtual int32 GetRVOAvoidanceUID() override;
	UE_API virtual void SetRVOAvoidanceWeight(float Weight) override;
	UE_API virtual float GetRVOAvoidanceWeight() override;
	UE_API virtual FVector GetRVOAvoidanceOrigin() override;
	UE_API virtual float GetRVOAvoidanceRadius() override;
	UE_API virtual float GetRVOAvoidanceHeight() override;
	UE_API virtual float GetRVOAvoidanceConsiderationRadius() override;
	UE_API virtual FVector GetVelocityForRVOConsideration() override;
	UE_API virtual void SetAvoidanceGroupMask(int32 GroupFlags) override;
	UE_API virtual int32 GetAvoidanceGroupMask() override;
	UE_API virtual void SetGroupsToAvoidMask(int32 GroupFlags) override;
	UE_API virtual int32 GetGroupsToAvoidMask() override;
	UE_API virtual void SetGroupsToIgnoreMask(int32 GroupFlags) override;
	UE_API virtual int32 GetGroupsToIgnoreMask() override;
	/** END IRVOAvoidanceInterface */
};

#undef UE_API
