// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovementMode.h"
#include "CommonLegacyMovementSettings.generated.h"


/**
 * CommonLegacyMovementSettings: collection of settings that are shared between several of the legacy movement modes
 */ 
UCLASS(MinimalAPI, BlueprintType)
class UCommonLegacyMovementSettings : public UObject, public IMovementSettingsInterface
{
	GENERATED_BODY()

	virtual FString GetDisplayName() const override { return GetName(); }

public:
	/**
	 * If true, the actor will remain upright with gravity despite any rotation applied to the actor
	 */
	UPROPERTY(Category="General", EditAnywhere, BlueprintReadWrite)
	bool bShouldRemainVertical = true;
	
	// What movement mode to use when on the ground.
	UPROPERTY(Category="General", EditAnywhere, BlueprintReadWrite)
	FName GroundMovementModeName = DefaultModeNames::Walking;

	// What movement mode to use when airborne.
	UPROPERTY(Category="General", EditAnywhere, BlueprintReadWrite)
	FName AirMovementModeName = DefaultModeNames::Falling;

	// What movement mode to use when airborne.
	UPROPERTY(Category = "General", EditAnywhere, BlueprintReadWrite)
	FName SwimmingMovementModeName = DefaultModeNames::Swimming;

	/** Walkable slope angle, represented as cosine(max slope angle) for performance reasons. Ex: for max slope angle of 30 degrees, value is cosine(30 deg) = 0.866 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Ground Movement")
	float MaxWalkSlopeCosine = 0.71f;

	/** Max distance to scan for floor surfaces under a Mover actor */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Ground Movement", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm"))
	float FloorSweepDistance = 40.0f;

	/** Performs floor checks as if the object is using a shape with a flat base.
	 *  This avoids the situation where we slowly lower off the side of a ledge when using shapes with rounded bottoms, like a capsule. 
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Ground Movement")
	bool bUseFlatBaseForFloorChecks = true;

	/** Distance from the capsule edge within which floor surface contact is rejected for perching purposes.
	  * A value of 0 means the character can stand right at the capsule edge (no perch restriction). Increasing values make a character fall from a ledge earlier as it is approached.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Ground Movement", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm"))
	float PerchRadiusThreshold = 0.f;

	/** Mover actors will be able to step up onto or over obstacles shorter than this */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Ground Movement", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm"))
	float MaxStepHeight = 40.0f;

	/** Maximum speed in the movement plane */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="General", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s"))
	float MaxSpeed = 800.f;

	/**
	 * Should use acceleration for velocity based movement intent?
	 * If true, acceleration is applied when using velocity input to reach the target velocity.
	 * If false, velocity is set directly, disregarding acceleration.
	 */
	UPROPERTY(Category="General", EditAnywhere, BlueprintReadWrite, AdvancedDisplay)
	bool bUseAccelerationForVelocityMove = true;
	
	/**
	 * Setting that affects movement control. Higher values allow faster changes in direction. This can be used to simulate slippery
	 * surfaces such as ice or oil by lowering the value (possibly based on the material the actor is standing on).
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="General|Friction", meta = (ClampMin = "0", UIMin = "0"))
	float GroundFriction = 8.0f;

	/**
	  * If true, BrakingFriction will be used to slow the character to a stop (when there is no Acceleration).
	  * If false, braking uses the same friction passed to CalcVelocity() (ie GroundFriction when walking), multiplied by BrakingFrictionFactor.
	  * This setting applies to all movement modes; if only desired in certain modes, consider toggling it when movement modes change.
	  * @see BrakingFriction
	  */
	UPROPERTY(Category="General|Friction", EditDefaultsOnly, BlueprintReadWrite)
	uint8 bUseSeparateBrakingFriction:1;

	/**
	 * Friction (drag) coefficient applied when braking (whenever Acceleration = 0, or if character is exceeding max speed); actual value used is this multiplied by BrakingFrictionFactor.
	 * When braking, this property allows you to control how much friction is applied when moving across the ground, applying an opposing force that scales with current velocity.
	 * Braking is composed of friction (velocity-dependent drag) and constant deceleration.
	 * This is the current value, used in all movement modes; if this is not desired, override it or bUseSeparateBrakingFriction when movement mode changes.
	 * @note Only used if bUseSeparateBrakingFriction setting is true, otherwise current friction such as GroundFriction is used.
	 * @see bUseSeparateBrakingFriction, BrakingFrictionFactor, GroundFriction, BrakingDecelerationWalking
	 */
	UPROPERTY(Category="General|Friction", EditAnywhere, BlueprintReadWrite, meta=(ClampMin="0", UIMin="0", EditCondition="bUseSeparateBrakingFriction"))
	float BrakingFriction = 8.0f;

	/**
	 * Factor used to multiply actual value of friction used when braking.
	 * This applies to any friction value that is currently used, which may depend on bUseSeparateBrakingFriction.
	 * @note This is 2 by default for historical reasons, a value of 1 gives the true drag equation.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="General|Friction", meta=(ClampMin="0", UIMin="0"))
	float BrakingFrictionFactor = 2.0f;
	
	/** Default max linear rate of deceleration when there is no controlled input */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="General", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s^2"))
	float Deceleration = 4000.f;

	/** Default max linear rate of acceleration for controlled input. May be scaled based on magnitude of input. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="General", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s^2"))
	float Acceleration = 4000.f;

	/** Maximum rate of turning rotation (degrees per second). Negative numbers indicate instant rotation and should cause rotation to snap instantly to desired direction. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="General", meta = (ClampMin = "-1", UIMin = "0", ForceUnits = "degrees/s"))
	float TurningRate = 500.f;

	/** Speeds velocity direction changes while turning, to reduce sliding */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="General", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "Multiplier"))
	float TurningBoost = 8.f;

 	/** Whether the actor ignores changes in rotation of the base it is standing on when using based movement.
  	 * If true, the actor maintains its current world rotation.
  	 * If false, the actor rotates with the moving base.
  	 */
 	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "General")
	bool bIgnoreBaseRotation = false;

	/** Instantaneous speed induced in an actor upon jumping */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Jumping", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s"))
	float JumpUpwardsSpeed = 500.0f;
	
	/** Depth at which the pawn starts swimming */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Swimming", meta = (Units = "cm"))
	float SwimmingStartImmersionDepth = 64.5f;

	/** Depth at which the pawn will float when in water */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Swimming", meta = (Units = "cm"))
	float SwimmingIdealImmersionDepth = 51.66f;

	/** Depth at which the pawn stops swimming */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Swimming", meta = (Units = "cm"))
	float SwimmingStopImmersionDepth = 39.9f;
	
};
