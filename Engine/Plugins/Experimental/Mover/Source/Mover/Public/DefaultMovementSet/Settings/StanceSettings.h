// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovementMode.h"
#include "StanceSettings.generated.h"

/**
 * StanceSettings: collection of settings that are shared through the mover component and contain stance related settings.
 */ 
UCLASS(MinimalAPI, BlueprintType)
class UStanceSettings : public UObject, public IMovementSettingsInterface
{
	GENERATED_BODY()

	virtual FString GetDisplayName() const override { return GetName(); }

public:
	// New max acceleration while in crouching
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Crouch", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s^2"))
	float CrouchingMaxAcceleration = 2000;

	// New max speed while in crouching
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Crouch", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm/s"))
	float CrouchingMaxSpeed = 200;
	
	// New capsule half height while crouching
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Crouch", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm"))
	float CrouchHalfHeight = 55;

	// New eye height while crouching
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Crouch")
	float CrouchedEyeHeight = 50;
};
