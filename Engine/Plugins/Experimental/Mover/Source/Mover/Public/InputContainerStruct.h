// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoverTypes.h"
#include "InputContainerStruct.generated.h"



#define UE_API MOVER_API

/** 
 * Wrapper class that's used to include input structs in the sync state without them causing reconciliation.
 * This is intended only for internal use.
 */

 USTRUCT()
 struct FMoverInputContainerDataStruct : public FMoverDataStructBase
 {
	GENERATED_BODY()

public:
	// All input data in this struct
	FMoverDataCollection InputCollection;

	// Implementation of FMoverDataStructBase

	// This struct never triggers reconciliation
	virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override { return false; }
	virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float LerpFactor) override;
	virtual FMoverDataStructBase* Clone() const override;
	virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;

	virtual UScriptStruct* GetScriptStruct() const override { return StaticStruct(); }

};


#undef UE_API
