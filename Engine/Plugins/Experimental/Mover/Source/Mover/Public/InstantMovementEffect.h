// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoveLibrary/MovementUtilsTypes.h"

#include "InstantMovementEffect.generated.h"

#define UE_API MOVER_API

class UMoverComponent;
class UMoverSimulation;
struct FMoverTimeStep;
struct FMoverTickStartData;
struct FMoverSyncState;
struct FMoverSimulationEventData;

struct FApplyMovementEffectParams
{
	USceneComponent* UpdatedComponent;

	UPrimitiveComponent* UpdatedPrimitive;

	const UMoverComponent* MoverComp;

	const FMoverTickStartData* StartState;

	const FMoverTimeStep* TimeStep;

	TArray<TSharedPtr<FMoverSimulationEventData>> OutputEvents;
};

/** 
 * Async safe parameters passed to ApplyMovementEffect_Async. 
 * It is almost certainly missing the Physics Object handle and other things, this is just a first pass
 */
struct FApplyMovementEffectParams_Async
{
	UMoverSimulation* Simulation;
	const FMoverTickStartData* StartState;
	const FMoverTimeStep* TimeStep;
};

/**
 * Instant Movement Effects are methods of affecting movement state directly on a Mover-based actor for one tick.
 * Note: This is only applied one tick and then removed
 * Common uses would be for Teleporting, Changing Movement Modes directly, one time force application, etc.
 * Multiple Instant Movement Effects can be active at the time
 */
USTRUCT(BlueprintInternalUseOnly)
struct FInstantMovementEffect
{
	GENERATED_BODY()

	FInstantMovementEffect() { }

	virtual ~FInstantMovementEffect() = default;
	
	// @return newly allocated copy of this FInstantMovementEffect. Must be overridden by child classes
	UE_API virtual FInstantMovementEffect* Clone() const;

	UE_API virtual void NetSerialize(FArchive& Ar);

	UE_API virtual UScriptStruct* GetScriptStruct() const;

	UE_API virtual FString ToSimpleString() const;

	virtual void AddReferencedObjects(class FReferenceCollector& Collector) {}

	// Returns true if this effect is considered nearly equal to Other for comparison purposes.
	// Other is guaranteed to be the same struct type. Override in subclasses to allow
	// tolerance-based equality instead of exact field matching (e.g. for server-side
	// proposal validation in ChaosMover's Proposed auth style).
	// Default: uses UScriptStruct's strict Identical test. If the struct's TStructOpsTypeTraits
	// declares WithIdenticalViaEquality, CompareScriptStruct calls operator== on the struct;
	// otherwise it compares each UPROPERTY field individually via reflection. Check the struct's
	// TStructOpsTypeTraits specialization to know which path applies to a given subtype.
	virtual bool IsNearlyEqualTo(const FInstantMovementEffect& Other) const
	{
		return GetScriptStruct()->CompareScriptStruct(this, &Other, /* PortFlags = */ 0);
	}

	// Return false is the effect didn't apply at all, true if anything was modified, especially the output state's movement mode
	virtual bool ApplyMovementEffect(FApplyMovementEffectParams& ApplyEffectParams, FMoverSyncState& OutputState) { return false; }
	virtual bool ApplyMovementEffect_Async(FApplyMovementEffectParams_Async& ApplyEffectParams, FMoverSyncState& OutputState) { return false; }
};

template<>
struct TStructOpsTypeTraits< FInstantMovementEffect > : public TStructOpsTypeTraitsBase2< FInstantMovementEffect >
{
	enum
	{
		//WithNetSerializer = true,
		WithCopy = true
	};
};

#undef UE_API
