// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoverSimulation.h"
#include "MoverSimulationTypes.h"

#include "KinematicActorSimulation.generated.h"

class UMovementModeStateMachine;
class UMovementMixer;
class UMoverComponent;
class UBaseMovementModeTransition;
class UBaseMovementMode;
struct FMoverInputCmdContext;
struct FMoverSyncState;
struct FMoverAuxStateContext;
struct FMoverTimeStep;
struct FMoverSimContext;
struct FMoverTickStartData;
struct FMoverTickEndData;


#define UE_API MOVER_API


UCLASS(MinimalAPI, BlueprintType)
class UKinematicActorSimulation : public UMoverSimulation
{
	GENERATED_BODY()

public:
	struct FInitParams;

	UE_API UKinematicActorSimulation();
	UE_API virtual void BeginDestroy() override;

	UE_API void Init(const FInitParams& InInitParams);
	UE_API void Uninit();


	// Simulation driver interface: backend will call these on this simulation
	UE_API void InitializeSimulationState(FMoverSyncState& OutSync, FMoverAuxStateContext& OutAux);
	//UE_API virtual void ProduceInput(const int32 DeltaTimeMS, FMoverInputCmdContext* Cmd);
	UE_API void DoSimulationTick(const FMoverTimeStep& InTimeStep, const FMoverTickStartData& StartData, OUT FMoverTickEndData& EndData);
	UE_API void RollbackToPriorState(const FMoverTimeStep& NewBaseTimeStep, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState);
	// End simulation driver interface

	// UMoverSimulation overrides:  threadsafe functions for external systems or internal simulation mechanisms to influence the simulation
	UE_API virtual void QueueNextMode(FModeChangeParams ModeChangeParams) override;

	UE_API virtual FName GetNextModeName() const override;

	UE_API virtual bool HasFeaturesWithTag(FGameplayTag TagToMatch, bool bRequireExactMatch) const override;
	UE_API virtual void CancelFeaturesWithTag(FGameplayTag TagToCancel, bool bRequireExactMatch = false) override;

	UE_API virtual void RegisterMovementMode(FModeRegistrationParams ModeRegisterParams) override;
	UE_API virtual void UnregisterMovementMode(FModeRegistrationParams ModeRegisterParams) override;

	UE_API virtual const FMovementModifierBase* FindQueuedModifier(FMovementModifierHandle ModifierHandle) const override;
	UE_API virtual const FMovementModifierBase* FindQueuedModifierByType(const UScriptStruct* ModifierType) const override;
	UE_API virtual FMovementModifierHandle QueueMovementModifier(TSharedPtr<FMovementModifierBase> Modifier) override;
	UE_API virtual void CancelModifierFromHandle(FMovementModifierHandle ModifierHandle) override;

	// End UMoverSimulation overrides

protected:
	// Setup functions
	UE_API void RegisterMovementModes(const TMap<FName, TWeakObjectPtr<UBaseMovementMode>>& ModeMap, FName StartingModeName);
	UE_API void RegisterGlobalTransitions(const TArray<TWeakObjectPtr<UBaseMovementModeTransition>>& Transitions);

	UE_API virtual void CheckForExternalChanges(const FMoverTickStartData& SimStartingData);


	// JAH NOTE: this is a good candidate for being in the base class
	UE_API virtual void PrepareStateDataForNewFrame(const FMoverTimeStep& TimeStep, const FMoverTickStartData& StartData, FMoverTickEndData& EndData);


	// Customizable events called by this simulation
	//UE_API virtual void OnInit();
	//UE_API virtual void OnDeinit();
	UE_API virtual void OnPreSimulationAdvance(const FMoverTimeStep& TimeStep, const FMoverTickStartData& StartData, FMoverTickEndData& EndData);
	UE_API void AdvanceSimulation(const FMoverTimeStep& TimeStep, const FMoverTickStartData& StartData, FMoverTickEndData& EndData);
	UE_API virtual void OnPostSimulationAdvance(const FMoverTimeStep& TimeStep, const FMoverTickStartData& StartData, FMoverTickEndData& EndData);
	//UE_API virtual void OnSimulationTick(const FMoverSimContext& SimContext, const FMoverTimeStep& TimeStep, const UE::ChaosMover::FSimulationInputData& InputData, UE::ChaosMover::FSimulationOutputData& OutputData);
	//UE_API virtual void OnPostSimulationTick(const FMoverTimeStep& TimeStep, UE::ChaosMover::FSimulationOutputData& OutputData);

	// JAH TODO: Make a version of this that provides the PREVIOUS state as well
	//UE_API virtual void OnSimulationRolledBack(const FMoverTimeStep& NewBaseTimeStep, const FMoverSyncState& PrevSyncState);
	UE_API virtual void OnSimulationRolledBack(const FMoverTimeStep& NewBaseTimeStep, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState);



public:
	struct FInitParams
	{
		TMap<FName, TWeakObjectPtr<UBaseMovementMode>> ModesToRegister;
		TArray<TWeakObjectPtr<UBaseMovementModeTransition>> TransitionsToRegister;
		FName StartingMovementMode = NAME_None;
		//TWeakObjectPtr<UNullMovementMode> NullMovementMode = nullptr;
		//TWeakObjectPtr<UImmediateMovementModeTransition> ImmediateModeTransition = nullptr;
		TWeakObjectPtr<UMovementMixer> MovementMixer = nullptr;
		//FTransform TransformOnInit = FTransform::Identity;
		UWorld* World = nullptr;
		TWeakObjectPtr<UMoverComponent> MoverComp = nullptr;
		TWeakObjectPtr<USceneComponent> UpdatedComp = nullptr;
		//bool bIsUsingAsyncPhysics = true;
	};

protected:

	FInitParams InitParams;

	UPROPERTY()
	TObjectPtr<UMovementModeStateMachine> ModeFSM;

	UPROPERTY()
	TObjectPtr<UMoverComponent> MoverComp;


	FMoverSyncState LastSyncState;
	FMoverAuxStateContext LastAuxState;

	FMoverTimeStep LastSimTickTimeStep;		// Saved timestep info from our last simulation tick, used during rollback handling. This will rewind during corrections.
	FMoverTimeStep NewestSimTickTimeStep;	// Saved timestep info from the newest (farthest-advanced) simulation tick. This will not rewind during corrections.
	bool bJustStartedResim = false;

private: 
	FString GetContextNameSafe() const;	// internally used for logging actor owner name
};

#undef UE_API
