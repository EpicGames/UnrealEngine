// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayTagContainer.h"
#include "MoveLibrary/MovementUtilsTypes.h"
#include "StructUtils/InstancedStruct.h"
#include "LayeredMove.generated.h"

#define UE_API MOVER_API

class UMovementMixer;
struct FMoverTickStartData;
struct FMoverTimeStep;
struct FMoverTime;
class UMoverBlackboard;
class UMoverComponent;


UENUM(BlueprintType)
enum class ELayeredMoveFinishVelocityMode : uint8
{
	// Maintain the last velocity root motion gave to the character
	MaintainLastRootMotionVelocity = 0,
	// Set Velocity to the specified value (for example, 0,0,0 to stop the character)
	SetVelocity,
	// Clamp velocity magnitude to the specified value. Note that it will not clamp Z if negative (falling). it will clamp Z positive though. 
	ClampVelocity,
};

/** 
 * Struct for LayeredMove Finish Velocity options.
 */
USTRUCT(BlueprintType)
struct FLayeredMoveFinishVelocitySettings
{
	GENERATED_BODY()

	FLayeredMoveFinishVelocitySettings()
		: SetVelocity(FVector::ZeroVector)
		, ClampVelocity(0.f)
		, FinishVelocityMode(ELayeredMoveFinishVelocityMode::MaintainLastRootMotionVelocity)
	{}

	void NetSerialize(FArchive& Ar);
	
	// Velocity that the actor will use if Mode == SetVelocity
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FVector SetVelocity;

	// Actor's Velocity will be clamped to this value if Mode == ClampVelocity
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	float ClampVelocity;
	
	// What mode we want to happen when a Layered Move ends, see @ELayeredMoveFinishVelocityMode
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	ELayeredMoveFinishVelocityMode FinishVelocityMode;
};

/** 
* Layered Moves are methods of affecting motion on a Mover-based actor, typically for a limited time. 
* Common uses would be for jumping, dashing, blast forces, etc.
* They are ticked as part of the Mover simulation, and produce a proposed move. These proposed moves 
* are aggregated and applied to the overall attempted move.
* Multiple layered moves can be active at any time, and may produce additive motion or motion that overrides
* what the current Movement Mode may intend.
* Layered moves can also set a preferred movement mode that only changes the movement mode at the start of
* the move. Any movement mode changes that need to happen as part of the layered move after the start of the move
* need to be queued through an Instant Effect or the QueueNextMode function
*/

// Base class for all layered moves
USTRUCT(BlueprintInternalUseOnly)
struct FLayeredMoveBase
{
	GENERATED_BODY()

	UE_API FLayeredMoveBase();
	virtual ~FLayeredMoveBase() = default;

	// Determines how this object's movement contribution should be mixed with others
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	EMoveMixMode MixMode;

	// Determines if this layered move should take priority over other layered moves when different moves have conflicting overrides - higher numbers taking precedent.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	uint8 Priority;
	
	/**
	 * This move will expire after a set amount of time if > 0. If 0, it will be ticked only once, regardless of time step. It will need to be manually ended if < 0.
	 * Note: If changed after starting to a value beneath the current lifetime of the move, it will immediately finish (so if your move finishes early, setting this to 0 is equivalent to returning true from IsFinished())
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	float DurationMs;

	// The simulation time this move first ticked (< 0 means it hasn't started yet)
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Transient, Category = Mover)
	double StartSimTimeMs;

	// Settings related to velocity applied to the actor after a layered move has finished
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FLayeredMoveFinishVelocitySettings FinishVelocitySettings;

	/**
	 * Check Layered Move for a gameplay tag.
	 *
	 * @param TagToFind			Tag to check on the Mover systems
	 * @param bExactMatch		If true, the tag has to be exactly present, if false then TagToFind will include its parent tags while matching
	 * 
	 * @return True if the TagToFind was found
	 */
	virtual bool HasGameplayTag(FGameplayTag TagToFind, bool bExactMatch) const
	{
		return false;
	}

	/** Adds all gameplay tags currently active on this layered move to InOutTags. */
	virtual void GetGameplayTags(FGameplayTagContainer& InOutTags) const {}

	// Kicks off this move, allowing any initialization to occur.
	UE_API void StartMove(const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, double CurrentSimTimeMs);
	// Async version of the above
	UE_API void StartMove_Async(UMoverBlackboard* SimBlackboard, const FMoverTime& SimTime);

	// Called when this layered move starts.
	virtual void OnStart(const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard) {}
	// Async version of the above
	virtual void OnStart_Async(UMoverBlackboard* SimBlackboard, const FMoverTime& SimTime) {}

	// TODO: consider whether MoverComp should just be part of FMoverTickStartData
	// Generate a movement that will be combined with other sources
	virtual bool GenerateMove(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove) { return false; }
	// Async version of the above
	UE_API virtual bool GenerateMove_Async(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove);

	// Returns true if this move supports async (worker thread) execution via GenerateMove_Async.
	// Moves that return false will only have GenerateMove called on them, never GenerateMove_Async.
	virtual bool SupportsAsync() const { return false; }

	// Runtime query whether this move is finished and can be destroyed. The default implementation is based on DurationMs.
	UE_API virtual bool IsFinished(double CurrentSimTimeMs) const;

	// Ends this move, allowing any cleanup to occur.
	UE_API void EndMove(const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, double CurrentSimTimeMs);
	// Async version of the above
	UE_API void EndMove_Async(UMoverBlackboard* SimBlackboard, double CurrentSimTimeMs);
	
	// Called when this layered move ends.
	virtual void OnEnd(const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, double CurrentSimTimeMs) {}
	// Async version of the above
	virtual void OnEnd_Async(UMoverBlackboard* SimBlackboard, double CurrentSimTimeMs) {}

	// @return newly allocated copy of this FLayeredMoveBase. Must be overridden by child classes
	UE_API virtual FLayeredMoveBase* Clone() const;

	UE_API virtual void NetSerialize(FArchive& Ar);

	UE_API virtual UScriptStruct* GetScriptStruct() const;

	UE_API virtual FString ToSimpleString() const;

	virtual void AddReferencedObjects(class FReferenceCollector& Collector) {}

	// Returns true if this move is considered nearly equal to Other for comparison purposes.
	// Other is guaranteed to be the same struct type. Override in subclasses to allow
	// tolerance-based equality instead of exact field matching (e.g. for server-side
	// proposal validation in ChaosMover's Proposed auth style).
	// Default: uses UScriptStruct's strict Identical test. If the struct's TStructOpsTypeTraits
	// declares WithIdenticalViaEquality, CompareScriptStruct calls operator== on the struct;
	// otherwise it compares each UPROPERTY field individually via reflection. Check the struct's
	// TStructOpsTypeTraits specialization to know which path applies to a given subtype.
	virtual bool IsNearlyEqualTo(const FLayeredMoveBase& Other) const
	{
		return GetScriptStruct()->CompareScriptStruct(this, &Other, /* PortFlags = */ 0);
	}
};

template<>
struct TStructOpsTypeTraits< FLayeredMoveBase > : public TStructOpsTypeTraitsBase2< FLayeredMoveBase >
{
	enum
	{
		//WithNetSerializer = true,
		WithCopy = true
	};
};


// A collection of layered moves affecting a movable actor
USTRUCT(BlueprintType)
struct FLayeredMoveGroup
{
	GENERATED_BODY()

	UE_API FLayeredMoveGroup();
	
	/**
	 * If bApplyResidualVelocity is true this actors velocity will be set to this.
	 * Note: This is set automatically when a layered move has ended based off of end velocity settings - it is not meant to be set by the user see @FLayeredMoveFinishVelocitySettings
     */
	UPROPERTY(NotReplicated, VisibleAnywhere, BlueprintReadOnly, Category = Mover)
	FVector ResidualVelocity;
	
	UE_API void QueueLayeredMove(TSharedPtr<FLayeredMoveBase> Move);

	/** Schedule matching layered moves to be cancelled ASAP */
	void CancelMovesByTag(FGameplayTag Tag, bool bRequireExactMatch=false);

	bool HasAnyMoves() const { return (!ActiveLayeredMoves.IsEmpty() || !QueuedLayeredMoves.IsEmpty()); }

	/** @return True if there is at least one layered move of MoveType that's either active or queued */
	template <typename MoveType UE_REQUIRES(std::is_base_of_v<FLayeredMoveBase, MoveType>)>
	bool HasMove() const
	{
		return FindActiveMove<MoveType>() || FindQueuedMove<MoveType>();
	}

	// Generates active layered move list (by calling FlushMoveArrays) and returns the an array of all currently active layered moves
	UE_API TArray<TSharedPtr<FLayeredMoveBase>> GenerateActiveMoves(const FMoverTimeStep& TimeStep, const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard);
	UE_API TArray<TSharedPtr<FLayeredMoveBase>> GenerateActiveMoves_Async(const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard);

	/** Serialize all moves and their states for this group */
	UE_API void NetSerialize(FArchive& Ar, uint8 MaxNumMovesToSerialize = MAX_uint8);

	/** Copy operator - deep copy so it can be used for archiving/saving off moves */
	UE_API FLayeredMoveGroup& operator=(const FLayeredMoveGroup& Other);

	/** Comparison operator - needs matching LayeredMoves along with identical states in those structs */
	UE_API bool operator==(const FLayeredMoveGroup& Other) const;

	/** Comparison operator */
	UE_API bool operator!=(const FLayeredMoveGroup& Other) const;

	/** Checks only whether there are matching LayeredMoves, but NOT necessarily identical states of each move */
	UE_API bool HasSameContents(const FLayeredMoveGroup& Other) const;

	/** Exposes references to GC system */
	UE_API void AddStructReferencedObjects(FReferenceCollector& Collector) const;

	/** Get a simplified string representation of this group. Typically for debugging. */
	UE_API FString ToSimpleString() const;

#if WITH_CHAOS_VISUAL_DEBUGGER
	// Populate Debug_ActiveMoves from ActiveLayeredMoves so CVD can display them via reflection.
	// Call just before tracing.
	UE_API void PrepareCVDMirror() const;
#endif

	const TArray<TSharedPtr<FLayeredMoveBase>>& GetActiveMoves() const { return ActiveLayeredMoves; }
	const TArray<TSharedPtr<FLayeredMoveBase>>& GetQueuedMoves() const { return QueuedLayeredMoves; }

	/** Returns the first active layered move of the specified type, if one exists */
	template <typename MoveType UE_REQUIRES(std::is_base_of_v<FLayeredMoveBase, MoveType>)>
	const MoveType* FindActiveMove() const
	{
		return static_cast<const MoveType*>(FindActiveMove(MoveType::StaticStruct()));
	}
	UE_API const FLayeredMoveBase* FindActiveMove(const UScriptStruct* LayeredMoveStructType) const;

	/** Calls Func for every active move whose struct type is or derives from MoveType. */
	template <typename MoveType UE_REQUIRES(std::is_base_of_v<FLayeredMoveBase, MoveType>)>
	void ForEachActiveMoveOfType(TFunctionRef<void(MoveType&)> Func)
	{
		for (const TSharedPtr<FLayeredMoveBase>& Move : ActiveLayeredMoves)
		{
			if (Move.IsValid() && Move->GetScriptStruct()->IsChildOf(MoveType::StaticStruct()))
			{
				Func(*static_cast<MoveType*>(Move.Get()));
			}
		}
	}

	/** Const overload: calls Func for every active move whose struct type is or derives from MoveType. */
	template <typename MoveType UE_REQUIRES(std::is_base_of_v<FLayeredMoveBase, MoveType>)>
	void ForEachActiveMoveOfType(TFunctionRef<void(const MoveType&)> Func) const
	{
		for (const TSharedPtr<FLayeredMoveBase>& Move : ActiveLayeredMoves)
		{
			if (Move.IsValid() && Move->GetScriptStruct()->IsChildOf(MoveType::StaticStruct()))
			{
				Func(*static_cast<const MoveType*>(Move.Get()));
			}
		}
	}

	/** Returns the first queued layered move of the specified type, if one exists */
	template <typename MoveType UE_REQUIRES(std::is_base_of_v<FLayeredMoveBase, MoveType>)>
	const MoveType* FindQueuedMove() const
	{
		return static_cast<const MoveType*>(FindQueuedMove(MoveType::StaticStruct()));
	}
	UE_API const FLayeredMoveBase* FindQueuedMove(const UScriptStruct* LayeredMoveStructType) const;
	
	// Resets residual velocity related settings
	UE_API void ResetResidualVelocity();

	// Resets residual velocity and clears active and queued moves
	UE_API void Reset();

protected:
	// Clears out any finished or invalid active moves and adds any queued moves to the active moves
	UE_API void FlushMoveArrays(const UMoverComponent* MoverComp, UMoverBlackboard* SimBlackboard, const FMoverTime& SimTime, bool bIsAsync);

	// Helper function for gathering any residual velocity settings from layered moves that just ended
	UE_API void GatherResidualVelocitySettings(const TSharedPtr<FLayeredMoveBase>& Move, bool& bResidualVelocityOverriden, bool& bClampVelocityOverriden);
	
	/** Helper function for serializing array of root motion sources */
	static UE_API void NetSerializeLayeredMovesArray(FArchive& Ar, TArray< TSharedPtr<FLayeredMoveBase> >& LayeredMovesArray, uint8 MaxNumLayeredMovesToSerialize = MAX_uint8);

	/** Layered moves currently active in this group */
	TArray< TSharedPtr<FLayeredMoveBase> > ActiveLayeredMoves;

	/** Moves that are queued to become active next sim frame */
	TArray< TSharedPtr<FLayeredMoveBase> > QueuedLayeredMoves;

	// Reflected snapshot of ActiveLayeredMoves, populated just before CVD tracing in
	// UChaosMoverSimulation::TraceMoverData. TSharedPtr arrays cannot be UPROPERTY so
	// this mirror is the only way for CVD's reflection-based display to see active moves.
	// mutable so it can be filled from a const reference during tracing.
	UPROPERTY(VisibleAnywhere, Transient, Category = Mover, DisplayName = "ActiveMoves")
	mutable TArray<FInstancedStruct> Debug_ActiveMoves;

	/** Used during simulation to cancel any moves that match a tag */
	TArray<TPair<FGameplayTag, bool>> TagCancellationRequests;
public:

	/**
	 * Clamps an actors velocity to this value when a layered move ends. This expects Value >= 0.
	 * Note: This is set automatically when a layered move has ended based off of end velocity settings - it is not meant to be set by the user see @FLayeredMoveFinishVelocitySettings
	 */
	UPROPERTY(NotReplicated, VisibleAnywhere, BlueprintReadOnly, Category = Mover)
	float ResidualClamping;

	/**
	 * If true ResidualVelocity will be the next velocity used for this actor
	 * Note: This is set automatically when a layered move has ended based off of end velocity settings - it is not meant to be set by the user see @FLayeredMoveFinishVelocitySettings
	 */
	UPROPERTY(NotReplicated, VisibleAnywhere, BlueprintReadOnly, Category = Mover)
	bool bApplyResidualVelocity;
};

template<>
struct TStructOpsTypeTraits<FLayeredMoveGroup> : public TStructOpsTypeTraitsBase2<FLayeredMoveGroup>
{
	enum
	{
		WithCopy = true,		// Necessary so that TSharedPtr<FLayeredMoveBase> Data is copied around
		//WithNetSerializer = true,
		WithIdenticalViaEquality = true,
		WithAddStructReferencedObjects = true,
	};
};

#undef UE_API
