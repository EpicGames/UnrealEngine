// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayTagContainer.h"
#include "MoveLibrary/MovementUtilsTypes.h"
#include "Templates/SubclassOf.h"
#include "LayeredMove.h"
#include "MoverLog.h"

#include "LayeredMoveBase.generated.h"

//////////////////////////////////////////////////////////////////////////
// TODO: Consider relocating this class and functionality, a lot of what's here belongs in the original LayeredMove.h/.cpp as replacements for FLayeredMoveBase and such
//		 but we wanted to explore this implementation of Blueprintable layered moves without disrupting the old system so it lives here for now.
//////////////////////////////////////////////////////////////////////////

#define UE_API MOVER_API
#define USING_PARAMS(ActivationParamsClass) using ActivationParamsType = ActivationParamsClass;
#define USING_MOVE_DATA(MoveDataClass) using MoveDataType = MoveDataClass;

class UMovementMixer;
struct FMoverTickStartData;
struct FMoverTimeStep;
class UMoverBlackboard;
class UMoverComponent;
class ULayeredMoveLogic;

/**
 * Packaged params struct for initializing a corresponding FLayeredMoveInstancedData
 * Allows BP to do "templated" move data creation. Optional in C++, where params can be forwarded to the FLayeredMoveInstancedData ctor directly
 * The base class can also be used on any activation to just use default values
 */
USTRUCT(Blueprintable)
struct FLayeredMoveActivationParams
{
	GENERATED_BODY()
	
	/**
 	 * This move will expire after a set amount of time if > 0. If 0, it will be ticked only once, regardless of time step. It will need to be manually ended if < 0.
 	 * Note: If changed after starting to a value beneath the current lifetime of the move, it will immediately finish (so if your move finishes early, setting this to 0 is equivalent to returning true from IsFinished())
 	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	double DurationMs = 0.f;
};

/** Instanced data created and replicated for each activation of a layered move */
USTRUCT(Blueprintable)
struct FLayeredMoveInstancedData
{
	GENERATED_BODY()

	//@todo DanH: If we're already using a macro, might as well put the boilerplate virtuals in here
	USING_PARAMS(FLayeredMoveActivationParams);
	
	FLayeredMoveInstancedData() {}
	virtual ~FLayeredMoveInstancedData() = default;
	// Checks if 
	UE_API bool operator==(const FLayeredMoveInstancedData& Other) const;
	bool operator!=(const FLayeredMoveInstancedData& Other) const { return !operator==(Other); }

	/** @return A newly allocated copy of this FLayeredMoveInstancedData. Must be overridden by child classes */
	virtual FLayeredMoveInstancedData* Clone() const { return new FLayeredMoveInstancedData(*this); }

	/** @return The UScriptStruct describing this struct. Must be overridden by child classes */
	virtual UScriptStruct* GetScriptStruct() const { return StaticStruct(); }
	
	/** @return True if this move data is identical to OtherData. OtherData is guaranteed to be safe to cast to the implementing type. */
	UE_API virtual bool Equals(const FLayeredMoveInstancedData& OtherData) const;

	// Returns true if this data is considered nearly equal to Other for comparison purposes.
	// Other is guaranteed to be the same struct type. Override in subclasses to allow
	// tolerance-based equality instead of exact field matching (e.g. for server-side
	// proposal validation in ChaosMover's Proposed auth style).
	// Default: uses UScriptStruct's strict Identical test. If the struct's TStructOpsTypeTraits
	// declares WithIdenticalViaEquality, CompareScriptStruct calls operator== on the struct;
	// otherwise it compares each UPROPERTY field individually via reflection. Check the struct's
	// TStructOpsTypeTraits specialization to know which path applies to a given subtype.
	virtual bool IsNearlyEqualTo(const FLayeredMoveInstancedData& Other) const
	{
		return GetScriptStruct()->CompareScriptStruct(this, &Other, /* PortFlags = */ 0);
	}

	/** Called when a queued layered move is activated. Provides an opportunity to initialize Layered MoveData */
	UE_API virtual void ActivateFromContext(const FLayeredMoveActivationParams* ActivationParams);
	
	UE_API virtual void NetSerialize(FArchive& Ar);

	virtual void AddReferencedObjects(FReferenceCollector& Collector) {}

	/** Is this move considered to "have" a given gameplay tag? */
	virtual bool HasGameplayTag(FGameplayTag TagToFind, bool bExactMatch) const { return false; }
	
	double GetStartSimTimeMs() const { return StartSimTimeMs; }

	/**
	 * This move will expire after a set amount of time if > 0. If 0, it will be ticked only once, regardless of time step. It will need to be manually ended if < 0.
	 * Note: If changed after starting to a value beneath the current lifetime of the move, it will immediately finish (so if your move finishes early, setting this to 0 is equivalent to returning true from IsFinished())
	 */
	UPROPERTY(BlueprintReadWrite, Category = Mover)
	double DurationMs = -1.f;
	
	/** The simulation time this move first ticked (< 0 means it hasn't started yet) */
    UPROPERTY(BlueprintReadOnly, Category = Mover)
    double StartSimTimeMs = -UE_BIG_NUMBER;

	friend class ULayeredMoveLogic;
};

template<>
struct TStructOpsTypeTraits<FLayeredMoveInstancedData> : public TStructOpsTypeTraitsBase2<FLayeredMoveInstancedData>
{
	enum
	{
		//WithNetSerializer = true,
		WithCopy = true
	};
};

/**
 * Base class for all layered move logic that operates in tandem with instanced FLayeredMoveInstancedData.
 * The logic object itself is not meant to ever be replicated, and a maximum of one instance of each logic class
 * need ever exist on a given MoverComponent. Repeated and/or simultaneous activations of the same move on a component
 * are represented, tracked, and replicated through instances of the FLayeredMoveInstancedData struct type
 * that the logic class indicates in the ActiveMoveDataStructType property.
 *
 * The virtual methods on this class are invoked in a special and strict pattern that guarantees AccessExecutionMoveData
 * will return the valid data instance relevant to that function execution.
 *
 * Refer to [Examples when they exist] for implementation examples
 */
UCLASS(Abstract, BlueprintType, Blueprintable)
class ULayeredMoveLogic : public UObject
{
	GENERATED_BODY()

public:
	USING_MOVE_DATA(FLayeredMoveInstancedData)
	
	UE_API ULayeredMoveLogic();

	UScriptStruct* GetInstancedDataType() const { return InstancedDataStructType; }
	
	const FLayeredMoveFinishVelocitySettings& GetFinishVelocitySettings() const { return FinishVelocitySettings; }
	EMoveMixMode GetMixMode() const { return MixMode; }
	uint8 GetPriority() const { return Priority; }

	// Helper function for validating move data when passing data and logic to/from BP
	static bool ValidateMoveDataGetSet(const UObject* ObjectValidatingData, const ULayeredMoveLogic* MoveLogic, const FStructProperty* MoveDataProperty, const uint8* MoveDataPtr, FFrame& StackFrame);

	// Returns true if this logic class supports async (worker-thread) execution via GenerateMove_Async.
	// Subclasses that override GenerateMove_Async must also return true here.
	virtual bool SupportsAsync() const { return false; }

private:
	// These functions all assume validity of CurrentActiveMoveData to perform logic from the perspective of that chunk of properties
	// To that end, the only external entity capable of invoking them is the FScopedMoveLogicExecContext, which guarantees the active move data has been set
	friend class FScopedMoveLogicExecContext;
	
protected:
	/** Called when this move is initially activated  */
	UFUNCTION(BlueprintNativeEvent)
	UE_API void OnStart(const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard);

	/** Called when this move has ended  */
	UFUNCTION(BlueprintNativeEvent)
	UE_API void OnEnd(const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard);

	/** Generate a movement that will be combined with other sources */
	UFUNCTION(BlueprintNativeEvent)
	UE_API bool GenerateMove(const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, UPARAM(ref) const FMoverTickStartData& StartState, UPARAM(ref) FProposedMove& OutProposedMove);
	
	//@todo: Will need to cache whether to treat the instance data as const (i.e. whether to disregard & complain if someone tries to set the data in BP during that window) 
	UFUNCTION(BlueprintNativeEvent)
	UE_API bool IsFinished(const FMoverTimeStep& TimeStep, const UMoverBlackboard* SimBlackboard);
	
	virtual void OnStart_Implementation(const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard) {}
	virtual void OnEnd_Implementation(const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard) {}
	virtual bool GenerateMove_Implementation(const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, const FMoverTickStartData& StartState, FProposedMove& OutProposedMove) { return false; }
	UE_API virtual bool IsFinished_Implementation(const FMoverTimeStep& TimeStep, const UMoverBlackboard* SimBlackboard);

	/** Async version of OnStart, called on the simulation worker thread when SupportsAsync() returns true */
	UFUNCTION(BlueprintNativeEvent)
	UE_API void OnStart_Async(UMoverBlackboard* SimBlackboard, const FMoverTime& SimTime);

	/** Async version of GenerateMove, called on the simulation worker thread when SupportsAsync() returns true */
	UFUNCTION(BlueprintNativeEvent)
	UE_API bool GenerateMove_Async(const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, UPARAM(ref) const FMoverTickStartData& StartState, UPARAM(ref) FProposedMove& OutProposedMove);

	/** Async version of OnEnd, called on the simulation worker thread when SupportsAsync() returns true */
	UFUNCTION(BlueprintNativeEvent)
	UE_API void OnEnd_Async(UMoverBlackboard* SimBlackboard, const FMoverTime& SimTime);

	virtual void OnStart_Async_Implementation(UMoverBlackboard*, const FMoverTime&) {}
	virtual bool GenerateMove_Async_Implementation(const FMoverTimeStep&, UMoverBlackboard*, const FMoverTickStartData&, FProposedMove&) { return false; }
	virtual void OnEnd_Async_Implementation(UMoverBlackboard*, const FMoverTime&) {}

	/** Accessor to the FLayeredMoveInstancedData established for the execution of a virtual move logic function */
	template <typename MoveDataT = FLayeredMoveInstancedData UE_REQUIRES(std::is_base_of_v<FLayeredMoveInstancedData, MoveDataT>)>
	MoveDataT& AccessExecutionMoveData() const
	{
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
		if (!ensure(InstancedDataStructType))
        {
			UE_LOGF(LogMover, Error, "Move Logic needs an active data struct type. If no data is needed consider using the default move data type.")
        	static MoveDataT GarbageData;
        	return GarbageData;
        }
#endif
		
		check(InstancedDataStructType);
		return static_cast<MoveDataT&>(*CurrentInstancedData);
	}
	
	/**
	 * Gets active move data that is tied to this active move. Use SetActiveMoveData to write active move data.
	 * Note: Make sure break node is the correct data type specified by the layered move logic.
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = "Mover | Layered Move", DisplayName = "Get Active Move Data", meta = (BlueprintProtected, BlueprintInternalUseOnly, CustomStructureParam = "OutMoveData", DefaultToSelf = "MoveLogic"))
	static bool K2_GetActiveMoveData(ULayeredMoveLogic* MoveLogic, FLayeredMoveInstancedData& OutMoveData);
	DECLARE_FUNCTION(execK2_GetActiveMoveData);
	
	/**
	 * Sets active move data that is tied to this active move. Use GetActiveMoveData to read current active move data.
	 * Note: Make sure make node is the correct data type specified by the layered move logic.
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = "Mover | Layered Move", DisplayName = "Set Active Move Data", meta = (BlueprintProtected, BlueprintInternalUseOnly, CustomStructureParam = "OutMoveData", DefaultToSelf = "MoveLogic"))
	static void K2_SetActiveMoveData(ULayeredMoveLogic* MoveLogic, const FLayeredMoveInstancedData& OutMoveData);
	DECLARE_FUNCTION(execK2_SetActiveMoveData);
	
	/**
	 * This move will expire after a set amount of time if > 0. If 0, it will be ticked only once, regardless of time step. It will need to be manually ended if < 0.
	 * Note: If changed after starting to a value beneath the current lifetime of the move, it will immediately finish (so if your move finishes early, setting this to 0 is equivalent to returning true from IsFinished())
	 */
	UPROPERTY(BlueprintReadWrite, Category = Mover)
	double DefaultDurationMs = -1.f;
	
	/** Determines how this object's movement contribution should be mixed with others */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Mover)
	EMoveMixMode MixMode = EMoveMixMode::AdditiveVelocity;

	/** Determines if this layered move should take priority over other layered moves when different moves have conflicting overrides - higher numbers taking precedent. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Mover)
	uint8 Priority = 0;

	//@todo: Does this need to be a separate struct?
	/** Settings related to velocity applied to the actor after the move has finished */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Mover)
	FLayeredMoveFinishVelocitySettings FinishVelocitySettings;

	//@todo DanH: Fail validation if this isn't set or isn't an FLayeredMoveInstancedData
	UPROPERTY(EditAnywhere, NoClear, BlueprintReadOnly, Category = Mover, meta=(MetaStruct="/Script/Mover.LayeredMoveInstancedData"))
	TObjectPtr<UScriptStruct> InstancedDataStructType;
	
private:
	/**
	 * The FLayeredMoveInstancedData provided to each of the base virtual move functions, valid only for the duration a single virtual function execution.
	 * Direct access is only for internal plumbing - use AccessExecutionMoveData() in virtuals to obtain a typed reference to this.
	 */
	TSharedPtr<FLayeredMoveInstancedData> CurrentInstancedData;
};

/**
 * Wrapper to encapsulate the split implementation of a move between a stateless ULayeredMoveLogic* object and an instance of FLayeredMoveInstancedData
 * Those two pieces, in tandem, represent a "whole" functional Layered Move.
 */
USTRUCT()
struct FLayeredMoveInstance
{
	GENERATED_BODY()
	
	UE_API FLayeredMoveInstance();
	UE_API FLayeredMoveInstance(const FLayeredMoveInstance& OtherLayeredMoveInstance);
	UE_API FLayeredMoveInstance(const TSharedRef<FLayeredMoveInstancedData>& InMoveData, ULayeredMoveLogic* InMoveLogic = nullptr);
	
	/** TArray::FindByKey() enablers */
	bool operator==(const TSubclassOf<ULayeredMoveLogic>& LogicClass) const { return MoveLogic && MoveLogic->IsA(LogicClass); }
	bool operator==(const UScriptStruct* MoveDataType) const { return InstanceMoveData->GetScriptStruct()->IsChildOf(MoveDataType); }

	bool HasLogic() const { return MoveLogic != nullptr; }
	const UClass* GetLogicClass() const { return MoveLogic->GetClass(); }
	
	UE_API void StartMove(const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard) const;
	UE_API bool GenerateMove(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove) const;
	UE_API void EndMove(const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard) const;
	UE_API bool IsFinished(const FMoverTimeStep& TimeStep, const UMoverBlackboard* SimBlackboard) const;

	/** Async equivalents, safe to call from the simulation worker thread when the logic SupportsAsync() */
	UE_API void StartMove_Async(UMoverBlackboard* SimBlackboard, const FMoverTime& SimTime) const;
	UE_API bool GenerateMove_Async(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, FProposedMove& OutProposedMove) const;
	UE_API void EndMove_Async(UMoverBlackboard* SimBlackboard, const FMoverTime& SimTime) const;

	/** Returns the logic object; used by GenerateActiveMoves_Async to filter by SupportsAsync() */
	ULayeredMoveLogic* GetLogic() const { return MoveLogic.Get(); }

	UE_API const FLayeredMoveFinishVelocitySettings& GetFinishVelocitySettings() const;
	UE_API EMoveMixMode GetMixMode() const;
	uint8 GetPriority() const { return MoveLogic->GetPriority(); }
	double GetStartingTimeMs() const { return InstanceMoveData->GetStartSimTimeMs(); }
	bool HasGameplayTag(FGameplayTag TagToFind, bool bExactMatch) const { return InstanceMoveData->HasGameplayTag(TagToFind, bExactMatch); }

	UE_API FLayeredMoveInstance Clone() const;
	UScriptStruct* GetDataStructType() const { return InstanceMoveData->GetScriptStruct(); }
	const FLayeredMoveInstancedData* GetMoveData() const { return InstanceMoveData.Get(); }
	void AddReferencedObjects(FReferenceCollector& Collector) const { InstanceMoveData->AddReferencedObjects(Collector); } 

	UE_API void NetSerialize(FArchive& Ar);

	UE_API const UClass* GetSerializedMoveLogicClass() const;

	/**
	 * This function populates the MoveLogic reference of ActiveMoves that don't have any logic classes. This is necessary as
	 * active move data received from NetSerialize doesn't necessarily have the Logic class it was activated with.
	 * Note: Currently we are NetSerializing the MoveLogic class so that we can then populate Logic from using the registered move array
	 * on the MoverComponent but eventually we should use a ID mapping or something better to avoid serializing the MoveLogic class.
	 */
	UE_API bool PopulateMissingActiveMoveLogic(const TArray<TObjectPtr<ULayeredMoveLogic>>& RegisteredMoves);
	
private:
	TSharedPtr<FLayeredMoveInstancedData> InstanceMoveData;
	
	/**
	 * Is used in PopulateMissingActiveMoveLogic to help populate logic classes in active moves that were NetSerialized since MoveLogic isn't NetSerialized
	 * TODO: Eventually replace this with some ID/Mapping system so we can NetSerialize that instead of a full UClass
	 */
	UPROPERTY(Transient)
	TSubclassOf<ULayeredMoveLogic> MoveLogicClassType;
	
	//@todo DanH: This exists on the PT for the chaos mover - can/should this still be a UProperty TObjectPtr?
	UPROPERTY(Transient)
	TObjectPtr<ULayeredMoveLogic> MoveLogic;
};

template<>
struct TStructOpsTypeTraits<FLayeredMoveInstance> : public TStructOpsTypeTraitsBase2<FLayeredMoveInstance>
{
	enum
	{
		//WithNetSerializer = true,
		WithCopy = true
	};
};

#undef UE_API
