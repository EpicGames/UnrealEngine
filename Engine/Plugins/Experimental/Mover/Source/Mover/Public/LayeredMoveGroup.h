// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayTagContainer.h"
#include "MoveLibrary/MovementUtilsTypes.h"
#include "LayeredMoveGroup.generated.h"

#define UE_API MOVER_API

struct FLayeredMoveInstance;
struct FLayeredMoveInstancedData;
struct FProposedMove;
class UMovementMixer;
struct FMoverTickStartData;
struct FMoverTimeStep;
class UMoverBlackboard;
class ULayeredMoveLogic;

/**
 * The group of information about currently active and queued moves.
 * This replicates info for FLayeredMoveInstancedData only - it is expected that the corresponding ULayeredMoveLogic is 
 * already registered with the mover component.
 */
USTRUCT(BlueprintType)
struct FLayeredMoveInstanceGroup
{
	GENERATED_BODY()

	UE_API FLayeredMoveInstanceGroup();
	UE_API FLayeredMoveInstanceGroup& operator=(const FLayeredMoveInstanceGroup& Other);
	UE_API bool operator==(const FLayeredMoveInstanceGroup& Other) const;
	bool operator!=(const FLayeredMoveInstanceGroup& Other) const { return !operator==(Other); }

	/** Checks only whether there are matching LayeredMoves, but NOT necessarily identical states of each move */
	UE_API bool HasSameContents(const FLayeredMoveInstanceGroup& Other) const;
	
	UE_API bool GenerateMixedMove(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, UMovementMixer& MovementMixer, UMoverBlackboard* SimBlackboard, FProposedMove& OutMixedMove);
	UE_API void ApplyResidualVelocity(FProposedMove& InOutProposedMove);
	UE_API void NetSerialize(FArchive& Ar, uint8 MaxNumMovesToSerialize = MAX_uint8);
	UE_API void AddStructReferencedObjects(FReferenceCollector& Collector) const;
	UE_API void ResetResidualVelocity();
	UE_API void Reset();

	/**
	 * Loops through all Queued and Active moves and populates any missing MoveLogic using FLayeredMoveInstance::PopulateMissingActiveMoveLogic.
	 * See FLayeredMoveInstance::PopulateMissingActiveMoveLogic function for more details.
	 */
	UE_API void PopulateMissingActiveMoveLogic(const TArray<TObjectPtr<ULayeredMoveLogic>>& RegisteredMoves);
	
	/** Adds the active move to the queued array of the move group */
	UE_API void QueueLayeredMove(const TSharedPtr<FLayeredMoveInstance>& Move);
	
	/** @return True if there are any active or queued moves in this group */
	bool HasAnyMoves() const { return (!ActiveMoves.IsEmpty() || !QueuedMoves.IsEmpty()); }
	
	/** @return True if there is at least one layered move that's either active or queued and is associated with the provided logic or data type */
	template <typename MoveElementT UE_REQUIRES(std::is_base_of_v<FLayeredMoveInstancedData, MoveElementT> || std::is_base_of_v<ULayeredMoveLogic, MoveElementT>)>
	bool HasMove() const
	{
		return FindActiveMove<MoveElementT>() || FindQueuedMove<MoveElementT>();
	}
	
	/** Get a simplified string representation of this group. Typically for debugging. */
	UE_API FString ToSimpleString() const;
	
	/** Returns the first active layered move associated with logic of the specified type, if one exists */
	template <typename MoveLogicT = ULayeredMoveLogic UE_REQUIRES(std::is_base_of_v<ULayeredMoveLogic, MoveLogicT>)>
	const FLayeredMoveInstance* FindActiveMove(TSubclassOf<ULayeredMoveLogic> MoveLogicClass = MoveLogicT::StaticClass()) const
	{
		return PrivateFindActiveMove(MoveLogicClass);
	}
	
	/** Returns the first active layered move using data of the specified type, if one exists */
	template <typename MoveDataT = FLayeredMoveInstancedData UE_REQUIRES(std::is_base_of_v<FLayeredMoveInstancedData, MoveDataT>)>
	const FLayeredMoveInstance* FindActiveMove(const UScriptStruct* MoveDataType = MoveDataT::StaticStruct()) const
	{
		return PrivateFindActiveMove(MoveDataType);
	}
	
	/** Returns the first queued layered move associated with logic of the specified type, if one exists */
	template <typename MoveLogicT = ULayeredMoveLogic UE_REQUIRES(std::is_base_of_v<ULayeredMoveLogic, MoveLogicT>)>
	const FLayeredMoveInstance* FindQueuedMove(TSubclassOf<ULayeredMoveLogic> MoveLogicClass = MoveLogicT::StaticClass()) const
	{
		return PrivateFindQueuedMove(MoveLogicClass);
	}

	/** Returns the first queued layered move using data of the specified type, if one exists */
	template <typename MoveDataT = FLayeredMoveInstancedData UE_REQUIRES(std::is_base_of_v<FLayeredMoveInstancedData, MoveDataT>)>
	const FLayeredMoveInstance* FindQueuedMove(const UScriptStruct* MoveDataType = MoveDataT::StaticStruct()) const
	{
		return PrivateFindQueuedMove(MoveDataType);
	}

	/** Cancel any active or queued moves with a matching tag */
	void CancelMovesByTag(FGameplayTag Tag, bool bRequireExactMatch); 

	// Clears out any finished or invalid active moves and adds any queued moves to the active moves
	UE_API void FlushMoveArrays(const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard, bool bIsAsync = false);

	// Flushes move arrays using async start/end calls, then returns only moves whose logic SupportsAsync()
	UE_API TArray<TSharedPtr<FLayeredMoveInstance>> GenerateActiveMoves_Async(const FMoverTimeStep& TimeStep, UMoverBlackboard* SimBlackboard);
	
protected:
	// Helper function for gathering any residual velocity settings from layered moves that just ended
	void ProcessFinishedMove(const FLayeredMoveInstance& Move, bool& bResidualVelocityOverriden, bool& bClampVelocityOverriden);
	
	/** Moves that are currently active in this group */
	TArray<TSharedPtr<FLayeredMoveInstance>> ActiveMoves;

	/** Moves that are queued to become active next sim frame */
	TArray<TSharedPtr<FLayeredMoveInstance>> QueuedMoves;

private:
	const FLayeredMoveInstance* PrivateFindActiveMove(const TSubclassOf<ULayeredMoveLogic>& MoveLogicClass) const;
	const FLayeredMoveInstance* PrivateFindActiveMove(const UScriptStruct* MoveDataType) const;
	const FLayeredMoveInstance* PrivateFindQueuedMove(const TSubclassOf<ULayeredMoveLogic>& MoveLogicClass) const;
	const FLayeredMoveInstance* PrivateFindQueuedMove(const UScriptStruct* MoveDataType) const;
	
	//@todo DanH: Maybe these should be grouped in a struct?
	/**
	 * Clamps an actors velocity to this value when a layered move ends. This expects Value >= 0.
	 * Note: This is set automatically when a layered move has ended based off of end velocity settings - it is not meant to be set by the user see @FLayeredMoveFinishVelocitySettings
	 */
	UPROPERTY(NotReplicated, VisibleAnywhere, BlueprintReadOnly, Category = Mover, meta = (AllowPrivateAccess))
	float ResidualClamping;

	/**
	 * If true ResidualVelocity will be the next velocity used for this actor
	 * Note: This is set automatically when a layered move has ended based off of end velocity settings - it is not meant to be set by the user see @FLayeredMoveFinishVelocitySettings
	 */
	UPROPERTY(NotReplicated, VisibleAnywhere, BlueprintReadOnly, Category = Mover, meta = (AllowPrivateAccess))
	bool bApplyResidualVelocity;

	/**
	 * If bApplyResidualVelocity is true this actors velocity will be set to this.
	 * Note: This is set automatically when a layered move has ended based off of end velocity settings - it is not meant to be set by the user see @FLayeredMoveFinishVelocitySettings
	 */
	UPROPERTY(NotReplicated, VisibleAnywhere, BlueprintReadOnly, Category = Mover, meta = (AllowPrivateAccess))
	FVector ResidualVelocity;

	/** Used during simulation to cancel any moves that match a tag */
	TArray<TPair<FGameplayTag, bool>> TagCancellationRequests;
};

template<>
struct TStructOpsTypeTraits<FLayeredMoveInstanceGroup> : public TStructOpsTypeTraitsBase2<FLayeredMoveInstanceGroup>
{
	enum
	{
		WithCopy = true,
		//WithNetSerializer = true,
		WithIdenticalViaEquality = true,
		WithAddStructReferencedObjects = true,
	};
};

#undef UE_API
