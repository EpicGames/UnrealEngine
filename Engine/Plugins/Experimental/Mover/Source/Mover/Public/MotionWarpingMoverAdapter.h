// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MotionWarpingAdapter.h"
#include "MoverComponent.h"
#include "MotionWarpingMoverAdapter.generated.h"

#define UE_API MOVER_API

// Adapter for MoverComponent actors to participate in motion warping

UCLASS(MinimalAPI)
class UMotionWarpingMoverAdapter : public UMotionWarpingBaseAdapter
{
	GENERATED_BODY()

public:
	UE_API virtual void BeginDestroy() override;

	UE_API void SetMoverComp(UMoverComponent* InMoverComp);

	UE_API virtual AActor* GetActor() const override;
	UE_API virtual USkeletalMeshComponent* GetMesh() const override;
	UE_API virtual FVector GetVisualRootLocation() const override;
	UE_API virtual FVector GetBaseVisualTranslationOffset() const override;
	UE_API virtual FQuat GetBaseVisualRotationOffset() const override;

private:
	// This is called when our Mover actor wants to warp local motion, and passes the responsibility onto the warping component
	FTransform WarpLocalRootMotionOnMoverComp(const FTransform& LocalRootMotionTransform, float DeltaSeconds, const FMotionWarpingUpdateContext* OptionalWarpingContext);

	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<UMoverComponent> TargetMoverComp;
};

#undef UE_API
