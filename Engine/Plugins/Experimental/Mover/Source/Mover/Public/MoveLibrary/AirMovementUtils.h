// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MoverDataModelTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "AirMovementUtils.generated.h"

#define UE_API MOVER_API

struct FProposedMove;
struct FFloorCheckSettings;

// Input parameters for controlled free movement function
USTRUCT(BlueprintType)
struct FFreeMoveParams
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	EMoveInputType MoveInputType = EMoveInputType::DirectionalIntent;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FVector MoveInput = FVector::ZeroVector;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FRotator OrientationIntent = FRotator::ZeroRotator;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FVector PriorVelocity = FVector::ZeroVector;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FRotator PriorOrientation = FRotator::ZeroRotator;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float MaxSpeed = 800.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float Acceleration = 4000.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float Deceleration = 8000.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float TurningBoost = 8.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float TurningRate = 500.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float DeltaSeconds = 0.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float Friction = 0.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FQuat WorldToGravityQuat = FQuat::Identity;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	bool bUseAccelerationForVelocityMove = true;
};

/**
 * AirMovementUtils: a collection of stateless static BP-accessible functions for a variety of air movement-related operations
 */
UCLASS(MinimalAPI)
class UAirMovementUtils : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
	
public:
	/** Generate a new movement based on move/orientation intents and the prior state, unconstrained like when flying */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API FProposedMove ComputeControlledFreeMove(const FFreeMoveParams& InParams);
	
    // Checks if a hit result represents a walkable location that an actor can land on
    UFUNCTION(BlueprintCallable, Category=Mover)
    static UE_API bool IsValidLandingSpot(const FMovingComponentSet& MovingComps, const FVector& Location, const FHitResult& Hit, const FFloorCheckSettings& FloorCheckSettings, UPARAM(Ref) FFloorCheckResult& OutFloorResult);

    UE_DEPRECATED(5.8, "Use the overload taking FFloorCheckSettings instead.")
    static UE_API bool IsValidLandingSpot(const FMovingComponentSet& MovingComps, const FVector& Location, const FHitResult& Hit, float FloorSweepDistance, float MaxWalkSlopeCosine, bool bUseFlatBaseForFloorChecks, UPARAM(Ref) FFloorCheckResult& OutFloorResult, float PerchRadiusThreshold = 0.f);

    /** Attempts to move a component along a surface, while checking for landing on a walkable surface. Intended for use while falling. Returns the percent of time applied, with 0.0 meaning no movement occurred. */
    UFUNCTION(BlueprintCallable, Category=Mover)
    static UE_API float TryMoveToFallAlongSurface(const FMovingComponentSet& MovingComps, const FVector& Delta, float PctOfDeltaToMove, const FQuat Rotation, const FVector& Normal, UPARAM(Ref) FHitResult& Hit, bool bHandleImpact, const FFloorCheckSettings& FloorCheckSettings, UPARAM(Ref) FFloorCheckResult& OutFloorResult, UPARAM(Ref) FMovementRecord& MoveRecord);

    UE_DEPRECATED(5.8, "Use the overload taking FFloorCheckSettings instead.")
    static UE_API float TryMoveToFallAlongSurface(const FMovingComponentSet& MovingComps, const FVector& Delta, float PctOfDeltaToMove, const FQuat Rotation, const FVector& Normal, UPARAM(Ref) FHitResult& Hit, bool bHandleImpact, float FloorSweepDistance, float MaxWalkSlopeCosine, bool bUseFlatBaseForFloorChecks, UPARAM(Ref) FFloorCheckResult& OutFloorResult, UPARAM(Ref) FMovementRecord& MoveRecord, float PerchRadiusThreshold = 0.f);

public:	
	// Threadsafe movement queries

	/** Tests potential movement of a component falling/sliding along a surface, while checking for landing on a walkable surface. Intended for use while falling. 
	 * Modifies InOutHit with final movement hit data
	 * Sets OutFloorResult with any found walkable surface info
     * Appends to InOutMoveRecord with any movement substeps
	 * Returns the percent of time applied, with 0.0 meaning no movement would occur. 
	 */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API float TestFallingMoveAlongHitSurface(const FMovingComponentSet& MovingComps, const FVector& OriginalMoveDelta, const FVector& LocationAtHit, const FQuat& TargetRotation, bool bHandleImpact, const FFloorCheckSettings& FloorCheckSettings, UPARAM(Ref) FHitResult& InOutHit, UPARAM(Ref) FFloorCheckResult& OutFloorResult, UPARAM(Ref) FMovementRecord& InOutMoveRecord);

	UE_DEPRECATED(5.8, "Use the overload taking FFloorCheckSettings instead.")
	static UE_API float TestFallingMoveAlongHitSurface(const FMovingComponentSet& MovingComps, const FVector& OriginalMoveDelta, const FVector& LocationAtHit, const FQuat& TargetRotation, bool bHandleImpact, float FloorSweepDistance, float MaxWalkSlopeCosine, bool bUseFlatBaseForFloorChecks, UPARAM(Ref) FHitResult& InOutHit, UPARAM(Ref) FFloorCheckResult& OutFloorResult, UPARAM(Ref) FMovementRecord& InOutMoveRecord, float PerchRadiusThreshold = 0.f);

};

#undef UE_API
