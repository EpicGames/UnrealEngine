// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Components/SceneComponent.h"
#include "MoverDataModelTypes.h"
#include "MovementUtilsTypes.h"

#include "AsyncMovementUtils.generated.h"

#define UE_API MOVER_API

struct FMovementRecord;
class UMoverComponent;
struct FOptionalFloorCheckResult;


/**
 * AsyncMovementUtils: a collection of stateless static BP-accessible functions focused on testing potential movements in a 
 * threadsafe manner without actually causing immediate changes.
 */
UCLASS(MinimalAPI, Experimental)
class UAsyncMovementUtils : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Tests potential movement of a component without actually moving it, taking penetration resolution issues into account first.  
	 * Returns true if any movement was possible
	 * Modifies OutHit with final movement hit data
	 * Appends to InOutMoveRecord with any movement substeps
	 */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API bool TestDepenetratingMove(const FMovingComponentSet& MovingComps, const FVector& StartLocation, const FVector& TargetLocation, const FQuat& StartRotation, const FQuat& TargetRotation, bool bShouldSweep, FHitResult& OutHit, UPARAM(ref) FMovementRecord& InOutMoveRecord);

	/** Tests potential movement of a component without actually moving it, taking penetration resolution issues into account first.  Relies on CollisionParams to describe the query.
	 * Returns true if any movement was possible
	 * Modifies OutHit with final movement hit data
	 * Appends to InOutMoveRecord with any movement substeps
	 */
	static UE_API bool TestDepenetratingMove(const FMovingComponentSet& MovingComps, const FVector& StartLocation, const FVector& TargetLocation, const FQuat& StartRotation, const FQuat& TargetRotation, bool bShouldSweep, FMoverCollisionParams& CollisionParams, FHitResult& OutHit, FMovementRecord& InOutMoveRecord);

	/** Tests potential movement of a component sliding along a surface, without actually moving it. 
	 * Returns the percent of time applied, with 0.0 meaning no movement would occur. 
	 * Modifies InOutHit with final movement hit data
	 * Appends to InOutMoveRecord with any movement substeps
	 */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API float TestSlidingMoveAlongHitSurface(const FMovingComponentSet& MovingComps, const FVector& OriginalMoveDelta, const FVector& LocationAtHit, const FQuat& TargetRotation, UPARAM(ref) FHitResult& InOutHit, UPARAM(ref) FMovementRecord& InOutMoveRecord);
	

	/** Tests potential movement of a component sliding along a surface, without actually moving it. Relies on CollisionParams to describe the query.
	 * Returns the percent of time applied, with 0.0 meaning no movement would occur. 
	 * Modifies InOutHit with final movement hit data
	 * Appends to InOutMoveRecord with any movement substeps
	 */
	static UE_API float TestSlidingMoveAlongHitSurface(const FMovingComponentSet& MovingComps, const FVector& OriginalMoveDelta, const FVector& LocationAtHit, const FQuat& TargetRotation, FMoverCollisionParams& CollisionParams, FHitResult& InOutHit, FMovementRecord& InOutMoveRecord);


	/** Attempts to find a move that would resolve an initially penetrating blockage.
	 * Returns true if an adjustment was found. The adjustment will be in OutAdjustmentDelta.
	 */
	static UE_API bool FindMoveToResolveInitialPenetration_Internal(const FMovingComponentSet& MovingComps, const FVector& StartLocation, const FQuat& StartRotation, const FHitResult& PenetratingHit, FMoverCollisionParams& CollisionParams, FVector& OutAdjustmentDelta);


	/** Tests a move of a component without actually moving it. 
	 * Returns true if any motion could occur. Detailed blocking hit results are written to OutHit.  
	 * If not sweeping, full movement will always be allowed even if new blocking overlaps would occur.
	 */
	static UE_API bool TestMoveComponent_Internal(const FMovingComponentSet& MovingComps, const FVector& StartLocation, const FVector& TargetLocation, const FQuat& StartRotation, const FQuat& TargetRotation, bool bShouldSweep, FMoverCollisionParams& CollisionParams, FHitResult& OutHit);

	
};

#undef UE_API
