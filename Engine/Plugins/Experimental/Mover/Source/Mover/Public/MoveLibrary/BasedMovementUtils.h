// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoverTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Engine/EngineBaseTypes.h"
#include "BasedMovementUtils.generated.h"

#define UE_API MOVER_API

class UMoverComponent;
struct FMovementRecord;
struct FFloorCheckResult;

/** Data about the object a Mover actor is basing its movement on, such as when standing on a moving platform */
USTRUCT(BlueprintType)
struct FRelativeBaseInfo
{
	GENERATED_BODY()

	/** Component we are moving relative to */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadWrite, Category="Mover")
	TWeakObjectPtr<UPrimitiveComponent> MovementBase = nullptr;

	/** Bone name on component, for skeletal meshes. NAME_None if not a skeletal mesh or if bone is invalid. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadWrite, Category = "Mover")
	FName BoneName = NAME_None;

	/** Last captured worldspace location of MovementBase / Bone */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadWrite, Category = "Mover")
	FVector Location = FVector::ZeroVector;

	/** Last captured worldspace orientation of MovementBase / Bone */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadWrite, Category = "Mover")
	FQuat Rotation = FQuat::Identity;

	/** Last captured location of the tethering point where the Mover actor is "attached", relative to the base. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadWrite, Category = "Mover")
	FVector ContactLocalPosition = FVector::ZeroVector;

	/** Last captured offset of the Mover actor from contact location, in world space. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadWrite, Category = "Mover")
	FVector WorldspaceOffsetFromContactPos = FVector::ZeroVector;


public:

	UE_API void Clear();

	UE_API bool HasRelativeInfo() const;
	UE_API bool UsesSameBase(const FRelativeBaseInfo& Other) const;
	UE_API bool UsesSameBase(const UPrimitiveComponent* OtherComp, FName OtherBoneName=NAME_None) const;

	UE_API void SetFromFloorResult(const FFloorCheckResult& FloorTestResult, const FVector ObjectWorldLocation, const FVector UpDir);

	UE_API void UpdateToLatestBaseTransform();

	UE_API FString ToString() const;
};


/**
 * MovementBaseUtils: a collection of stateless static BP-accessible functions for based movement
 */
UCLASS(MinimalAPI)
class UBasedMovementUtils : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Determine whether MovementBase can possibly move. */
	UFUNCTION(BlueprintCallable, Category = "Mover/MovementBases")
	static UE_API bool IsADynamicBase(const UPrimitiveComponent* MovementBase);

	/** Determine whether MovementBase's movement is performed via physics. */
	UFUNCTION(BlueprintCallable, Category = "Mover/MovementBases")
	static UE_API bool IsBaseSimulatingPhysics(const UPrimitiveComponent* MovementBase);
	
	/** Get the transform (local-to-world) for the given MovementBase, optionally at the location of a bone. Returns false if MovementBase is nullptr, or if BoneName is not a valid bone. */
	UFUNCTION(BlueprintCallable, Category="Mover/MovementBases")
	static UE_API bool GetMovementBaseTransform(const UPrimitiveComponent* MovementBase, const FName BoneName, FVector& OutLocation, FQuat& OutQuat);

	/** Get the velocity (world space) for the given MovementBase. Returns false and sets OutWorldVelocity to zero if MovementBase/Bone isn't valid. */
	UFUNCTION(BlueprintCallable, Category = "Mover/MovementBases", meta=(ReturnDisplayName = "Did Get Velocity"))
	static UE_API bool GetMovementBaseVelocity(const UPrimitiveComponent* MovementBase, FVector& OutWorldVelocity);

	/** Get the velocity (world space) for the given MovementBase at a given point, taking its translation and angular velocity into account. Returns false and sets OutWorldVelocity to zero if MovementBase/Bone isn't valid. */
	UFUNCTION(BlueprintCallable, Category = "Mover/MovementBases", meta=(ReturnDisplayName = "Did Get Velocity"))
	static UE_API bool GetMovementBaseVelocityAtPoint(const UPrimitiveComponent* MovementBase, const FVector& WorldPoint, FVector& OutWorldVelocity);

	/** Convert a local location to a world location for a given MovementBase. Returns false if MovementBase is nullptr, or if BoneName is not a valid bone. Scaling is ignored. */
	UFUNCTION(BlueprintCallable, Category = "Mover/MovementBases")
	static UE_API bool TransformBasedLocationToWorld(const UPrimitiveComponent* MovementBase, const FName BoneName, FVector LocalLocation, FVector& OutLocationWorldSpace);

	/** Convert a world location to a local location for a given MovementBase, optionally at the location of a bone. Returns false if MovementBase is nullptr, or if BoneName is not a valid bone. Scaling is ignored. */
	UFUNCTION(BlueprintCallable, Category = "Mover/MovementBases")
	static UE_API bool TransformWorldLocationToBased(const UPrimitiveComponent* MovementBase, const FName BoneName, FVector WorldSpaceLocation, FVector& OutLocalLocation);

	/** Convert a local direction to a world direction for a given MovementBase. Returns false if MovementBase is nullptr, or if BoneName is not a valid bone. Scaling is ignored. */
	UFUNCTION(BlueprintCallable, Category = "Mover/MovementBases")
	static UE_API bool TransformBasedDirectionToWorld(const UPrimitiveComponent* MovementBase, const FName BoneName, FVector LocalDirection, FVector& OutDirectionWorldSpace);

	/** Convert a world direction to a local direction for a given MovementBase, optionally relative to the orientation of a bone. Returns false if MovementBase is nullptr, or if BoneName is not a valid bone. Scaling is ignored. */
	UFUNCTION(BlueprintCallable, Category = "Mover/MovementBases")
	static UE_API bool TransformWorldDirectionToBased(const UPrimitiveComponent* MovementBase, const FName BoneName, FVector WorldSpaceDirection, FVector& OutLocalDirection);

	/** Convert a local rotator to world space for a given MovementBase. Returns false if MovementBase is nullptr, or if BoneName is not a valid bone. Scaling is ignored. */
	UFUNCTION(BlueprintCallable, Category = "Mover/MovementBases")
	static UE_API bool TransformBasedRotatorToWorld(const UPrimitiveComponent* MovementBase, const FName BoneName, FRotator LocalRotator, FRotator& OutWorldSpaceRotator);

	/** Convert a world space rotator to a local rotator for a given MovementBase, optionally relative to the orientation of a bone. Returns false if MovementBase is nullptr, or if BoneName is not a valid bone. Scaling is ignored. */
	UFUNCTION(BlueprintCallable, Category = "Mover/MovementBases")
	static UE_API bool TransformWorldRotatorToBased(const UPrimitiveComponent* MovementBase, const FName BoneName, FRotator WorldSpaceRotator, FRotator& OutLocalRotator);

	/** Convert a local location to a world location for a given MovementBase. Returns false if MovementBase is nullptr, or if BoneName is not a valid bone. Scaling is ignored. */
	UFUNCTION(BlueprintCallable, Category = "Mover/MovementBases")
	static UE_API void TransformLocationToWorld(FVector BasePos, FQuat BaseQuat, FVector LocalLocation, FVector& OutLocationWorldSpace);

	/** Convert a world location to a local location for a given MovementBase, optionally at the location of a bone. Returns false if MovementBase is nullptr, or if BoneName is not a valid bone. Scaling is ignored. */
	UFUNCTION(BlueprintCallable, Category = "Mover/MovementBases")
	static UE_API void TransformLocationToLocal(FVector BasePos, FQuat BaseQuat, FVector WorldSpaceLocation, FVector& OutLocalLocation);

	/** Convert a local direction to a world direction for a given MovementBase. Returns false if MovementBase is nullptr, or if BoneName is not a valid bone. Scaling is ignored. */
	UFUNCTION(BlueprintCallable, Category = "Mover/MovementBases")
	static UE_API void TransformDirectionToWorld(FQuat BaseQuat, FVector LocalDirection, FVector& OutDirectionWorldSpace);

	/** Convert a world direction to a local direction for a given MovementBase, optionally relative to the orientation of a bone. Returns false if MovementBase is nullptr, or if BoneName is not a valid bone. Scaling is ignored. */
	UFUNCTION(BlueprintCallable, Category = "Mover/MovementBases")
	static UE_API void TransformDirectionToLocal(FQuat BaseQuat, FVector WorldSpaceDirection, FVector& OutLocalDirection);

	/** Convert a local rotator to world space for a given MovementBase. Returns false if MovementBase is nullptr, or if BoneName is not a valid bone. Scaling is ignored. */
	UFUNCTION(BlueprintCallable, Category = "Mover/MovementBases")
	static UE_API void TransformRotatorToWorld(FQuat BaseQuat, FRotator LocalRotator, FRotator& OutWorldSpaceRotator);

	/** Convert a world space rotator to a local rotator for a given MovementBase, optionally relative to the orientation of a bone. Returns false if MovementBase is nullptr, or if BoneName is not a valid bone. Scaling is ignored. */
	UFUNCTION(BlueprintCallable, Category = "Mover/MovementBases")
	static UE_API void TransformRotatorToLocal(FQuat BaseQuat, FRotator WorldSpaceRotator, FRotator& OutLocalRotator);

	/** Apply a sweeping movement to attempt to follow a movement base that we're moving relative to. This is typically called during movement simulation. */
	static UE_API FRelativeBaseInfo ApplyMovementBaseChange(UMoverComponent* TargetMoverComp, const FRelativeBaseInfo& RelativeBaseInfo, bool bIgnoreBaseRotation);

	/** Makes it so BasedObjectTick ticks after NewBase's actor ticking */
	static UE_API void AddTickDependency(FTickFunction& BasedObjectTick, UPrimitiveComponent* NewBase);
	
	/** Removes ticking dependency of BasedObjectTick on OldBase */
	static UE_API void RemoveTickDependency(FTickFunction& BasedObjectTick, UPrimitiveComponent* OldBase);

	/** Attempts to move the actor to keep up with its base's movement using a simple sweep. This function is not intended to be called during a Mover actor's simulation tick. */
	static UE_API void UpdateSimpleBasedMovement(UMoverComponent* TargetMoverComp);
};


/**
 * Tick function used to perform based movement at dynamic times throughout the world update time, typically out-of-band with the movement simulation
 **/
USTRUCT()
struct FMoverDynamicBasedMovementTickFunction : public FTickFunction
{
	GENERATED_BODY()

	/** MoverComponent that is the target of this tick **/
	UMoverComponent* TargetMoverComp;

	/** If true, this tick function will self-disable after running */
	bool bAutoDisableAfterTick = false;

	/**
	 * Abstract function actually execute the tick.
	 * @param DeltaTime - frame time to advance, in seconds
	 * @param TickType - kind of tick for this frame
	 * @param CurrentThread - thread we are executing on, useful to pass along as new tasks are created
	 * @param MyCompletionGraphEvent - completion event for this task. Useful for holding the completion of this task until certain child tasks are complete.
	 **/
	virtual void ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
	/** Abstract function to describe this tick. Used to print messages about illegal cycles in the dependency graph **/
	virtual FString DiagnosticMessage() override;
	/** Function used to describe this tick for active tick reporting. **/
	virtual FName DiagnosticContext(bool bDetailed) override;
};

template<>
struct TStructOpsTypeTraits<FMoverDynamicBasedMovementTickFunction> : public TStructOpsTypeTraitsBase2<FMoverDynamicBasedMovementTickFunction>
{
	enum
	{
		WithCopy = false
	};
};

#undef UE_API
