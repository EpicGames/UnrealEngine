// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Engine/EngineBaseTypes.h"
#include "ConstrainedMoveUtils.generated.h"

#define UE_API MOVER_API



/** Encapsulates info about constraining movement to a plane, such as in a side-scroller */
USTRUCT(BlueprintType)
struct FPlanarConstraint
{
	GENERATED_BODY()

	/** If true, movement will be constrained to a plane. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PlanarMovement)
	bool bConstrainToPlane = false;

	/**
	 * The normal or axis of the plane that constrains movement, if bConstrainToPlane is enabled.
	 * If for example you wanted to constrain movement to the X-Z plane (so that Y cannot change), the normal would be set to X=0 Y=1 Z=0.
	 * It is normalized once the component is registered with the game world.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PlanarMovement, meta = (editcondition = bConstrainToPlane))
	FVector PlaneConstraintNormal = FVector::ForwardVector;

	/** The origin of the plane that constrains movement, if plane constraint is enabled. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PlanarMovement, meta = (editcondition = bConstrainToPlane))
	FVector PlaneConstraintOrigin = FVector::ZeroVector;
};



/**
 * PlanarConstraintUtils: a collection of stateless static BP-accessible functions for working with planar constraints
 */
UCLASS(MinimalAPI)
class UPlanarConstraintUtils : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Sets whether or not the constraint is enabled */
	UFUNCTION(BlueprintCallable, Category = "Mover|PlanarConstraint")
	static UE_API void SetPlanarConstraintEnabled(UPARAM(ref) FPlanarConstraint& Constraint, bool bEnabled);

	/**
	 * Sets the normal of the plane that constrains movement, enforced if the plane constraint is enabled.
	 *
	 * @param PlaneNormal	The normal of the plane. If non-zero in length, it will be normalized.
	 */
	UFUNCTION(BlueprintCallable, Category = "Mover|PlanarConstraint")
	static UE_API void SetPlanarConstraintNormal(UPARAM(ref) FPlanarConstraint& Constraint, FVector PlaneNormal);

	/** Sets the origin of the plane that constrains movement, enforced if the plane constraint is enabled. */
	UFUNCTION(BlueprintCallable, Category = "Mover|PlanarConstraint")
	static UE_API void SetPlaneConstraintOrigin(UPARAM(ref) FPlanarConstraint& Constraint, FVector PlaneOrigin);

	// APPLICATION OF CONSTRAINT

	/** Constrains a direction to be on the plane, if enabled */
	UFUNCTION(BlueprintCallable, Category = "Mover|PlanarConstraint")
	static UE_API FVector ConstrainDirectionToPlane(const FPlanarConstraint& Constraint, FVector Direction, bool bMaintainMagnitude=false);

	/** Constrains a location to be on the plane, if enabled */
	UFUNCTION(BlueprintCallable, Category = "Mover|PlanarConstraint")
	static UE_API FVector ConstrainLocationToPlane(const FPlanarConstraint& Constraint, FVector Location);

	/** Constrains a normal to be on the plane, if enabled. Result will be re-normalized. */
	UFUNCTION(BlueprintCallable, Category = "Mover|PlanarConstraint")
	static UE_API FVector ConstrainNormalToPlane(const FPlanarConstraint& Constraint, FVector Normal);

}; // UPlanarConstraintUtils

#undef UE_API
