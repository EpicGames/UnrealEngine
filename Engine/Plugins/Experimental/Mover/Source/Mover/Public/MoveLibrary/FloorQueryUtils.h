// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovementUtilsTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Components/PrimitiveComponent.h"
#include "FloorQueryUtils.generated.h"

#define UE_API MOVER_API

namespace UE::FloorQueryUtility
{
	extern const float MIN_FLOOR_DIST;
	extern const float MAX_FLOOR_DIST;
	extern const float SWEEP_EDGE_REJECT_DISTANCE;
}

/** Settings that control how floor checks are performed */
USTRUCT(BlueprintType)
struct FFloorCheckSettings
{
	GENERATED_BODY()

	/** Max distance to scan for floor surfaces under a Mover actor */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Floor", meta = (ForceUnits = "cm"))
	float FloorSweepDistance = 40.0f;

	/** Walkable slope angle, represented as cosine(max slope angle) for performance reasons */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Floor")
	float MaxWalkSlopeCosine = 0.71f;

	/** If true, use a flat base shape for floor checks instead of a rounded capsule bottom */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Floor")
	bool bUseFlatBaseForFloorChecks = true;

	/** Distance from the capsule edge within which floor surface contact is rejected for perching purposes */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Floor", meta = (ClampMin = "0", UIMin = "0", ForceUnits = "cm"))
	float PerchRadiusThreshold = 0.f;
};


/** Data about the floor for walking movement, used by Mover simulations */
USTRUCT(BlueprintType)
struct FFloorCheckResult
{
	GENERATED_BODY()

	/**
	* True if there was a blocking hit in the floor test that was NOT in initial penetration.
	* The HitResult can give more info about other circumstances.
	*/
	UPROPERTY(VisibleInstanceOnly, BlueprintReadWrite, Category = CharacterFloor)
	uint32 bBlockingHit : 1;

	/** True if the hit found a valid walkable floor. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadWrite, Category = CharacterFloor)
	uint32 bWalkableFloor : 1;

	/** True if the hit found a valid walkable floor using a line trace (rather than a sweep test, which happens when the sweep test fails to yield a walkable surface). */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadWrite, Category = CharacterFloor)
	uint32 bLineTrace : 1;

	/** The distance to the floor, computed from the trace. Only valid if bLineTrace is true. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadWrite, Category = CharacterFloor)
    float LineDist;

	/** The distance to the floor, computed from the swept capsule trace. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadWrite, Category = CharacterFloor)
	float FloorDist;

	/** Hit result of the test that found a floor. Includes more specific data about the point of impact and surface normal at that point. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadWrite, Category = CharacterFloor)
	FHitResult HitResult;

public:

	FFloorCheckResult()
		: bBlockingHit(false)
		, bWalkableFloor(false)
		, bLineTrace(false)
		, LineDist(0.f)
		, FloorDist(0.f)
		, HitResult(1.f)
	{
	}

	/** Returns true if the floor result hit a walkable surface. */
	bool IsWalkableFloor() const
	{
		return bBlockingHit && bWalkableFloor;
	}

	void Clear()
	{
		bBlockingHit = false;
		bWalkableFloor = false;
		FloorDist = 0.f;
		bLineTrace = false;
		LineDist = 0.f;
		HitResult.Reset(1.f, false);
	}

	/** Gets the distance to floor, either LineDist or FloorDist. */
	float GetDistanceToFloor() const
	{
		return FloorDist;
	}

	/** Sets the passed in hit result with data from the floor Sweep */
	UE_API void SetFromSweep(const FHitResult& InHit, const float InSweepFloorDist, const bool bIsWalkableFloor);
	/** Sets the passed in hit result with data from the line trace */
	UE_API void SetFromLineTrace(const FHitResult& InHit, const float InSweepFloorDist, const float InLineDist, const bool bIsWalkableFloor);
};

/** Used by some movement operations to conditionally return a floor check result, if one was performed */
struct FOptionalFloorCheckResult
{
	uint8 bHasFloorResult : 1;			// If true, the floor test result has been computed
	FFloorCheckResult FloorTestResult;

	FOptionalFloorCheckResult()
	{
		bHasFloorResult = false;
	}
};

/**
 * FloorQueryUtils: a collection of stateless static BP-accessible functions for a variety of operations involving floor checks
 */
UCLASS(MinimalAPI)
class UFloorQueryUtils : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	static UE_API void FindFloor(const FMovingComponentSet& MovingComps, const FFloorCheckSettings& FloorCheckSettings, const FVector& Location, FFloorCheckResult& OutFloorResult);

	UE_DEPRECATED(5.8, "Use the overload taking FFloorCheckSettings instead.")
	static UE_API void FindFloor(const FMovingComponentSet& MovingComps, float FloorSweepDistance, float MaxWalkSlopeCosine, bool bUseFlatBaseForFloorChecks, const FVector& Location, FFloorCheckResult& OutFloorResult, float PerchRadiusThreshold = 0.f);

	static UE_API void ComputeFloorDist(const FMovingComponentSet& MovingComps, const FFloorCheckSettings& FloorCheckSettings, float LineTraceDistance, const FVector& Location, FFloorCheckResult& OutFloorResult);

	UE_DEPRECATED(5.8, "Use the overload taking FFloorCheckSettings instead.")
	static UE_API void ComputeFloorDist(const FMovingComponentSet& MovingComps, float LineTraceDistance, float FloorSweepDistance, float MaxWalkSlopeCosine, const FVector& Location, bool bUseFlatBaseForFloorChecks, FFloorCheckResult& OutFloorResult, float PerchRadiusThreshold = 0.f);

	static UE_API bool FloorSweepTest(const FMovingComponentSet& MovingComps, FHitResult& OutHit, const FVector& Start, const FVector& End, ECollisionChannel TraceChannel, const FCollisionShape& CollisionShape, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParam, bool bUseFlatBaseForFloorChecks, float PerchRadiusThreshold = 0.f);

	UFUNCTION(BlueprintCallable, Category=Mover)
	static UE_API bool IsHitSurfaceWalkable(const FHitResult& Hit, const FVector& UpDirection, float MaxWalkSlopeCosine);

	/**
	 * Return true if the 2D distance to the impact point is inside the edge tolerance (CapsuleRadius minus a small rejection threshold).
	 * Useful for rejecting adjacent hits when finding a floor or landing spot.
	 */
	static UE_API bool IsWithinEdgeTolerance(const FVector& CapsuleLocation, const FVector& TestImpactPoint, float CapsuleRadius, const FVector& UpDirection);


	/** 
	 * Try to find a floor underneath a Mover actor. Requires use of UCommonLegacyMovementSettings. 
	 */
	UFUNCTION(BlueprintCallable, Category=Mover)
	static UE_API void TryFindFloor(UMoverComponent* MoverComp, bool& DidFindWalkableFloor, FFloorCheckResult& FloorResult);

};

#undef UE_API
