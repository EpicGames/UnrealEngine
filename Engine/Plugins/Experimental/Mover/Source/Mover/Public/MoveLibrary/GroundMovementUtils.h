// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MoverDataModelTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GroundMovementUtils.generated.h"

#define UE_API MOVER_API

class UMoverComponent;
struct FMovementRecord;
struct FFloorCheckResult;
struct FFloorCheckSettings;
struct FOptionalFloorCheckResult;
struct FHitResult;
struct FProposedMove;

// Input parameters for controlled ground movement function
USTRUCT(BlueprintType)
struct FGroundMoveParams
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	EMoveInputType MoveInputType = EMoveInputType::DirectionalIntent;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FVector MoveInput = FVector::ZeroVector;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FRotator OrientationIntent = FRotator::ZeroRotator;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FVector PriorVelocity = FVector::ZeroVector;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FRotator PriorOrientation = FRotator::ZeroRotator;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float MaxSpeed = 800.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float Acceleration = 4000.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float Deceleration = 8000.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float Friction = 0.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float TurningRate = 500.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float TurningBoost = 8.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FVector GroundNormal = FVector::UpVector;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float DeltaSeconds = 0.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FVector UpDirection = FVector::UpVector;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FQuat WorldToGravityQuat = FQuat::Identity;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	bool bUseAccelerationForVelocityMove = true;
};

/**
 * GroundMovementUtils: a collection of stateless static BP-accessible functions for a variety of ground movement-related operations
 */
UCLASS(MinimalAPI)
class UGroundMovementUtils : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
	
public:
	/** Generate a new movement based on move/orientation intents and the prior state, constrained to the ground movement plane. Also applies deceleration friction as necessary. */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API FProposedMove ComputeControlledGroundMove(const FGroundMoveParams& InParams);
	
	/** Move up steps or slope. Returns the percent of time applied, with 0.0 meaning no movement occurred and hit surface is invalid for step-up use. */
	static UE_API float TryMoveToStepUp(const FMovingComponentSet& MovingComps, const FVector& GravDir, float MaxStepHeight, const FFloorCheckSettings& FloorCheckSettings, const FVector& MoveDelta, const FHitResult& Hit, const FFloorCheckResult& CurrentFloor, bool bIsFalling, FOptionalFloorCheckResult* OutStepDownResult, FMovementRecord& MoveRecord);

	UE_DEPRECATED(5.8, "Use the overload taking FFloorCheckSettings instead.")
	static UE_API float TryMoveToStepUp(const FMovingComponentSet& MovingComps, const FVector& GravDir, float MaxStepHeight, float MaxWalkSlopeCosine, bool bUseFlatBaseForFloorChecks, float FloorSweepDistance, const FVector& MoveDelta, const FHitResult& Hit, const FFloorCheckResult& CurrentFloor, bool bIsFalling, FOptionalFloorCheckResult* OutStepDownResult, FMovementRecord& MoveRecord, float PerchRadiusThreshold = 0.f);

	/** Moves vertically to stay within range of the walkable floor. Does nothing and returns false if floor is unwalkable or if already in range. */ 
	static UE_API bool TryMoveToAdjustHeightAboveFloor(const FMovingComponentSet& MovingComps, FFloorCheckResult& CurrentFloor, float MaxWalkSlopeCosine, FMovementRecord& MoveRecord);

	/** Moves vertically to maintain the minimum distance above the walkable floor. Does nothing and returns false if floor is unwalkable or if already far enough away. */
	static UE_API bool TryMoveToKeepMinHeightAboveFloor(const FMovingComponentSet& MovingComps, FFloorCheckResult& CurrentFloor, float MaxWalkSlopeCosine, FMovementRecord& MoveRecord);

	/** Attempts to move a component along a surface in the walking mode. Returns the percent of time applied, with 0.0 meaning no movement occurred.
     *  Note: This modifies the normal and calls UMovementUtils::TryMoveToSlideAlongSurface
     */
    UFUNCTION(BlueprintCallable, Category=Mover)
    static UE_API float TryWalkToSlideAlongSurface(const FMovingComponentSet& MovingComps, const FVector& Delta, float PctOfDeltaToMove, const FQuat Rotation, const FVector& Normal, UPARAM(ref) FHitResult& Hit, bool bHandleImpact, UPARAM(ref) FMovementRecord& MoveRecord, float MaxWalkSlopeCosine, float MaxStepHeight);

	/** Used to change a movement to be along a ramp's surface, typically to prevent slow movement when running up/down a ramp */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API FVector ComputeDeflectedMoveOntoRamp(const FVector& OrigMoveDelta, const FVector& UpDirection, const FHitResult& RampHitResult, float MaxWalkSlopeCosine, const bool bHitFromLineTrace);

	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API bool CanStepUpOnHitSurface(const FHitResult& Hit);

public:	
	// Threadsafe movement queries

	/** Tests a potential movement along a walkable surface. 
	 * Modifies InOutHit with final movement hit data
     * Appends to InOutMoveRecord with any movement substeps
	 * Returns the percent of time applied, with 0.0 meaning no movement would occur. 
	 */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API float TestGroundedMoveAlongHitSurface(const FMovingComponentSet& MovingComps, const FVector& OriginalMoveDelta, const FVector& LocationAtHit, const FQuat& TargetRotation, bool bHandleImpact, float MaxStepHeight, float MaxWalkSlopeCosine, UPARAM(ref) FHitResult& InOutHit, UPARAM(ref) FMovementRecord& InOutMoveRecord);

	/** Tests potential movement of a component up/down to adjust to a walkable floor. Intended for use while performing ground movement. 
	 * Modifies InOutCurrentFloor to account for adjustments, if any was made
     * Appends to InOutMoveRecord with any movement substeps
	 * Returns new location, including any adjustment if it was made
	 */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API FVector TestMoveToAdjustToFloor(const FMovingComponentSet& MovingComps, const FVector& Location, const FQuat& Rotation, float MaxWalkSlopeCosine, UPARAM(ref) FFloorCheckResult& InOutCurrentFloor, UPARAM(ref) FMovementRecord& InOutMoveRecord);

	/** Tests potential vertical movement of a component to float a bit above a walkable floor. Intended for use while moving along ground surfaces.
	 * Modifies InOutCurrentFloor to account for adjustments, if any was made
     * Appends to InOutMoveRecord with any movement substeps
	 * Returns new location, including any adjustment if it was made
	 */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API FVector TestMoveToKeepMinHeightAboveFloor(const FMovingComponentSet& MovingComps, const FVector& Location, const FQuat& Rotation, float MaxWalkSlopeCosine, UPARAM(ref) FFloorCheckResult& InOutCurrentFloor, UPARAM(ref) FMovementRecord& InOutMoveRecord);

	/** Tests potential up-and-over movement of a component to overcome a blocking obstacle hit, such as moving up steps or slope.
	 * Sets OutFloorTestResult with final floor info (optional)
	 * Sets OutFinalLocation with final component location
     * Appends to InOutMoveRecord with any movement substeps
	 * Returns the percent of time applied, with 0.0 meaning no movement would occur.
	 */
	static UE_API float TestMoveToStepOver(const FMovingComponentSet& MovingComps, const FVector& GravDir, float MaxStepHeight, const FFloorCheckSettings& FloorCheckSettings, const FVector& MoveDelta, const FQuat& Rotation, const FHitResult& MoveHitResult, const FFloorCheckResult& CurrentFloor, bool bIsFalling, FOptionalFloorCheckResult* OutFloorTestResult, FVector& OutFinalLocation, FMovementRecord& InOutMoveRecord);

	UE_DEPRECATED(5.8, "Use the overload taking FFloorCheckSettings instead.")
	static UE_API float TestMoveToStepOver(const FMovingComponentSet& MovingComps, const FVector& GravDir, float MaxStepHeight, float MaxWalkSlopeCosine, bool bUseFlatBaseForFloorChecks, float FloorSweepDistance, const FVector& MoveDelta, const FQuat& Rotation, const FHitResult& MoveHitResult, const FFloorCheckResult& CurrentFloor, bool bIsFalling, FOptionalFloorCheckResult* OutFloorTestResult, FVector& OutFinalLocation, FMovementRecord& InOutMoveRecord, float PerchRadiusThreshold = 0.f);


};

#undef UE_API
