// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "ModularMovement.generated.h"

#define UE_API MOVER_API

struct FMoverTickStartData;
struct FMoverTimeStep;
struct FProposedMove;
class UMoverBlackboard;



UINTERFACE(BlueprintType, MinimalAPI)
class UTurnGeneratorInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * TurnGeneratorInterface: API for generating an in-place orientation change, based on a starting state and a target orientation
 */
class ITurnGeneratorInterface : public IInterface
{
	GENERATED_BODY()

public:
	// TODO: consider making GetTurn take a params struct instead of a fixed arg list

	/** Returns an additive angular velocity (degrees/second) based on the starting state and timestep. The vector points in the direction of the rotation axis */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category=Mover)
	MOVER_API FVector GetTurn(FRotator TargetOrientation, const FMoverTickStartData& FullStartState, const FMoverDefaultSyncState& MoverState, const FMoverTimeStep& TimeStep, const FProposedMove& ProposedMove, UMoverBlackboard* SimBlackboard);
};


/**
 * Modular mechanism for turning a Mover actor in place using linear angular velocity.
 */
UCLASS(MinimalAPI, BlueprintType, EditInlineNew)
class ULinearTurnGenerator : public UObject, public ITurnGeneratorInterface
{
	GENERATED_BODY()

public:

	/** Maximum angular velocity of heading changes (degrees per second). AKA Yaw, AKA Z. Negative numbers will cause rotation to snap instantly to desired orientation. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Linear Turning", meta = (ClampMin = "-1", UIMin = "0", ForceUnits = "deg/s"))
	float HeadingRate = 500.f;

	/** Maximum angular velocity of pitch changes (degrees per second). Negative numbers will cause rotation to snap instantly to desired orientation. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Linear Turning", meta = (ClampMin = "-1", UIMin = "0", ForceUnits = "deg/s"))
	float PitchRate = -1.f;

	/** Maximum angular velocity of roll changes (degrees per second). Negative numbers will cause rotation to snap instantly to desired orientation. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Linear Turning", meta = (ClampMin = "-1", UIMin = "0", ForceUnits = "deg/s"))
	float RollRate = -1.f;


	UE_API virtual FVector GetTurn_Implementation(FRotator TargetOrientation, const FMoverTickStartData& FullStartState, const FMoverDefaultSyncState& MoverState, const FMoverTimeStep& TimeStep, const FProposedMove& ProposedMove, UMoverBlackboard* SimBlackboard) override;
};


/**
 * Modular mechanism for turning a Mover actor in place using an exact damped spring
 */
UCLASS(MinimalAPI, BlueprintType, EditInlineNew)
class UExactDampedTurnGenerator : public UObject, public ITurnGeneratorInterface
{
	GENERATED_BODY()

public:

	/** Time required to reach halfway to the target orientation (smaller = quicker) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Exact Damped Turning", meta = (ClampMin = "0.01", UIMin = "0", ForceUnits = "seconds"))
	float HalfLifeSeconds = 0.15f;

	UE_API virtual FVector GetTurn_Implementation(FRotator TargetOrientation, const FMoverTickStartData& FullStartState, const FMoverDefaultSyncState& MoverState, const FMoverTimeStep& TimeStep, const FProposedMove& ProposedMove, UMoverBlackboard* SimBlackboard) override;
};


/**
 * Base class for blueprint-implemented turn generators. This is necessary due to the lack of support for specifying 
 * EditInlineNew on a BP class, so it has to inherit the flag from a native parent.
 */
UCLASS(MinimalAPI, Blueprintable, EditInlineNew, Abstract)
class UBlueprintableTurnGenerator : public UObject, public ITurnGeneratorInterface
{
	GENERATED_BODY()
};

#undef UE_API
