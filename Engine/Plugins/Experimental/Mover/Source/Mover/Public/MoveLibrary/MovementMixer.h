// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MovementMixer.generated.h"

#define UE_API MOVER_API

struct FLayeredMoveBase;
struct FLayeredMoveInstance;
struct FProposedMove;

/**
 * Class in charge of Mixing various moves when evaluating/combining moves. The mixer used can be set on the MoverComponent itself.
 */
UCLASS(MinimalAPI, BlueprintType)
class UMovementMixer : public UObject
{
	GENERATED_BODY()

public:
	UE_API UMovementMixer();
	
	/** In charge of mixing Layered Move proposed moves into a cumulative proposed move based on mix mode and priority.*/
	UE_API virtual void MixLayeredMove(const FLayeredMoveBase& ActiveMove, const FProposedMove& MoveStep, FProposedMove& OutCumulativeMove);

	/** In charge of mixing Layered Move proposed moves into a cumulative proposed move based on mix mode and priority.*/
	UE_API virtual void MixLayeredMove(const FLayeredMoveInstance& ActiveMove, const FProposedMove& MoveStep, FProposedMove& OutCumulativeMove);

	/** In charge of mixing proposed moves together. Is similar to MixLayeredMove but is only responsible for mixing proposed moves instead of layered moves. */
	UE_API virtual void MixProposedMoves(const FProposedMove& MoveToMix, FVector UpDirection, FProposedMove& OutCumulativeMove);

	/** Resets all state used for mixing. Should be called before or after finished mixing moves. */
	UE_API virtual void ResetMixerState();
	
protected:
	// Stores the current highest priority we've hit during this round of mixing. Will get reset. Note: Currently only used for mixing layered moves
	uint8 CurrentHighestPriority;

	// Earliest start time of the layered move with highest priority. Used to help break ties of moves with same priority. Note: Currently only used for mixing layered moves
	double CurrentLayeredMoveStartTime;
	
	/**
	 * Helper function for layered move mixing to check priority and start time if priority is the same.
	 * Returns true if this layered move should take priority given current HighestPriority and CurrentLayeredMoveStartTimeMs
	 */
	static UE_API bool CheckPriority(const FLayeredMoveBase* LayeredMove, uint8& InOutHighestPriority, double& InOutCurrentLayeredMoveStartTimeMs);
	static UE_API bool CheckPriority(uint8 LayeredMovePriority, double LayeredMoveStartTimeMs, uint8& InOutHighestPriority, double& InOutCurrentLayeredMoveStartTimeMs);
};

#undef UE_API
