// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "MovementRecord.generated.h"

#define UE_API MOVER_API


/** A part of movement accounting, representing a single piece of a move operation, such as a slide, floor adjustment, etc. */
USTRUCT(BlueprintType)
struct FMovementSubstep
{
	GENERATED_BODY()

	FMovementSubstep() {}
	FMovementSubstep(FName InMoveName, FVector InMoveDelta, bool InIsRelevant = true)
		: MoveName(InMoveName), MoveDelta(InMoveDelta), bIsRelevant(InIsRelevant) {}

	FName MoveName = NAME_None;
	FVector	MoveDelta = FVector::ZeroVector;
	// TODO: Replace this simple flag with a bit mask of different aspects of relevancy (i.e. the move delta is relevant BUT not to vertical velocity)
	bool	bIsRelevant = false;	// Whether or not this movement is relevant to true actor state (velocity, accel, etc.)
};


/** Accounting record of a move as it is processed.
* Moves are composed of substeps, and these can be marked to indicate how they influence the final collapsed move.
* Relevancy means the substep (or part thereof) is contributing to the reflected movement state. Example: a character moves forward across a slightly
* irregular sidewalk, and then is adjusted downward to keep it sticking closely to the sidewalk. The forward move is a relevant substep, while the vertical heigh adjustment is not.
*/
USTRUCT(BlueprintType)
struct FMovementRecord
{
	GENERATED_BODY()

	UE_API void Reset();

	UE_API void Append(FMovementSubstep Substep);

	// TODO: consider changing this to only allow locking as irrelevant
	// TODO: this relevancy locking is hacky, could be replaced by a scoping-style mechanism to allow higher-level functions to override a lower-level function's default relevancy
	void LockRelevancy(bool bLockValue) { bIsRelevancyLocked = true; bRelevancyLockValue = bLockValue; }
	void UnlockRelevancy() { bIsRelevancyLocked = false; }

	void SetDeltaSeconds(float DeltaSeconds) { TotalDeltaSeconds = DeltaSeconds; }

	const TArray<FMovementSubstep>& GetSubsteps() const { return Substeps; }
	const FVector& GetTotalMoveDelta() const { return TotalMoveDelta; }
	const FVector& GetRelevantMoveDelta() const { return RelevantMoveDelta; }
	FVector GetRelevantVelocity() const { return ((TotalDeltaSeconds > 0.f) ? (RelevantMoveDelta / TotalDeltaSeconds) : FVector::ZeroVector); }

	UE_API FString ToString() const;

private:
	FVector TotalMoveDelta = FVector::ZeroVector;
	FVector RelevantMoveDelta = FVector::ZeroVector;	// This is the portion of movement that counts as relevant to actor velocity
	float	TotalDeltaSeconds = 0.f;

	bool bIsRelevancyLocked = false;
	bool bRelevancyLockValue = false;

	// TODO: Perf, need to reserve a reasonable number or possibly refactor this to be some kind of pool
	TArray<FMovementSubstep> Substeps;
};

#undef UE_API
