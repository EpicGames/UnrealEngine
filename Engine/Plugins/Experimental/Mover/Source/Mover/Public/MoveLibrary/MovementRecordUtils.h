// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "MovementRecordUtils.generated.h"

#define UE_API MOVER_API

struct FMovementRecord;

/**
 * MovementRecordUtils: a collection of stateless static BP-accessible functions for movement record related operations
 */
UCLASS(MinimalAPI)
class UMovementRecordUtils : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Sets the delta time these moves took place over */
	UFUNCTION(BlueprintCallable, Category = Mover, DisplayName="Set Delta Seconds")
	static UE_API void K2_SetDeltaSeconds(UPARAM(Ref) FMovementRecord& OutMovementRecord, float DeltaSeconds);

	/** Get all the move delta applied to the actor/record */
	UFUNCTION(BlueprintPure, Category = Mover, DisplayName="Get Total Move Delta")
	static UE_API const FVector& K2_GetTotalMoveDelta(const FMovementRecord& MovementRecord);

	/** Get relevant move delta applied to the actor/record */
	UFUNCTION(BlueprintPure, Category = Mover, DisplayName="Get Relevant Move Delta")
	static UE_API const FVector& K2_GetRelevantMoveDelta(const FMovementRecord& MovementRecord);

	/** Get relevant velocity applied to the actor/record */
	UFUNCTION(BlueprintPure, Category = Mover, DisplayName="Get Relevant Velocity")
	static UE_API FVector K2_GetRelevantVelocity(const FMovementRecord& MovementRecord);
};

#undef UE_API
