// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "MoverSimulationTypes.h"
#include "MoverDataModelTypes.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/Blueprint.h" // For gathering CDO info from a BP
#include "Engine/SCS_Node.h" // For gathering CDO info from a BP
#include "Engine/SimpleConstructionScript.h" // For gathering CDO info from a BP

#include "MovementUtils.generated.h"

#define UE_API MOVER_API

struct FMovementRecord;
class UMoverComponent;

namespace UE::MoverUtils
{
	MOVER_API extern const double SMALL_MOVE_DISTANCE;
	MOVER_API extern const double VERTICAL_SLOPE_NORMAL_MAX_DOT;
}

/** Encapsulates detailed trajectory sample info, from a move that has already occurred or one projected into the future */
USTRUCT(BlueprintType)
struct FTrajectorySampleInfo
{
	GENERATED_BODY()

	// Position and orientation (world space)
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FTransform Transform;

	// Velocity at the time of this sample (world space, units/sec)
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FVector LinearVelocity = FVector::ZeroVector;

	// Acceleration at the time of this sample (world space, units/sec^2)
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FVector InstantaneousAcceleration = FVector::ZeroVector;

	// Rotational velocity (world space, degrees/sec)
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FRotator AngularVelocity = FRotator::ZeroRotator;

	// Time stamp of this sample, in server simulation time
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	double SimTimeMs = 0.0f;
};

// Input parameters for compute velocity function
USTRUCT(BlueprintType)
struct FComputeVelocityParams
{
	GENERATED_BODY()
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float DeltaSeconds = 0.f;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FVector InitialVelocity = FVector::ZeroVector;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FVector MoveDirectionIntent = FVector::ZeroVector;

	// AuxState variables
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float MaxSpeed = 0.f;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float TurningBoost = 0.f;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float Friction = 0.f;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float Deceleration = 0.f;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float Acceleration = 0.f;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FVector MoveInput = FVector::ZeroVector;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	EMoveInputType MoveInputType = EMoveInputType::DirectionalIntent;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	bool bUseAccelerationForVelocityMove = true;
};

// Input parameters for ComputeCombinedVelocity()
USTRUCT(BlueprintType)
struct FComputeCombinedVelocityParams
{
	GENERATED_BODY()
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float DeltaSeconds = 0.f;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FVector InitialVelocity = FVector::ZeroVector;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FVector MoveDirectionIntent = FVector::ZeroVector;

	// AuxState variables
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float MaxSpeed = 0.f;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float TurningBoost = 0.f;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float Friction = 0.f;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float Deceleration = 0.f;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float Acceleration = 0.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FVector ExternalAcceleration = FVector::ZeroVector;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float OverallMaxSpeed = 0.f;
};

/**
 * MovementUtils: a collection of stateless static BP-accessible functions for a variety of movement-related operations
 */
UCLASS(MinimalAPI)
class UMovementUtils : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	// JAH TODO: Ideally, none of these functions should deal with simulation input/state types. Rework them to take only the core types they actually need
	// JAH TODO: Make sure all 'out' parameters are last in the param list and marked as "Out"
	// JAH TODO: separate out the public-facing ones from the internally-used ones and make all public-facing ones BlueprintCallable

	// Gets CDO component type - useful for getting original values
	template <class ComponentType>
	static const ComponentType* GetOriginalComponentType(const AActor* MoverCompOwner);
	
	/** Checks whether a given velocity is exceeding a maximum speed, with some leeway to account for numeric imprecision */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API bool IsExceedingMaxSpeed(const FVector& Velocity, float InMaxSpeed);

	/** Returns new ground-based velocity (worldspace) based on previous state, movement intent (worldspace), and movement settings */
	UFUNCTION(BlueprintCallable, Category=Mover)
    static UE_API FVector ComputeVelocity(const FComputeVelocityParams& InParams);

	/** Returns new velocity based on previous state, movement intent, movement mode's influence and movement settings */
	UFUNCTION(BlueprintCallable, Category=Mover)
    static UE_API FVector ComputeCombinedVelocity(const FComputeCombinedVelocityParams& InParams);

	/** Returns velocity (units per second) contributed by gravitational acceleration over a given time */
	UFUNCTION(BlueprintCallable, Category=Mover)
	static FVector ComputeVelocityFromGravity(const FVector& GravityAccel, float DeltaSeconds) { return GravityAccel * DeltaSeconds; }

	/** Returns the up direction deduced from gravity acceleration, but defaults to mover constants up direction if zero */
	UFUNCTION(BlueprintCallable, Category=Mover)
	static UE_API FVector DeduceUpDirectionFromGravity(const FVector& GravityAcceleration);

	/** Checks whether a given velocity is strong enough to lift off against gravity */
	UFUNCTION(BlueprintCallable, Category=Mover)
	static UE_API bool CanEscapeGravity(const FVector& PriorVelocity, const FVector& NewVelocity, const FVector& GravityAccel, float DeltaSeconds);

	/** Ensures input Vector (typically a velocity, acceleration, or move delta) is limited to a movement plane. 
	* @param bMaintainMagnitude - if true, vector will be scaled after projection in an attempt to keep magnitude the same 
	*/
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API FVector ConstrainToPlane(const FVector& Vector, const FPlane& MovementPlane, bool bMaintainMagnitude=true);

	/** Converts intended orientation into orientation rotated by out current gravity */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API FRotator ApplyGravityToOrientationIntent(const FRotator& IntendedOrientation, const FQuat& WorldToGravity, bool bStayVertical = true);

	/** Project a vector onto the floor defined by the gravity direction. */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static FVector ProjectToGravityFloor(const FVector& Vector, const FVector& UpDirection) { return FVector::VectorPlaneProject(Vector, -UpDirection); }

	/** Returns the component of the vector in the gravity-space vertical direction.  */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static FVector GetGravityVerticalComponent(const FVector& Vector, const FVector& UpDirection) { return Vector.Dot(-UpDirection) * -UpDirection; }

	/** Set the vertical component of the vector to the given value in the gravity-space vertical direction. */
	static void SetGravityVerticalComponent(FVector& Vector, const FVector::FReal VerticalValue, const FVector& UpDirection) { Vector = ProjectToGravityFloor(Vector, UpDirection) - VerticalValue * -UpDirection; }
	
	// Surface sliding

	/** Returns an alternative move delta to slide along a surface, based on parameters describing a blocked attempted move */
	UFUNCTION(BlueprintCallable, Category=Mover)
	static UE_API FVector ComputeSlideDelta(const FMovingComponentSet& MovingComps, const FVector& Delta, const float PctOfDeltaToMove, const FVector& Normal, const FHitResult& Hit);

	/** Returns an alternative move delta when we are in contact with 2 surfaces */
	static UE_API FVector ComputeTwoWallAdjustedDelta(const FMovingComponentSet& MovingComps, const FVector& MoveDelta, const FHitResult& Hit, const FVector& OldHitNormal);

	/**
	 * Attempts to move a component along a surface. Returns the percent of time applied, with 0.0 meaning no movement occurred.
	 * Note: This function takes a movement record that collects moves applied to the actor see @FMovementRecord
	 */
	UFUNCTION(BlueprintCallable, Category=Mover, meta=(DisplayName="Try Move To Slide Along Surface (Movement Record)"))
	static UE_API float TryMoveToSlideAlongSurface(const FMovingComponentSet& MovingComps, const FVector& Delta, float PctOfDeltaToMove, const FQuat Rotation, const FVector& Normal, UPARAM(Ref) FHitResult& Hit, bool bHandleImpact, UPARAM(Ref) FMovementRecord& MoveRecord);

	/**
	 * Attempts to move a component along a surface. Returns the percent of time applied, with 0.0 meaning no movement occurred.
	 * Note: This function doesn't update a movement record so velocity should be gathered/set using a different method
	 */
	UFUNCTION(BlueprintCallable, Category=Mover, meta=(DisplayName="Try Move To Slide Along Surface (No Movement Record)"))
	static UE_API float TryMoveToSlideAlongSurfaceNoMovementRecord(const FMovingComponentSet& MovingComps, const FVector& Delta, float PctOfDeltaToMove, const FQuat Rotation, const FVector& Normal, UPARAM(Ref) FHitResult& Hit, bool bHandleImpact);

	// Component movement

	/**
	 * Attempts to move a component and resolve any penetration issues with the proposed move Delta.
	 * This function handles blocking moves and slides along the surface or stops on impact. It uses @TryMoveToSlideAlongSurface to slide
	 * along the surface on hit, so if other behavior is needed for sliding (like Falling based sliding) consider using
	 * @TrySafeMoveUpdatedComponent and a slide function or setting @bSlideAlongSurface to false and then using a separate sliding function.
	 * Note: This function takes a movement record that collects moves applied to the actor see @FMovementRecord.
	 * @param MovingComps Encapsulates components involved in movement
	 * @param Delta The desired location change in world space
	 * @param NewRotation The new desired rotation in world space
	 * @param bSweep Whether we sweep to the destination location
	 * @param OutHit Optional output describing the blocking hit that stopped the move, if any
	 * @param Teleport Whether we teleport the physics state (if physics collision is enabled for this object)
	 * @param MoveRecord Move record to add moves to
	 * @param bSlideAlongSurface If true the actor slides along a blocking surface. If false the actor will stop if the move was blocked
	 * @return Returns the percent of the move applied 0 to 1, where 1 represents the whole move being applied
	 */
	UFUNCTION(BlueprintCallable, Category=Mover, meta=(DisplayName="Try Safe Move And Slide (Movement Record)"))
	static UE_API float TrySafeMoveAndSlideUpdatedComponent(const FMovingComponentSet& MovingComps, const FVector& Delta, const FQuat& NewRotation, bool bSweep, UPARAM(Ref) FHitResult& OutHit, ETeleportType Teleport, UPARAM(Ref) FMovementRecord& MoveRecord, bool bSlideAlongSurface = true);

	/**
 	 * Attempts to move a component and resolve any penetration issues with the proposed move Delta.
 	 * This function handles blocking moves and slides along the surface or stops on impact. It uses @TryMoveToSlideAlongSurface to slide
 	 * along the surface on hit, so if other behavior is needed for sliding (like Falling based sliding) consider using
 	 * @TrySafeMoveUpdatedComponent and a slide function or setting @bSlideAlongSurface to false and then using a separate sliding function.
 	 * Note: This function doesn't update a movement record so velocity should be gathered/set using a different method
 	 * @param MovingComps Encapsulates components involved in movement
 	 * @param Delta The desired location change in world space
 	 * @param NewRotation The new desired rotation in world space
 	 * @param bSweep Whether we sweep to the destination location
 	 * @param OutHit Optional output describing the blocking hit that stopped the move, if any
 	 * @param Teleport Whether we teleport the physics state (if physics collision is enabled for this object)
 	 * @param bSlideAlongSurface If true the actor slides along a blocking surface. If false the actor will stop if the move was blocked
 	 * @return Returns the percent of the move applied 0 to 1, where 1 represents the whole move being applied
 	 */
	UFUNCTION(BlueprintCallable, Category=Mover, meta=(DisplayName="Try Safe Move And Slide (No Movement Record)"))
	static UE_API float TrySafeMoveAndSlideUpdatedComponentNoMovementRecord(const FMovingComponentSet& MovingComps, const FVector& Delta, const FQuat& NewRotation, bool bSweep, UPARAM(Ref) FHitResult& OutHit, ETeleportType Teleport, bool bSlideAlongSurface = true);
	
	/**
	 * Attempts to move a component and resolve any penetration issues with the proposed move Delta
	 * Note: This function takes a movement record that collects moves applied to the actor see @FMovementRecord
	 */
	UFUNCTION(BlueprintCallable, Category=Mover, meta=(DisplayName="Try Safe Move Updated Component (Movement Record)"))
	static UE_API bool TrySafeMoveUpdatedComponent(const FMovingComponentSet& MovingComps, const FVector& Delta, const FQuat& NewRotation, bool bSweep, UPARAM(Ref) FHitResult& OutHit, ETeleportType Teleport, UPARAM(Ref) FMovementRecord& MoveRecord);
	
	/**
 	 * Attempts to move a component and resolve any penetration issues with the proposed move Delta
 	 * Note: This function doesn't update a movement record so velocity should be gathered/set using a different method
 	 */
	UFUNCTION(BlueprintCallable, Category=Mover, meta=(DisplayName="Try Safe Move Updated Component (No Movement Record)"))
	static UE_API bool TrySafeMoveUpdatedComponentNoMovementRecord(const FMovingComponentSet& MovingComps, const FVector& Delta, const FQuat& NewRotation, bool bSweep, UPARAM(Ref) FHitResult& OutHit, ETeleportType Teleport);

	/** Returns a movement step that should get the subject of the hit result out of an initial penetration condition */
	static UE_API FVector ComputePenetrationAdjustment(const FHitResult& Hit);
	
	/** Attempts to move out of a situation where the component is stuck in geometry, using a suggested adjustment to start. */
	static UE_API bool TryMoveToResolvePenetration(const FMovingComponentSet& MovingComps, EMoveComponentFlags MoveComponentFlags, const FVector& ProposedAdjustment, const FHitResult& Hit, const FQuat& NewRotationQuat, FMovementRecord& MoveRecord);
	static UE_API void InitCollisionParams(const UPrimitiveComponent* UpdatedPrimitive, FCollisionQueryParams& OutParams, FCollisionResponseParams& OutResponseParam);
	static UE_API bool OverlapTest(const USceneComponent* UpdatedComponent, const UPrimitiveComponent* UpdatedPrimitive, const FVector& Location, const FQuat& RotationQuat, const ECollisionChannel CollisionChannel, const FCollisionShape& CollisionShape, const AActor* IgnoreActor);

	/** Computes velocity based on start and end positions over time */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API FVector ComputeVelocityFromPositions(const FVector& FromPos, const FVector& ToPos, float DeltaSeconds);

	/** Computes the angular velocity needed to change from one orientation to another within a time frame. Use the optional TurningRateLimit to clamp to a maximum step (negative=unlimited). */
	UE_DEPRECATED(5.7, "Rotator is no longer used for angular velocity")
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API FRotator ComputeAngularVelocity(const FRotator& From, const FRotator& To, const FQuat& WorldToGravity, float DeltaSeconds, float TurningRateLimit=-1.f);
	
	/** Computes the angular velocity needed to change from one orientation to another within a time frame. Use the optional TurningRateLimit to clamp to a maximum step in degrees per second (negative=unlimited). */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API FVector ComputeAngularVelocityDegrees(const FRotator& From, const FRotator& To, float DeltaSeconds, float TurningRateLimit = -1.0f);

	/** Computes the directional movement intent based on input vector and associated type */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API FVector ComputeDirectionIntent(const FVector& MoveInput, EMoveInputType MoveInputType, float MaxSpeed);
	
	/** Returns whether this rotator representing angular velocity has any non-zero values. This function exists due to FRotator::IsZero queries performing undesired wrapping and clamping. */
	UE_DEPRECATED(5.7, "Rotator is no longer used for angular velocity")
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API bool IsAngularVelocityZero(const FRotator& AngularVelocity);

	/** Applies a desired angular velocity to a starting rotator over time. Returns the new orientation rotator. */
	UFUNCTION(BlueprintPure, Category = Mover)
	static UE_API FRotator ApplyAngularVelocityToRotator(const FRotator& StartingOrient, const FVector& AngularVelocityDegrees, float DeltaSeconds);

	/** Applies a desired angular velocity to a starting quaternion over time. Returns the new orientation quaternion. */
	UFUNCTION(BlueprintPure, Category = Mover)
	static UE_API FQuat ApplyAngularVelocityToQuat(const FQuat& StartingOrient, const FVector& AngularVelocityDegrees, float DeltaSeconds);

	/** Applies a desired angular velocity to a starting rotator over time. Returns the new orientation rotator. */
	UFUNCTION(BlueprintCallable, Category = Mover)
	UE_DEPRECATED(5.7, "Rotator is no longer used for angular velocity")
	static UE_API FRotator ApplyAngularVelocity(const FRotator& StartingOrient, const FRotator& AngularVelocity, float DeltaSeconds);

	/**
	 * Try to find an acceptable non-colliding location to place TestActor as close to possible to TestLocation. Expects TestLocation to be a valid location inside the level.
	 * Returns true if a location without blocking collision is found, in which case TestLocation is overwritten with the new clear location.
	 * Returns false if no suitable location could be found, in which case TestLocation is unmodified.
	 */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API bool FindTeleportSpot(const UMoverComponent* MoverComp, UPARAM(Ref) FVector& TestLocation, FRotator TestRotation);

	/** Returns whether MoverComp's Actor would encroach at TestLocation on something that blocks it. */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API bool TestEncroachment(const UMoverComponent* MoverComp, FVector TestLocation, FRotator TestRotation);

	/** 
	 * Returns whether MoverComp's Actor would encroach at TestLocation on something that blocks it.
	 * If blocked, we'll attempt to find an adjustment and set OutProposedAdjustment accordingly. 
	 * OutProposedAdjustment will be zero'd if there's no blockage or an adjustment could not be found.
	 */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API bool TestEncroachmentAndAdjust(const UMoverComponent* MoverComp, FVector TestLocation, FRotator TestRotation, FVector& OutProposedAdjustment);


	// Internal functions - not meant to be called outside of this library
	
	/** Internal function that other move functions use to perform all actual component movement and retrieve results
	 *  Note: This function moves the character directly and should only be used if needed. Consider using something like TrySafeMoveUpdatedComponent.
	 */
	static UE_API bool TryMoveUpdatedComponent_Internal(const FMovingComponentSet& MovingComps, const FVector& Delta, const FQuat& NewRotation, bool bSweep, EMoveComponentFlags MoveComponentFlags, FHitResult* OutHit, ETeleportType Teleport);

	// Internal function for testing whether a Mover actor would encroach at a test location
	static UE_API bool TestEncroachment_Internal(const UWorld* World, const AActor* TestActor, const UPrimitiveComponent* PrimComp, const FTransform& TestWorldTransform, const TArray<AActor*>& IgnoreActors);


	// Internal function for testing whether a Mover actor would encroach at a test location, and computes a proposed adjustment where they won't encroach (if found).
	static UE_API bool TestEncroachmentWithAdjustment_Internal(const UWorld* World, const AActor* TestActor, const UPrimitiveComponent* PrimComp, const FTransform& TestWorldTransform, const TArray<AActor*>& IgnoreActors, OUT FVector& OutProposedAdjustment);

};

template <class ComponentType>
const ComponentType* UMovementUtils::GetOriginalComponentType(const AActor* MoverCompOwner)
{
	const ComponentType* OriginalComponent = nullptr;

	if (MoverCompOwner)
	{
		if (const AActor* OwnerCDO = Cast<AActor>(MoverCompOwner->GetClass()->GetDefaultObject()))
		{
			// Check if native CDO has Capsule component
			OriginalComponent = OwnerCDO->FindComponentByClass<ComponentType>();

			// check if it comes from a BP
			if (!OriginalComponent)
			{
				if (const UBlueprintGeneratedClass* OwnerClassAsBP = Cast<UBlueprintGeneratedClass>(OwnerCDO->GetClass()))
				{
					TArray<const UBlueprintGeneratedClass*> BlueprintClasses;
					UBlueprintGeneratedClass::GetGeneratedClassesHierarchy(OwnerClassAsBP, BlueprintClasses);
					for (const UBlueprintGeneratedClass* BlueprintClass : BlueprintClasses)
					{
						if (BlueprintClass->SimpleConstructionScript)
						{
							// Check Simple construction script
							const TArray<USCS_Node*>& SCSNodes = BlueprintClass->SimpleConstructionScript->GetAllNodes();
							for (USCS_Node* SCSNode : SCSNodes)
							{
								if (SCSNode)
								{
									if (const ComponentType* BPComponent = Cast<ComponentType>(SCSNode->ComponentTemplate))
									{
										OriginalComponent = BPComponent;
										break;
									}
								}
							}
						}
					}
				}
			}
		}
	}

	return OriginalComponent;
}

#undef UE_API
