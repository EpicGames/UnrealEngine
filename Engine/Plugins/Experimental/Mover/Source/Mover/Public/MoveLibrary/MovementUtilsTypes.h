// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CollisionShape.h"
#include "CollisionQueryParams.h"
#include "Components/SceneComponent.h"
#include "MovementUtilsTypes.generated.h"

#define UE_API MOVER_API

class UPrimitiveComponent;
class UMoverComponent;


UENUM()
enum class EMoveMixMode : uint8
{
	/** Velocity (linear and angular) is intended to be added with other sources */
	AdditiveVelocity = 0,
	/** Velocity (linear and angular) should override others */
	OverrideVelocity = 1,
	/** All move parameters should override others */
	OverrideAll      = 2,

	/** All move parameters should override others except linear velocity along the up/down axis. Commonly used for falling. */
	OverrideAllExceptVerticalVelocity = 3,

};


/** Encapsulates info about an intended move that hasn't happened yet */
USTRUCT(BlueprintType)
struct FProposedMove
{
	GENERATED_BODY()

	FProposedMove() : 
		bHasDirIntent(false)
	{}	

	/**
	 * Indicates that we should switch to a particular movement mode before the next simulation step is performed.
	 * Note: If this is set from a layered move the preferred mode will only be set at the beginning of the layered move, not continuously.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FName PreferredMode = NAME_None;	

	// Directional, per-axis magnitude [-1, 1] in world space (length of 1 indicates max speed intent). Only valid if bHasDirIntent is set.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FVector DirectionIntent = FVector::ZeroVector;

	// Units per second, world space, possibly mapped onto walking surface
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FVector LinearVelocity = FVector::ZeroVector;

	// Angular velocity in degrees per second. Direction points along rotation axis.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FVector AngularVelocityDegrees = FVector::ZeroVector;
	
	// Signals whether there was any directional intent specified
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	uint8 bHasDirIntent : 1;
	
	// Determines how this move should resolve with other moves
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	EMoveMixMode MixMode = EMoveMixMode::AdditiveVelocity;
};


/** 
 * Encapsulates components involved in movement. Used by many library functions. 
 * Only a scene component is required for movement, but this is typically a primitive
 * component so we provide a pre-cast ptr for convenience.
 */
USTRUCT(BlueprintType)
struct FMovingComponentSet
{
	GENERATED_BODY()

	FMovingComponentSet() {}
	FMovingComponentSet(USceneComponent* InUpdatedComponent) { SetFrom(InUpdatedComponent); }
	FMovingComponentSet(UMoverComponent* InMoverComponent) { SetFrom(InMoverComponent); }

	UE_API void SetFrom(USceneComponent* InUpdatedComponent);
	UE_API void SetFrom(UMoverComponent* InMoverComponent);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	TWeakObjectPtr<USceneComponent> UpdatedComponent;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	TWeakObjectPtr<UPrimitiveComponent> UpdatedPrimitive;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	TWeakObjectPtr<UMoverComponent> MoverComponent;
};


/**
 * Encapsulates a collision context involved in movement. Useful for altering queries without changing the component.
 */
struct FMoverCollisionParams
{
public: 
	FMoverCollisionParams() {}
	UE_API FMoverCollisionParams(const USceneComponent* SceneComp);

	UE_API void SetFromPrimitiveComponent(const UPrimitiveComponent* PrimitiveComp);

	FCollisionShape Shape;

	ECollisionChannel Channel = ECollisionChannel::ECC_Pawn;

	FCollisionQueryParams QueryParams;

	FCollisionResponseParams ResponseParams;

	EMoveComponentFlags MoveComponentFlags = MOVECOMP_NoFlags;
};

#undef UE_API
