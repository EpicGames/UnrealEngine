// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/TransactionallySafeRWLock.h"
#include "UObject/Object.h"
#include "Templates/SharedPointer.h"
#include "Templates/Casts.h"
#include "MoverBlackboard.generated.h"

#define UE_API MOVER_API



enum class EInvalidationReason : uint8
{
	FullReset,	// All blackboard objects should be invalidated
	Rollback,	// Invalidate any rollback-sensitive objects
};


/** MoverBlackboard: this is a simple generic map that can store any type, used as a way for decoupled systems to 
 *  store calculations or transient state data that isn't necessary to reconstitute the movement simulation. 
 *  It has support for invalidation, which could occur, for example, when a rollback is triggered.
 *  Values submitted are copy-in, copy-out. 
 *  Unlike a traditional blackboard pattern, there is no support for subscribing to changes. 
 * TODO: expand invalidation rules attached to BBObjs, for instance if we wanted some to invalidate upon rollback. Some might expire over time or after a number of simulation frames. Or an item could be tagged with a predicted sim frame #, and become cleared once that frame is finalized/confirmed.
 */
UCLASS(MinimalAPI, BlueprintType)
class UMoverBlackboard : public UObject
{
	GENERATED_BODY()

private:
	class BlackboardObject
	{
		// untyped base
		struct ObjectContainerBase
		{
		};

		// typed container
		template<typename T>
		struct ObjectContainer : ObjectContainerBase
		{
			ObjectContainer(const T& t) : Object(t) {}

			const T& Get() { return Object; }
			T& GetMutable() { return Object; }

		private:
			T Object;
		};


	public:
		template<typename T>
		BlackboardObject(const T& obj) : ContainerPtr(MakeShared<ObjectContainer<T>>(obj)) {}

		template<typename T>
		const T& Get() const
		{
			ObjectContainer<T>* TypedContainer = static_cast<ObjectContainer<T>*>(ContainerPtr.Get());
			return TypedContainer->Get();
		}

		template<typename T>
		T& GetMutable() const
		{
			ObjectContainer<T>* TypedContainer = static_cast<ObjectContainer<T>*>(ContainerPtr.Get());
			return TypedContainer->GetMutable();
		}

	private:
		TSharedPtr<ObjectContainerBase> ContainerPtr;
	};	// end BlackboardObject
 
public:

	/** Attempt to retrieve an object from the blackboard. If found, OutFoundValue will be set. Returns true/false to indicate whether it was found. */
 	template<typename T>
	bool TryGet(FName ObjName, T& OutFoundValue) const
	{
		UE::TReadScopeLock Lock(ObjectsMapLock);

		if (const TUniquePtr<BlackboardObject>* ExistingObject = ObjectsByName.Find(ObjName))
		{
			OutFoundValue = ExistingObject->Get()->Get<T>();
			return true;
		}

		return false;
	}

	// TODO: make GetOrAdds. One that takes a value, and one that takes a lambda that can generate the new value.
	/*
	template<typename T>
	bool GetOrAdd(FName ObjName, some kinda lambda that returns T, T& OutFoundValue)
	{
		if (const TUniquePtr<BlackboardObject>* ExistingObject = ObjectsByName.Find(ObjName))
		{
			OutFoundValue = ExistingObject->Get();
			return true;
		}

		RunTheExec lambda and store it by the key, then return that value
		return false;
	}
	template<typename T>
	T GetOrAdd(FName ObjName, T ObjToAdd)
	{
		return ObjectsByName.FindOrAdd(ObjName, ObjToAdd);
	}
	*/

	/** Returns true/false to indicate if an object is stored with that name */
	bool Contains(FName ObjName) const
	{
		UE::TReadScopeLock Lock(ObjectsMapLock);
		return ObjectsByName.Contains(ObjName);
	}

	/** Store object by a named key, overwriting any existing object */
	template<typename T>
	void Set(FName ObjName, T Obj)
	{
		UE::TWriteScopeLock Lock(ObjectsMapLock);
		ObjectsByName.Emplace(ObjName, MakeUnique<BlackboardObject>(Obj));
	}

	/** Invalidate an object by name */
	UE_API void Invalidate(FName ObjName);

	/** Invalidate all objects that can be affected by a particular circumstance (such as a rollback) */
	UE_API void Invalidate(EInvalidationReason Reason);
	
	/** Invalidate all objects */
	void InvalidateAll() { Invalidate(EInvalidationReason::FullReset); }

	// UObject interface
	UE_API virtual void BeginDestroy() override;
	// End UObject interface

private:
	mutable FTransactionallySafeRWLock ObjectsMapLock;		// used internally when reading/writing to ObjectsByName map
	TMap<FName, TUniquePtr<BlackboardObject>> ObjectsByName;
};

#undef UE_API
