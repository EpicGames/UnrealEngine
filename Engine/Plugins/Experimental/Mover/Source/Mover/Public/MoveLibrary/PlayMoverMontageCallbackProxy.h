// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "Animation/AnimInstance.h"
#include "PlayMontageCallbackProxy.h"
#include "MoveLibrary/MovementUtilsTypes.h"
#include "PlayMoverMontageCallbackProxy.generated.h"

#define UE_API MOVER_API

class UMoverComponent;
class UAnimMontage;
class USkeletalMeshComponent;

// Runtime object used as a proxy to an async blueprint task node that runs animation montages on Mover actors. This leverages 
// parent UPlayMontageCallbackProxy to perform animation, while adding an accompanying layered move to handle any root motion 
// from the montage.
UCLASS(MinimalAPI)
class UPlayMoverMontageCallbackProxy : public UPlayMontageCallbackProxy
{
	GENERATED_UCLASS_BODY()

	// Called to perform the query internally
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true"))
	static UE_API UPlayMoverMontageCallbackProxy* CreateProxyObjectForPlayMoverMontage(
		UMoverComponent* InMoverComponent,
		UAnimMontage* MontageToPlay,
		float PlayRate = 1.f,
		float StartingPosition = 0.f,
		FName StartingSection = NAME_None,
		bool bShouldStopAllMontages = true,
		float BlendOutOverride = -1.f);

public:
	//~ Begin UObject Interface
	UE_API virtual void BeginDestroy() override;
	//~ End UObject Interface

protected:
	UE_API bool PlayMoverMontage(
		UMoverComponent* InMoverComponent,
		USkeletalMeshComponent* InSkeletalMeshComponent,
		UAnimMontage* MontageToPlay,
		float PlayRate = 1.f,
		float StartingPosition = 0.f,
		FName StartingSection = NAME_None,
		bool bShouldStopAllMontages = true,
		float BlendOutOverride = -1.f,
		EMoveMixMode MixMode = EMoveMixMode::OverrideAll);

	UFUNCTION()
	UE_API void OnMoverMontageEnded(FName IgnoredNotifyName);

	UE_API void UnbindMontageDelegates();
};

#undef UE_API
