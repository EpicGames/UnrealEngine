// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RollbackCircularBuffer.h"
#include "MoverLog.h"
#include "UObject/Interface.h"
#include "UObject/Object.h"
#include "Templates/SharedPointer.h"
#include "Templates/Casts.h"
#include "RollbackBlackboard.generated.h"

struct FMoverTimeStep;

#define UE_API MOVER_API


/** RollbackBlackboard: this is a generic map that stores any type of data for local-only access. It will not be replicated.
 *  It can be used as a way for decoupled systems to store calculations or transient state data that isn't 
 *  necessary to reconstitute the movement simulation from scratch.  
 *  Examples: the normal of the current walkable floor, or time of the last movement mode change
 *  
 * Key Features:
 *  - Rollback support: when movement simulations roll back during a correction, this blackboard also rolls back to previous values.
 *  - Concurrent access: for async simulations, it provides concurrent access support so external systems / scripting can access data while new data is being authored
 *  - Policies: there are a variety of policy options to control buffer sizing, invalidation behavior, and entry persistence.
 * 
 * Notes:
 *  - This is implemented under-the-hood using circular buffers, alleviating the need for mem alloc/frees during use. It's important to create entries with policy settings appropriate for their use pattern.
 *  - We use an "InternalWrapper" class to allow in-simulation objects like movement modes to use the same API, without needing to choose between internal- or external-facing functions. 
 *  - The "InternalWrapper" class will be replaced by a different pattern to govern access levels
 */

 // TODO: 
 // - Implement stronger concurrency controls at time of frame changes, and when new in-sim entries are written
 // - Implement sizing policies that rely on knowing a max number of elements to buffer (the backend may know these details)
 // - Consider using TArrays instead of circular buffers, due to the power-of-2 trick potentially wasting a lot of memory
 // - Expose for Blueprint use

/**
 * Determines how a blackboard's entry buffer is sized
 */
UENUM()
enum class EBlackboardSizingPolicy : uint8
{
	/** Buffer size is set at a given number and and never changes */
	FixedDeclaredSize,

	/** Buffer size matches the backend simulation's history size, determined by ticking rate and history settings.
	 * A simulation running a fixed 30 fps with a 1 second history will need a buffer size of 30. For variable rate simulations, buffer size cam only be estimated. 
	 * NOT YET IMPLEMENTED
	 */
	FixedBackendBufferSize UMETA(Hidden),

	/** Buffer size is minimal, for a single value slot. Useful for cases where rolling back shouldn't change the value, 
	 * or for short-lived values that are only set & read within the same simulation frame.
	 * Example: a statistic tracking the maximum time spent in a particular mode's simulation tick.
	 * Note this will still have 2 elements for asynchronous simulations, to prevent thread contention over the entry.
	 */
	SingleEntry,
};

/**
 * Determines how long a blackboard entry can be read after its value is set. Time is in terms of simulation frames or simulation time.
 */
UENUM()
enum class EBlackboardPersistencePolicy : uint8
{
	/** Any edits to the entry remain valid and readable indefinitely */
	Forever = 0,

	/** Any edits remain valid until the end of the next simulation frame. If an edit occurs during frame N, the entry will be readable for the remainder of frame N and frame N+1. */
	NextFrameOnly UE_DEPRECATED(5.8, "Deprecated in favor of ThroughNextFrame, which has identical behavior") = 1,

	/** Any edits remain valid until the end of the next simulation frame. If an edit occurs during frame N, the entry will be readable for the remainder of frame N and frame N+1. */
	ThroughNextFrame = 2,

	/** Any edits remain valid until the end of the current simulation frame. If an edit occurs during frame N, the entry will be readable for the remainder of frame N, until frame N+1 begins. */
	CurrentFrameOnly = 3,

	/** NOT YET IMPLEMENTED. Any edits remain valid until a specified amount of simulation time has passed. */
	LimitedTime = 4 UMETA(Hidden)
};


/**
 * Determines how a blackboard's buffered entries are treated when a rollback occurs
 */
UENUM()
enum class EBlackboardRollbackPolicy : uint8
{
	/** Entry changes occurring after the rollback time are effectively erased and no longer retrievable. New entries may be written during resimulation. */
	InvalidatedOnRollback,

	/** Current value does not change when a rollback occurs. Typically paired with the SingleEntry sizing policy. */
	IgnoreRollback,

	/** Entries are not invalidated during rollback, and may be read/written during resim. If left alone, they will persist. NOT YET IMPLEMENTED. */
	WritableDuringResimulation UMETA(Hidden),

	/** Entries are not invalidated during rollback, and may be read during resim. But writes cannot be performed during resim. NOT YET IMPLEMENTED. */
	LockedDuringResimulation UMETA(Hidden),
};




/**
 * This is the core rollback blackboard, with API for external access and admin use. 
 * See @FRollbackBlackboardSimWrapper for in-simulation use.
 */ 
UCLASS(MinimalAPI)
class URollbackBlackboard : public UObject
{
	GENERATED_BODY()

public:
	struct EntrySettings
	{
		EBlackboardSizingPolicy      SizingPolicy      = EBlackboardSizingPolicy::FixedDeclaredSize;
		EBlackboardPersistencePolicy PersistencePolicy = EBlackboardPersistencePolicy::Forever;
		EBlackboardRollbackPolicy    RollbackPolicy    = EBlackboardRollbackPolicy::InvalidatedOnRollback;

		uint32                       FixedSize = 0;
		double                       MaxLifetimeSecs = -1.0;
	};

private:

	struct EntryTimeStamp
	{
		EntryTimeStamp() {}
		EntryTimeStamp(double InTimeMs, uint32 InFrame) : TimeMs(InTimeMs), Frame(InFrame) {}

		double TimeMs = -1.0;
		uint32 Frame = 0;

		bool IsValid() const
		{
			return TimeMs >= 0.0;
		}

		void Invalidate()
		{
			TimeMs = -1.0;
			Frame = 0;
		}
	};

	enum EEntryIndexType : uint8
	{
		External,   // entry index for outside systems to read
		Internal,   // entry index for in-simulation system to read/write
		Predictive  // entry index for a predicting system to read/write
	};

	// untyped base, so we can use entries of multiple types in a single-typed container
	struct BlackboardEntryBase
	{
	public:
		BlackboardEntryBase() = delete;
		UE_API BlackboardEntryBase(const EntrySettings& InSettings, uint32 BufferSize, ERollbackBufferFlags BufferFlags);

		virtual ~BlackboardEntryBase()
		{
		}
		
		void OnSimulationFrameEnd()
		{
			//Advances the external index to match
			ExternalIdx = InternalIdx;
		}

		// Note that NewPendingFrame means the frame that will now be re-simulated. So any existing entries with a Frame >= NewPendingFrame can be invalidated.
		UE_API void RollBack(uint32 NewPendingFrame);

		UE_API void InvalidatePredictiveState();
		

		UE_API static uint32 ComputeBufferSize(EBlackboardSizingPolicy InSizingPolicy, uint32 FixedBufferSize = -1);

	protected:
		UE_API bool CanReadEntryAtTime(const EntryTimeStamp& ReaderTimeStamp, EEntryIndexType IndexType) const;
		UE_API bool CanReadEntryAtTime(const EntryTimeStamp& ReaderTimeStamp, const EntryTimeStamp& EntryTimeStamp) const;

		virtual void ResetPredictiveElement() {}

		EntrySettings Settings;
		
		TRollbackCircularBuffer<EntryTimeStamp> Timestamps;

		uint32 ExternalIdx;	// Indexes to the last entry on a committed frame
		uint32 InternalIdx;	// Indexes to where in-progress simulations should read/write (matches ExternalIdx when not mid-simulation, and advances if written during simulation)
	};

	template<typename EntryT>
	struct BlackboardEntry : BlackboardEntryBase
	{
	private:

		BlackboardEntry() = delete;	// disallow use of default constructor

	public:

		// TODO: consider what options need to be specified
		BlackboardEntry(const EntrySettings& InSettings, uint32 InBufferSize, ERollbackBufferFlags InBufferFlags, const EntryT* OptionalInitialObj=nullptr)
			: BlackboardEntryBase(InSettings, InBufferSize, InBufferFlags)
			, EntryBuffer(InBufferSize, InBufferFlags), DefaultValue()
		{
			// JAH TODO: Consider making the initial object required, so we can also initialize where in the buffer we point to
			if (OptionalInitialObj)
			{
				for (uint32 i = 0; i < EntryBuffer.Capacity(); i++)
				{
					EntryBuffer[i] = *OptionalInitialObj;
				}

				DefaultValue = *OptionalInitialObj;
			}
		}

		void SetValueAtTime_External(const EntryT& Value, const EntryTimeStamp& TimeStamp)
		{
			EntryBuffer[ExternalIdx] = Value;
			Timestamps[ExternalIdx] = TimeStamp;
		}

		bool TryGetValueAtTime_External(const EntryTimeStamp& CurrentExternalTime, EntryT& ValueOut) const
		{
			if (CanReadEntryAtTime(CurrentExternalTime, EEntryIndexType::External))
			{
				ValueOut = EntryBuffer[ExternalIdx];
				return true;
			}

			return false;
		}

		// TODO: Consider whether it's worthwhile to have this function... dangerous to let external folks hold onto references to internal storage, but may allow more efficient editing
		const EntryT* TryGetRefAtTime_External(const EntryTimeStamp& CurrentExternalTime) const
		{
			if (CanReadEntryAtTime(CurrentExternalTime, EEntryIndexType::External))
			{ 
				return &EntryBuffer[ExternalIdx];
			}

			return nullptr;
		}

		bool TryGetValueAtTime_Internal(const EntryTimeStamp& InternalTime, EntryT& ValueOut) const
		{
			if (CanReadEntryAtTime(InternalTime, EEntryIndexType::Internal))
			{
				ValueOut = EntryBuffer[InternalIdx];
				return true;
			}

			return false;
		}

		const EntryT* TryGetRefAtTime_Internal(const EntryTimeStamp& InternalTime) const
		{
			if (CanReadEntryAtTime(InternalTime, EEntryIndexType::Internal))
			{
				return &EntryBuffer[InternalIdx];
			}

			return nullptr;
		}

		void SetValueAtTime_Internal(const EntryT& Value, const EntryTimeStamp& TimeStamp)
		{
			InternalIdx = ExternalIdx + 1;	// always writing internal, one slot ahead of external slot

			EntryBuffer[InternalIdx] = Value;
			Timestamps[InternalIdx] = TimeStamp;
		}

		bool TryGetValueAtTime_Predictive(const EntryTimeStamp& PredictiveReaderTime, EntryT& ValueOut) const
		{
			// For predictive reading, we can either be reading the current External entry
			// OR the Predictive entry if it's been written to during this Predictive window.
			const bool bHasPredictiveValue = Timestamps.GetPredictiveElement().IsValid();
			
			if (bHasPredictiveValue)
			{
				if (CanReadEntryAtTime(PredictiveReaderTime, EEntryIndexType::Predictive))
				{
					ValueOut = EntryBuffer.GetPredictiveElement();
					return true;
				}
			}
			else
			{
				// No prediction writes have occurred yet, so we'll try to read the external value
				return TryGetValueAtTime_External(PredictiveReaderTime, ValueOut);
			}

			return false;
		}

		const EntryT* TryGetRefAtTime_Predictive(const EntryTimeStamp& PredictiveReaderTime) const
		{
			// For predictive reading, we can either be reading the current External entry
			// OR the Predictive entry if it's been written to during this Predictive window.
			const bool bHasPredictiveValue = Timestamps.GetPredictiveElement().IsValid();

			if (bHasPredictiveValue)
			{
				if (CanReadEntryAtTime(PredictiveReaderTime, EEntryIndexType::Predictive))
				{
					return &EntryBuffer.GetPredictiveElement();
				}
			}
			else
			{
				// No prediction writes have occurred yet, so we'll try to read the external value
				return TryGetRefAtTime_External(PredictiveReaderTime);
			}

			return nullptr;
		}

		void SetValueAtTime_Predictive(const EntryT& Value, const EntryTimeStamp& TimeStamp)
		{
			// Writes ALWAYS go to the predictive element
			EntryBuffer.GetPredictiveElement() = Value;
			Timestamps.GetPredictiveElement() = TimeStamp;
		}

		virtual void ResetPredictiveElement() override
		{
			EntryBuffer.GetPredictiveElement() = DefaultValue;
		}

	
		TRollbackCircularBuffer<EntryT> EntryBuffer;
		EntryT DefaultValue;


	};	// end BlackboardEntry
	


public:
	template<typename EntryT>
	bool CreateEntry(FName EntryName, const EntrySettings& InSettings)
	{
		if (EntryMap.Contains(EntryName))
		{
			UE_LOGF(LogMover, Verbose, "Skipping attempt to create a new blackboard entry named %ls since it already exists. Policy settings from the first entry will be retained.", *EntryName.ToString());
			return false;
		}

		const uint32 BufferSize = BlackboardEntryBase::ComputeBufferSize(InSettings.SizingPolicy, InSettings.FixedSize);
		EntryMap.Emplace(EntryName, MakeUnique<BlackboardEntry<EntryT>>(InSettings, BufferSize, BufferFlags, nullptr));
		return true;
	}

	bool HasEntry(FName EntryName) const
	{
		return EntryMap.Contains(EntryName);
	}

private:
	/** Store object by a named key, overwriting any existing object */
	template<typename EntryT>
	bool TrySet_External(FName ObjName, const EntryT& Obj)
	{
		// TODO: Verify this isn't happening mid-sim on the sim thread (_Internal should be used instead)

		if (BlackboardEntry<EntryT>* Entry = static_cast<BlackboardEntry<EntryT>*>(FindEntry(ObjName)))
		{
			Entry->SetValueAtTime_External(Obj, CurrentSimTimeStamp);
			return true;
		}

		return false;
	}


	/** Attempt to retrieve an object from the blackboard. If found, OutFoundValue will be set. Returns true/false to indicate whether it was found. */
	template<typename EntryT>
	bool TryGet_External(FName ObjName, EntryT& OutFoundValue) const
	{
		if (BlackboardEntry<EntryT>* Entry = static_cast<BlackboardEntry<EntryT>*>(FindEntry(ObjName)))
		{
			return Entry->TryGetValueAtTime_External(CurrentSimTimeStamp, OutFoundValue);
		}

		return false;
	}

	/** Users may access the reference of the entry, which may be more efficient than copy - out, copy - in.
	 * Although you can get access to the reference, it is not safe to hold references over time. 
	 */
	template<typename EntryT>
	const EntryT* TryGetRef_External(FName ObjName) const
	{
		if (BlackboardEntry<EntryT>* Entry = static_cast<BlackboardEntry<EntryT>*>(FindEntry(ObjName)))
		{
			return Entry->TryGetRefAtTime_External(CurrentSimTimeStamp);
		}

		return nullptr;
	}


	/** Store object of a named key, overwriting any existing object. Should only be called from within the simulation. */
	template<typename EntryT>
	bool TrySet_Internal(FName ObjName, const EntryT& Obj)
	{
		check((bIsSimulationInProgress && (InProgressSimFrameThreadId == FPlatformTLS::GetCurrentThreadId())) || 
			  (bIsRollbackInProgress && (InRollbackThreadId == FPlatformTLS::GetCurrentThreadId()))) ;

		if (BlackboardEntry<EntryT>* Entry = static_cast<BlackboardEntry<EntryT>*>(FindEntry(ObjName)))
		{
			Entry->SetValueAtTime_Internal(Obj, InProgressSimTimeStamp);
			return true;
		}

		return false;
	}

	template<typename EntryT>
	bool TryGet_Internal(FName ObjName, EntryT& OutFoundValue) const
	{
		check((bIsSimulationInProgress && (InProgressSimFrameThreadId == FPlatformTLS::GetCurrentThreadId())) ||
			  (bIsRollbackInProgress && (InRollbackThreadId == FPlatformTLS::GetCurrentThreadId())));

		if (BlackboardEntry<EntryT>* Entry = static_cast<BlackboardEntry<EntryT>*>(FindEntry(ObjName)))
		{
			return Entry->TryGetValueAtTime_Internal(InProgressSimTimeStamp, OutFoundValue);
		}

		return false;
	}

	template<typename EntryT>
	const EntryT* TryGetRef_Internal(FName ObjName) const
	{
		check((bIsSimulationInProgress && (InProgressSimFrameThreadId == FPlatformTLS::GetCurrentThreadId())) ||
			  (bIsRollbackInProgress && (InRollbackThreadId == FPlatformTLS::GetCurrentThreadId())));

		if (BlackboardEntry<EntryT>* Entry = static_cast<BlackboardEntry<EntryT>*>(FindEntry(ObjName)))
		{
			return Entry->TryGetRefAtTime_Internal(InProgressSimTimeStamp);
		}

		return nullptr;
	}

private:
	/** Store object by a named key, overwriting any existing object. Should only be called from within the predicting simulation. */
	template<typename EntryT>
	bool TrySet_Predictive(FName ObjName, const EntryT& Obj)
	{
		check(bIsPredictionInProgress && (InPredictionThreadId == FPlatformTLS::GetCurrentThreadId()));

		if (BlackboardEntry<EntryT>* Entry = static_cast<BlackboardEntry<EntryT>*>(FindEntry(ObjName)))
		{
			Entry->SetValueAtTime_Predictive(Obj, InProgressSimTimeStamp);
			return true;
		}

		return false;
	}

	template<typename EntryT>
	bool TryGet_Predictive(FName ObjName, EntryT& OutFoundValue) const
	{
		check(bIsPredictionInProgress && (InPredictionThreadId == FPlatformTLS::GetCurrentThreadId()));

		if (BlackboardEntry<EntryT>* Entry = static_cast<BlackboardEntry<EntryT>*>(FindEntry(ObjName)))
		{
			return Entry->TryGetValueAtTime_Predictive(InProgressSimTimeStamp, OutFoundValue);
		}

		return false;
	}

	template<typename EntryT>
	const EntryT* TryGetRef_Predictive(FName ObjName) const
	{
		check(bIsPredictionInProgress && (InPredictionThreadId == FPlatformTLS::GetCurrentThreadId()));

		if (BlackboardEntry<EntryT>* Entry = static_cast<BlackboardEntry<EntryT>*>(FindEntry(ObjName)))
		{
			return Entry->TryGetRefAtTime_Predictive(InProgressSimTimeStamp);
		}

		return nullptr;
	}

private:

	BlackboardEntryBase* FindEntry(FName EntryName) const
	{
		if (const TUniquePtr<BlackboardEntryBase>* EntryPtr = EntryMap.Find(EntryName))
		{
			return EntryPtr->Get();
		}

		return nullptr;
	}


public:	// Administration functions for keeping the blackboard in sync with the simulation

	// Begin/EndSimulationFrames: advances true simulation flow, regardless of whether this is resimulation or not. Calls will bookend a simulation frame.
	UE_API void BeginSimulationFrame(const FMoverTimeStep& PendingTimeStep);
	UE_API void EndSimulationFrame();

	// Begin/EndRollback: reverts the blackboard back to an older state. EndRollback will be called before any simulation frames are started (resim or not).
	UE_API void BeginRollback(const FMoverTimeStep& NewBaseTimeStep);	// NewBaseTimeStep represents the current time and the PENDING step that's next to be resimulated
	UE_API void EndRollback();

	// BeginPredictionFrame/EndPrediction: this is for predictive simulation that is temporary and can't rollback. Note there may be multiple calls to BeginPredictionFrame before a single call to EndPrediction.
	UE_API void BeginPredictionFrame(const FMoverTimeStep& PendingTimeStep);	// Called to start a new prediction OR continue a sequential frame
	UE_API void EndPrediction();	// Called at the end of ALL prediction frames

private:
	UE_API bool IsInSimulationThread() const;
	UE_API bool IsInPredictionThread() const;

	ERollbackBufferFlags BufferFlags = ERollbackBufferFlags::Prediction;
	
	TMap<FName, TUniquePtr<BlackboardEntryBase>> EntryMap;


	bool bIsSimulationInProgress = false;
	bool bIsResimulating = false;
	uint32 InProgressSimFrameThreadId = 0;

	bool bIsRollbackInProgress = false;
	uint32 InRollbackThreadId = 0;

	bool bIsPredictionInProgress = false;
	uint32 InPredictionThreadId = 0;

	EntryTimeStamp CurrentSimTimeStamp;	// this is the "committed" simulation time, and will lag behind InProgressSimTimeStamp while in the middle of a simulation frame
	EntryTimeStamp InProgressSimTimeStamp;	// this is the sim time that's actively being used mid-simulation. It is only valid during a simulation step.

	// we expose user API through these wrapper types
	friend struct FRollbackBlackboardSimWrapper;
	friend struct FRollbackBlackboardExternalWrapper;
};


#undef UE_API
