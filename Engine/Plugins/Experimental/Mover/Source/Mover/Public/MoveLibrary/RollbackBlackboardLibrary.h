// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "MoveLibrary/RollbackBlackboard.h"

#include "RollbackBlackboardLibrary.generated.h"

#define UE_API MOVER_API


/**
 * RollbackBlackboardLibrary: a collection of static functions to help working with a rollback blackboard
 */
UCLASS(MinimalAPI)
class URollbackBlackboardLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * An 'event' entry is good for data that is rarely authored each frame and may be useful for some time into the future.
	 * These are commonly used to denote gameplay events that need to be referenced a short time later.
	 * Examples: 
	 *     - recording a jump apex's time & height, to be used upon landing to see how far a character fell
	 *     - denoting a state like "was ducking" at the start of navigating under an obstacle, to be used later to restore the ducking behavior after clearing the obstacle
	 * For MaxHistoryCount, consider how many times this entry might change during your worst-case prediction window, which is typically 1000 milliseconds or less.
	 */ 
	static UE_API URollbackBlackboard::EntrySettings MakeEventEntrySettings(int32 MaxHistoryCount);

	/**
	 * Single-frame entries are good for data that only needs to be communicated between different parts of the 
	 * movement sim during the current frame and has no use beyond that.
	 * Example: information about acceleration used in a mode's GenerateMove that you want to capture and include in the SyncState during SimulationTick
	 */
	static UE_API URollbackBlackboard::EntrySettings MakeSingleFrameEntrySettings();

	/**
	 * A 'rolling' entry is good for data that changes every frame.
	 * Examples:
	 *     - a dynamic homing target location, or motion warping position, that is moving as we relate to it
	 *     - information about animation state from an associated skeletal mesh
	 */
	static UE_API URollbackBlackboard::EntrySettings MakeRollingEntrySettings();

};



#undef UE_API