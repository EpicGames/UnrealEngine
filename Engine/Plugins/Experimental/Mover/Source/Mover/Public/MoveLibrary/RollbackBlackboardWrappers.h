// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoveLibrary/RollbackBlackboard.h"
#include "RollbackBlackboardWrappers.generated.h"


#define UE_API MOVER_API


/**
 * Wrapper class for external parties to access the blackboard. 
 * External parties = users outside of an active movement simulation such as animation and gameplay scripting
 */
USTRUCT(BlueprintType)
struct FRollbackBlackboardExternalWrapper
{
	GENERATED_BODY()

public:
	UE_API FRollbackBlackboardExternalWrapper(URollbackBlackboard& InBlackboard);
	UE_API FRollbackBlackboardExternalWrapper(const FRollbackBlackboardExternalWrapper& Other);
	// Default constructor exists only for reflection / BP support.  Typically constructed with a blackboard or as a copy.
	UE_API FRollbackBlackboardExternalWrapper();

	template<typename EntryT>
	bool CreateEntry(FName EntryName, const URollbackBlackboard::EntrySettings& InSettings)
	{
		if (Blackboard == nullptr)
		{
			return false;
		}

		return Blackboard->CreateEntry<EntryT>(EntryName, InSettings);
	}

	bool HasEntry(FName EntryName) const
	{
		if (Blackboard == nullptr)
		{
			return false;
		}

		return Blackboard->HasEntry(EntryName);
	}

	template<typename EntryT>
	bool TrySet(FName ObjName, const EntryT& Obj)
	{
		if (Blackboard == nullptr)
		{
			return false;
		}

		return Blackboard->TrySet_External<EntryT>(ObjName, Obj);
	}

	template<typename EntryT>
	bool TryGet(FName ObjName, EntryT& OutFoundValue) const
	{
		if (Blackboard == nullptr)
		{
			return false;
		}

		return Blackboard->TryGet_External(ObjName, OutFoundValue);
	}

	template<typename EntryT>
	const EntryT* TryGetRef(FName ObjName) const
	{
		if (Blackboard == nullptr)
		{
			return false;
		}

		return Blackboard->TryGetRef_External<EntryT>(ObjName);
	}

private:
	UPROPERTY()
	TObjectPtr<URollbackBlackboard> Blackboard = nullptr;
};


/**
 * Wrapper class for in-simulation access to the blackboard. This exposes otherwise private API, and redirects
 * set/get operations to the in-simulation locations. This is what all movement mechanisms will interact with, including
 * movement modes, layered moves, etc.
 * This also supports a predictive mode, so the blackboard can be used and even temporarily written to, without
 * affecting the true simulation's state or external parties. Users of this wrapper will not be aware whether it's
 * part of a true movement sim or a predictive one.
 */
USTRUCT(BlueprintType)
struct FRollbackBlackboardSimWrapper
{
	GENERATED_BODY()

public:
	UE_API FRollbackBlackboardSimWrapper(URollbackBlackboard& InBlackboard, bool bUsePredictiveMode = false);
	UE_API FRollbackBlackboardSimWrapper(const FRollbackBlackboardSimWrapper& Other);
	// Default constructor exists only for reflection / BP support. Typically constructed with a blackboard or as a copy.
	UE_API FRollbackBlackboardSimWrapper();

	template<typename EntryT>
	bool CreateEntry(FName EntryName, const URollbackBlackboard::EntrySettings& InSettings)
	{
		if (Blackboard == nullptr)
		{
			return false;
		}

		return Blackboard->CreateEntry<EntryT>(EntryName, InSettings);
	}

	bool HasEntry(FName EntryName) const
	{
		if (Blackboard == nullptr)
		{
			return false;
		}

		return Blackboard->HasEntry(EntryName);
	}

	template<typename EntryT>
	bool TrySet(FName ObjName, const EntryT& Obj)
	{
		if (Blackboard == nullptr)
		{
			return false;
		}

		if (bInPredictiveMode)
		{
			return Blackboard->TrySet_Predictive<EntryT>(ObjName, Obj);
		}
		else
		{
			return Blackboard->TrySet_Internal<EntryT>(ObjName, Obj);
		}
	}

	template<typename EntryT>
	bool TryGet(FName ObjName, EntryT& OutFoundValue) const
	{
		if (Blackboard == nullptr)
		{
			return false;
		}

		if (bInPredictiveMode)
		{
			return Blackboard->TryGet_Predictive(ObjName, OutFoundValue);
		}
		else
		{
			return Blackboard->TryGet_Internal(ObjName, OutFoundValue);
		}
	}

	template<typename EntryT>
	const EntryT* TryGetRef(FName ObjName) const
	{
		if (Blackboard == nullptr)
		{
			return false;
		}

		if (bInPredictiveMode)
		{
			return Blackboard->TryGetRef_Predictive<EntryT>(ObjName);
		}
		else
		{
			return Blackboard->TryGetRef_Internal<EntryT>(ObjName);
		}
	}

	// TODO: Also support a mutable TryGetRef that gives you a writable reference

private:

	UPROPERTY()
	TObjectPtr<URollbackBlackboard> Blackboard = nullptr;

	UPROPERTY()
	bool bInPredictiveMode = false;
};

#undef UE_API