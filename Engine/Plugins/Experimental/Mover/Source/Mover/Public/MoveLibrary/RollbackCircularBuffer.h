// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"

enum class ERollbackBufferFlags
{
	None = 0,
	Prediction = (1 << 0),		// whether this buffer will allocate additional space for use with prediction
};
ENUM_CLASS_FLAGS(ERollbackBufferFlags)


/**
 * Circular buffer specialized for use with rollback simulations providing concurrent access 
 * between active simulation and external users, as well as extra space for predictive 
 * work that's temporary and to be discarded.
 *
 * This is based on TCircularBuffer's implementation. The size of the buffer is rounded up to 
 * the next power of two in order speed up indexing operations using a simple bit mask instead 
 * of the commonly used modulus operator that may be slow on some platforms.
 */
template<typename InElementType> class TRollbackCircularBuffer
{
public:
	using ElementType = InElementType;

	/**
	 * Creates and initializes a new instance of the TRollbackCircularBuffer class.
	 *
	 * @param Capacity The number of elements that the buffer can store
	 */
	[[nodiscard]] explicit TRollbackCircularBuffer(uint32 Capacity, ERollbackBufferFlags Flags)
	{
		checkSlow(Capacity <= 0x8000000u);	// Round-up doesn't work for values above this.

		Capacity = FMath::Max(Capacity, 1u);	// The circular buffer indexmask doesn't work for a size of 0, so make 1 the minimum

		uint32 NumElementsToAllocate = FMath::RoundUpToPowerOfTwo(Capacity);
		NonPredictedCapacity = NumElementsToAllocate;
		
		// If prediction is enabled, we'll use the last element as a "shadow" element for writes during prediction
		if ((Flags & ERollbackBufferFlags::Prediction) == ERollbackBufferFlags::Prediction)
		{
			++NumElementsToAllocate;	// we need 1 additional predictive entry
		}

		check(NonPredictedCapacity <= NumElementsToAllocate);

		Elements.AddDefaulted(NumElementsToAllocate);
		IndexMask = NonPredictedCapacity - 1;
	}

	/**
	 * Creates and initializes a new instance of the TRollbackCircularBuffer class.
	 *
	 * @param Capacity The number of elements that the buffer can store (will be rounded up to the next power of 2).
	 * @param InFlags
	 * @param InitialValue The initial value for the buffer's elements.
	 */
	[[nodiscard]] TRollbackCircularBuffer(uint32 Capacity, ERollbackBufferFlags Flags, const ElementType& InitialValue)
	{
		checkSlow(Capacity <= 0x8000000u);	// Round-up doesn't work for values above this.

		Capacity = FMath::Max(Capacity, 1u);	// The circular buffer indexmask doesn't work for a size of 0, so make 1 the minimum

		uint32 NumElementsToAllocate = FMath::RoundUpToPowerOfTwo(Capacity);
		NonPredictedCapacity = NumElementsToAllocate;

		// If prediction is enabled, we'll use the last element as a "shadow" element for writes during prediction
		if ((Flags & ERollbackBufferFlags::Prediction) == ERollbackBufferFlags::Prediction)
		{
			++NumElementsToAllocate;	// we need 1 additional predictive entry
		}

		check(NonPredictedCapacity <= NumElementsToAllocate);

		Elements.Init(InitialValue, NumElementsToAllocate);
		IndexMask = NonPredictedCapacity - 1;
	}

public:

	/**
	 * Returns the mutable element at the specified index.
	 *
	 * @param Index The index of the element to return.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT ElementType& operator[](uint32 Index)
	{
		return Elements[Index & IndexMask];
	}

	/**
	 * Returns the mutable predictive element. Only valid if the buffer was created with prediction supported.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT ElementType& GetPredictiveElement()
	{
		check((uint32)Elements.Num() > NonPredictedCapacity);	// If this fails, it's likely this buffer was created without prediction support, but someone is trying to use it in a predictive context.

		return Elements[Elements.Num()-1];	// the predictive element is always the last one in the array
	}

	/*
	 * Returns the immutable element at the specified index.
	 *
	 * @param Index The index of the element to return.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT const ElementType& operator[](uint32 Index) const
	{
		return Elements[Index & IndexMask];
	}


	/**
	 * Returns the immutable predictive element. Only valid if the buffer was created with prediction supported.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT const ElementType& GetPredictiveElement() const
	{
		check((uint32)Elements.Num() > NonPredictedCapacity);  // If this fails, it's likely this buffer was created without prediction support, but someone is trying to use it in a predictive context. We don't have a valid element ref to return.

		return Elements[Elements.Num() - 1];	// the predictive element is always the last one in the array
	}

public:

	/**
	 * Returns the number of elements that the buffer can naturally hold. 
	 *
	 * @return Buffer capacity.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT uint32 Capacity() const
	{
		return NonPredictedCapacity;
	}

	/**
	 * Calculates the index that follows the given index.
	 *
	 * @param CurrentIndex The current index.
	 * @return The next index.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT uint32 GetNextIndex(uint32 CurrentIndex) const
	{
		return ((CurrentIndex + 1) & IndexMask);
	}

	/**
	 * Calculates the index previous to the given index.
	 *
	 * @param CurrentIndex The current index.
	 * @return The previous index.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT uint32 GetPreviousIndex(uint32 CurrentIndex) const
	{
		return ((CurrentIndex - 1) & IndexMask);
	}

private:

	/** Holds the mask for indexing the buffer's elements. This mask covers only the non-predicted elements. The predictive element in the array will not be accessible. */
	uint32 IndexMask;

	/** This is the true number of usable element slots. If prediction is enabled, this will be smaller than Elements.Num() */
	uint32 NonPredictedCapacity;	

	/** Holds the buffer's elements. If prediction is enabled, this will have 1 additional element at the end of the array. */
	TArray<ElementType> Elements;
};
