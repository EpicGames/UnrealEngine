// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MoverDataModelTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "WaterBodyActor.h"

#include "WaterMovementUtils.generated.h"

#define UE_API MOVER_API

struct FProposedMove;

// Input parameters for ComputeControlledWaterMove()
USTRUCT(BlueprintType)
struct FWaterMoveParams
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	EMoveInputType MoveInputType = EMoveInputType::DirectionalIntent;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FVector MoveInput = FVector::ZeroVector;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FRotator OrientationIntent = FRotator::ZeroRotator;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FVector PriorVelocity = FVector::ZeroVector;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FRotator PriorOrientation = FRotator::ZeroRotator;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float MaxSpeed = 800.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float Acceleration = 4000.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float Deceleration = 8000.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float Friction = 0.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float TurningRate = 500.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float TurningBoost = 8.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float DeltaSeconds = 0.f;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FVector MoveAcceleration = FVector::ZeroVector;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float MoveSpeed = 0.f;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FQuat WorldToGravityQuat = FQuat::Identity;
};

// Data about the water volume and its interaction with the pawn used in calculating swimming movement
USTRUCT(BlueprintType)
struct FWaterFlowSplineData
{
	GENERATED_BODY()
	
	FWaterFlowSplineData()
	{
		Reset();
	}

	float SplineInputKey;
	float ImmersionDepth;
	float ImmersionPercent;
	float WaterDepth;
	float WaterVelocityDepthMultiplier;
	float WaveAttenuationFactor;
	float WaveReferenceTime;
	FVector RawWaterVelocity;
	FVector WaterVelocity;
	FVector PlayerRelativeVelocityToWater;
	FVector WaterSurfaceLocation;
	FVector WaterSurfaceNormal;
	FVector WaterSurfaceOffset;
	FVector WaterPlaneLocation;
	FVector WaterPlaneNormal;
	TWeakObjectPtr<class AWaterBody> WaterBody;

	bool IsValid() const { return SplineInputKey >= 0; }

	void Reset()
	{
		SplineInputKey = -1.0f;
		ImmersionDepth = -1.0f;
		ImmersionPercent = 0.f;
		WaterDepth = 0.f;
		WaterVelocityDepthMultiplier = 1.0f;
		WaveAttenuationFactor = 1.0f;
		WaveReferenceTime = 0.0f;
		RawWaterVelocity = FVector::ZeroVector;
		WaterVelocity = FVector::ZeroVector;
		PlayerRelativeVelocityToWater = FVector::ZeroVector;
		WaterSurfaceLocation = FVector::ZeroVector;
		WaterSurfaceNormal = FVector::UpVector;
		WaterSurfaceOffset = FVector::ZeroVector;
		WaterPlaneLocation = FVector::ZeroVector;
		WaterPlaneNormal = FVector(0.0f, 0.0f, 1.0f);
		WaterBody = nullptr;
	}
};

/** Data about the overlapping volume typically used for swimming */
USTRUCT(BlueprintType)
struct FWaterCheckResult
{
	GENERATED_BODY()
	
	/** True if the hit found a valid swimmable volume. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = CharacterVolume)
	uint32 bSwimmableVolume : 1;
	
	/** Hit result of the test that found a volume. Includes more specific data about the point of impact and surface normal at that point. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = CharacterVolume)
	FHitResult HitResult;

	/** Water Spline data to be used in calculating swimming movement, FX, etc. */ 
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = CharacterVolume)
	FWaterFlowSplineData WaterSplineData; 
	
public:
	FWaterCheckResult()
		: bSwimmableVolume(false)
		, HitResult(-1.f)
		, WaterSplineData()
	{
	}
	
	/** Returns true if the volume result hit a walkable surface. */
	bool IsSwimmableVolume() const
	{
		return bSwimmableVolume;
	}
	
	void Clear()
	{
		bSwimmableVolume = false;
		HitResult.Reset(-1.f, false);
	}
};

// Input parameters for Updating WaterSplineData
USTRUCT(BlueprintType)
struct FUpdateWaterSplineDataParams
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float TargetImmersionDepth = 0.f;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float WaterVelocityDepthForMax = 0.f;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float WaterVelocityMinMultiplier = 0.f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FVector PlayerVelocity = FVector::ZeroVector;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FVector PlayerLocation = FVector::ZeroVector;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	float CapsuleHalfHeight = 0.f;	
};

/**
 * WaterMovementUtils: a collection of stateless static BP-accessible functions for a variety of water movement-related operations
 */
UCLASS(MinimalAPI)
class UWaterMovementUtils : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Generate a new movement based on move/orientation intents and the prior state for the swimming move */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API FProposedMove ComputeControlledWaterMove(const FWaterMoveParams& InParams);
	
	/** Updates the water spline data used in calculated swimming movement */
	UFUNCTION(BlueprintCallable, Category=Mover)
	static UE_API void UpdateWaterSplineData(const FUpdateWaterSplineDataParams& UpdateWaterSplineDataParams, UPARAM(ref) FWaterCheckResult& WaterCheckResult);
};

#undef UE_API
