// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameplayTagContainer.h"
#include "MoverSimulationTypes.h"
#include "MoverTypes.h"
#include "MoverSimulationTypes.h"
#include "MoveLibrary/MoverBlackboard.h"
#include "MovementModeTransition.h"
#include "UObject/Interface.h"
#include "Templates/SubclassOf.h"
#include "MovementMode.generated.h"

#define UE_API MOVER_API


/**
 * UMovementSettingsInterface: interface that must be implemented for any settings object to be shared between modes
 */
UINTERFACE(MinimalAPI, BlueprintType)
class UMovementSettingsInterface : public UInterface
{
	GENERATED_BODY()
};

class IMovementSettingsInterface
{
	GENERATED_BODY()

public:
	virtual FString GetDisplayName() const = 0;
};

/**
 * Base class for all movement modes, exposing simulation update methods for both C++ and blueprint extension
 */
UCLASS(MinimalAPI, Abstract, Blueprintable, BlueprintType, EditInlineNew, DefaultToInstanced)
class UBaseMovementMode : public UObject
{
	GENERATED_BODY()

public:
	UE_API virtual UWorld* GetWorld() const override;
	
	UE_API virtual void OnRegistered(const FName ModeName, const FMoverSimContext& SimContext);
	UE_API virtual void OnUnregistered(const FMoverSimContext& SimContext);
	
	// These functions are called immediately within the simulation. If using async movement, it may be on a worker thread and unsafe to perform some operations on external objects.
	UE_API virtual void Activate(const FMoverEventContext& Context, FName PrevModeName, const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, FMoverSyncState* OutSyncState, FMoverAuxStateContext* OutAuxState);
	UE_API virtual void Deactivate(const FMoverEventContext& Context, FName NextModeName, const FMoverSimContext& SimContext);

	// These functions are called when the sync state is changed on the game thread
	// and a new mode is activated/deactivated
	UE_API virtual void Activate_External(const FMoverEventContext& Context);
	UE_API virtual void Deactivate_External(const FMoverEventContext& Context);
	
	UFUNCTION(BlueprintNativeEvent, meta = (DisplayName = "Generate Move", ForceAsFunction))
	UE_API void GenerateMove(const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, UPARAM(ref) FProposedMove& OutProposedMove) const;

	UFUNCTION(BlueprintNativeEvent, meta = (DisplayName = "Simulation Tick", ForceAsFunction))
	UE_API void SimulationTick(const FSimulationTickParams& Params, UPARAM(ref) FMoverTickEndData& OutputState);
	
	/** Gets the MoverComponent that owns this movement mode */
	UFUNCTION(BlueprintCallable, Category=Mover, meta=(DisplayName="Get Mover Component", ScriptName = GetMoverComponent))
	UE_API UMoverComponent* K2_GetMoverComponent() const;

	/**
	 * Gets the outer mover component of the indicated type. Does not check on the type or the presence of the MoverComp outer. Safe to call on CDOs.
	 */
	template<typename MoverT = UMoverComponent UE_REQUIRES(std::is_base_of_v<UMoverComponent, MoverT>)>
	MoverT* GetMoverComponent() const
	{
		return Cast<MoverT>(GetOuter());
	}

	/**
	 * Gets the outer mover component of the indicated type, checked for validity.
	 */
	template<typename MoverT = UMoverComponent UE_REQUIRES(std::is_base_of_v<UMoverComponent, MoverT>)>
	MoverT& GetMoverComponentChecked() const
	{
		return *CastChecked<MoverT>(GetOuter());
	}

	/**
   	 * Check Movement Mode for a gameplay tag.
   	 *
   	 * @param TagToFind			Tag to check on the Mover systems
   	 * @param bExactMatch		If true, the tag has to be exactly present, if false then TagToFind will include it's parent tags while matching
   	 * 
   	 * @return True if the TagToFind was found
   	 */
	UE_API virtual bool HasGameplayTag(FGameplayTag TagToFind, bool bExactMatch) const;

	/** Adds all gameplay tags currently active on this movement mode to InOutTags. */
	UE_API virtual void GetGameplayTags(FGameplayTagContainer& InOutTags) const;

#if WITH_EDITOR
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR

	/** Settings object type that this mode depends on. May be shared with other movement modes. When the mode is added to a Mover Component, it will create a shared instance of this settings class. */
	UPROPERTY(EditDefaultsOnly, Category = Mover, meta = (MustImplement = "/Script/Mover.MovementSettingsInterface"))
	TArray<TSubclassOf<UObject>> SharedSettingsClasses;

	/** Transition checks for the current mode. Evaluated in order, stopping at the first successful transition check */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Instanced, Category = Mover, meta = (FullyExpand = true))
	TArray<TObjectPtr<UBaseMovementModeTransition>> Transitions;

	/** A list of gameplay tags associated with this movement mode */
	UPROPERTY(EditDefaultsOnly, Category = Mover)
	FGameplayTagContainer GameplayTags;

	/** 
	 * Whether this movement mode supports being part of an asynchronous movement simulation (running concurrently with the gameplay thread) 
	 * Specifically for the GenerateMove and SimulationTick functions
	 */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = Mover)
	bool bSupportsAsync = false;

protected:
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "On Activated", ScriptName = "OnActivated"))
	UE_API void K2_OnActivated();

	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "On Deactivated", ScriptName = "OnDeactivated"))
	UE_API void K2_OnDeactivated();
	
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "On Registered", ScriptName = "OnRegistered"))
	UE_API void K2_OnRegistered(const FName ModeName);

	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "On Unregistered", ScriptName = "OnUnregistered"))
	UE_API void K2_OnUnregistered();

	UE_DEPRECATED(5.6, "OnActivate() has been renamed to Activate().")
	virtual void OnActivate() final {}
	UE_DEPRECATED(5.6, "OnDeactivate() has been renamed to Deactivate().")
	virtual void OnDeactivate() final  {}
	UE_DEPRECATED(5.6, "OnGenerateMove() has been replaced with a GenerateMove() BlueprintNativeEvent. Rename your override to GenerateMove_Implementation().")
	virtual void OnGenerateMove(const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FProposedMove& OutProposedMove) const final {}
	UE_DEPRECATED(5.6, "OnSimulationTick() has been replaced with a SimulationTick() BlueprintNativeEvent. Rename your override to SimulationTick_Implementation().")
	virtual void OnSimulationTick(const FSimulationTickParams& Params, FMoverTickEndData& OutputState) final {}
};

/**
 * NullMovementMode: a default do-nothing mode used as a placeholder when no other mode is active
 */
 UCLASS(MinimalAPI, NotBlueprintable)
class UNullMovementMode : public UBaseMovementMode
{
	GENERATED_UCLASS_BODY()

public:
	UE_API virtual void SimulationTick_Implementation(const FSimulationTickParams& Params, FMoverTickEndData& OutputState) override;

	UE_API const static FName NullModeName;
};

#undef UE_API
