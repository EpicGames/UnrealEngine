// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Templates/SubclassOf.h"
#include "MovementMode.h"
#include "MoveLibrary/RollbackBlackboardWrappers.h"
#include "InstantMovementEffect.h"
#include "MoverSimulationTypes.h"
#include "MovementModeStateMachine.generated.h"

#define UE_API MOVER_API

struct FProposedMove;
class UImmediateMovementModeTransition;
class UMovementModeTransition;

/**
 * - Any movement modes registered are co-owned by the state machine
 * - There is always an active mode, falling back to a do-nothing 'null' mode
 * - Queuing a mode that is already active will cause it to exit and re-enter
 * - Modes only switch during simulation tick
 */
 UCLASS(MinimalAPI)
class UMovementModeStateMachine : public UObject
{
	 GENERATED_UCLASS_BODY()

public:
	UE_API virtual void BeginDestroy() override;

	UE_API void RegisterMovementMode(FName ModeName, TObjectPtr<UBaseMovementMode> Mode, bool bIsDefaultMode);
	UE_API void RegisterMovementMode(FName ModeName, TSubclassOf<UBaseMovementMode> ModeType, bool bIsDefaultMode);

	UE_API void UnregisterMovementMode(FName ModeName);
	UE_API void ClearAllMovementModes();

	UE_API void RegisterGlobalTransition(TObjectPtr<UBaseMovementModeTransition> Transition);
	UE_API void UnregisterGlobalTransition(TObjectPtr<UBaseMovementModeTransition> Transition);
	UE_API void ClearAllGlobalTransitions();

	// TODO: Warning: this function will be removed once the full simulation context (including blackboard) comes in as part of the Simulation**** function params
	UE_API void SetSimulation(UMoverSimulation& InSimulation);

	UE_API void SetDefaultMode(FName NewDefaultModeName);

	UE_API void QueueNextMode(FName DesiredNextModeName, bool bShouldReenter=false);
	UE_API void SetModeImmediately(FName DesiredModeName, bool bShouldReenter=false);
	UE_API void ClearQueuedMode();

	UE_API void OnSimulationTick(USceneComponent* UpdatedComponent, UPrimitiveComponent* UpdatedPrimitive, UMoverBlackboard* SimBlackboard, const FMoverSimContext& SimContext, const FMoverTickStartData& StartState, const FMoverTimeStep& TimeStep, FMoverTickEndData& OutputState);
 	UE_API void OnSimulationPreRollback(const FMoverSyncState* InvalidSyncState, const FMoverSyncState* SyncState, const FMoverAuxStateContext* InvalidAuxState, const FMoverAuxStateContext* AuxState, const FMoverTimeStep& NewBaseTimeStep);
	UE_API void OnSimulationRollback(const FMoverSyncState* SyncState, const FMoverAuxStateContext* AuxState, const FMoverTimeStep& NewBaseTimeStep);

	FName GetCurrentModeName() const { return CurrentModeName; }
	FName GetQueuedModeName() const;

	UE_API const UBaseMovementMode* GetCurrentMode() const;

	UE_API const UBaseMovementMode* FindMovementMode(FName ModeName) const;

	UE_API void QueueActiveLayeredMove(const TSharedPtr<FLayeredMoveInstance>& LayeredMove);

 	UE_API FMovementModifierHandle QueueMovementModifier(TSharedPtr<FMovementModifierBase> Modifier);

 	UE_API void CancelModifierFromHandle(FMovementModifierHandle ModifierHandle);

 	UE_API const FMovementModifierBase* FindQueuedModifier(FMovementModifierHandle ModifierHandle) const;
 	UE_API const FMovementModifierBase* FindQueuedModifierByType(const UScriptStruct* ModifierType) const;

	UE_API void CancelFeaturesWithTag(FGameplayTag TagToCancel, bool bRequireExactMatch=false);
	UE_API bool HasFeaturesWithTag(FGameplayTag TagToMatch, bool bRequireExactMatch = false) const;

	// This function is meant to be used only in async mode on a worker thread, not on the game thread
	void QueueInstantMovementEffect_Internal(const FScheduledInstantMovementEffect& ScheduledInstantMovementEffect);

	// This function is meant to be used only in async mode on a worker thread, not on the game thread
	void QueueLayeredMove_Internal(const FScheduledLayeredMove& ScheduledLayeredMove);

protected:
	UE_API void QueueInstantMovementEffect(const FScheduledInstantMovementEffect& ScheduledInstantMovementEffect);
	UE_API void QueueInstantMovementEffects(const TArray<FScheduledInstantMovementEffect>& ScheduledInstantMovementEffects);

	UE_API void QueueLayeredMove(const FScheduledLayeredMove& ScheduledLayeredMove);
	UE_API void QueueLayeredMoves(const TArray<FScheduledLayeredMove>& ScheduledLayeredMoves);

	void ProcessEvents(const TArray<TSharedPtr<FMoverSimulationEventData>>& InEvents);
	UE_API virtual void ProcessSimulationEvent(const FMoverSimulationEventData& EventData);

	UE_API void QueueNextModeInternal(FName DesiredNextModeName, bool bShouldReenter=false, bool bIsFromRollback=false);
	UE_API void SetModeImmediatelyInternal(FName DesiredModeName, bool bShouldReenter=false, bool bIsFromRollback=false);

	UE_API virtual void PostInitProperties() override;

	UPROPERTY()
	TMap<FName, TObjectPtr<UBaseMovementMode>> Modes;
	TArray<TObjectPtr<UBaseMovementModeTransition>> GlobalTransitions;

	UPROPERTY(Transient)
	TObjectPtr<UImmediateMovementModeTransition> QueuedModeTransition;

	UPROPERTY(Transient)
	TObjectPtr<UMoverSimulation> Simulation;

	FName DefaultModeName = NAME_None;
	FName CurrentModeName = NAME_None;

	// Represents the current sim time that's passed, and the next frame number that's next to be simulated.
	FMoverTimeStep CurrentBaseTimeStep;

	/** Moves that are queued to be added to the simulation when their scheduled time has come (earliest is at the start of the next sim subtick). Access covered by lock. */
	TArray<FScheduledLayeredMove> QueuedLayeredMoves;

	/** Moves that are queued to be added to the simulation at the start of the next sim subtick. Access covered by lock. */
 	TArray<TSharedPtr<FLayeredMoveInstance>> QueuedLayeredMoveInstances;
 	
 	/** Effects that are queued to be applied to the simulation when their scheduled time has come (earliest is at the start of the next sim subtick, or at the end of this tick).  Access covered by lock. */
 	TArray<FScheduledInstantMovementEffect> QueuedInstantEffects;

 	/** Modifiers that are queued to be added to the simulation at the start of the next sim subtick. Access covered by lock. */
 	TArray<TSharedPtr<FMovementModifierBase>> QueuedMovementModifiers;

 	/** Modifiers that are to be canceled at the start of the next sim subtick.  Access covered by lock. */
 	TArray<FMovementModifierHandle> ModifiersToCancel;
 	
	/** Tags that are used to cancel any matching movement features (modifiers, layered moves, etc). Access covered by lock. */
	TArray<TPair<FGameplayTag, bool>> TagCancellationRequests;

	// Internal-use-only tick data structs, for efficiency since they typically have the same contents from frame to frame
	FMoverTickStartData WorkingSubstepStartData;
	FSimulationTickParams WorkingSimTickParams;
	FMoverTickEndData* WorkingSimTickEndData = nullptr;	// Only valid during OnSimulationTick and null otherwise

private:
	// Locks for thread safety on queueing mechanisms
	mutable FTransactionallySafeRWLock LayeredMoveQueueLock;
	mutable FTransactionallySafeRWLock InstantEffectsQueueLock;
	mutable FTransactionallySafeRWLock ModifiersQueueLock;
	mutable FTransactionallySafeRWLock ModifierCancelQueueLock;
	mutable FTransactionallySafeRWLock TagCancellationRequestsLock;

	UE_API void ConstructDefaultModes();
	UE_API void AdvanceToNextMode();
	UE_API void FlushQueuedMovesToGroup(const FMoverTimeStep& TimeStep, FLayeredMoveGroup& Group);
	// Flushes queued ActiveLayeredMoves to FLayeredMoveInstanceGroup for this frame
 	UE_API void ActivateQueuedMoves(FLayeredMoveInstanceGroup& Group);
 	UE_API void FlushQueuedModifiersToGroup(FMovementModifierGroup& ModifierGroup);
 	UE_API void FlushModifierCancellationsToGroup(FMovementModifierGroup& ActiveModifierGroup);
	UE_API void FlushTagCancellationsToSyncState(FMoverSyncState& SyncState);
 	UE_API void RollbackModifiers(const FMoverSyncState* InvalidSyncState, const FMoverSyncState* SyncState, const FMoverAuxStateContext* InvalidAuxState, const FMoverAuxStateContext* AuxState);
	UE_API void RollBackQueuedInstantMovementEffects(const FMoverTimeStep& NewBaseTimeStep);
	UE_API void RollBackQueuedLayeredMoves(const FMoverTimeStep& NewBaseTimeStep);
	UE_API bool HasAnyInstantEffectsQueued() const;
 	UE_API bool ApplyInstantEffects(FApplyMovementEffectParams& ApplyEffectParams, FMoverSyncState& OutputState);
 	UE_API bool UpdateWorkingStateFromSyncState(FMoverSyncState& OutputState);
	UE_API FMoverSimContext GetSimContext() const;
};

#undef UE_API
