// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "MovementModeTransition.generated.h"

#define UE_API MOVER_API

struct FSimulationTickParams;
class UMoverComponent;

/** 
 * Results from a movement mode transition evaluation
 */
USTRUCT(BlueprintType)
struct FTransitionEvalResult
{
	GENERATED_BODY()

	FTransitionEvalResult() {}
	FTransitionEvalResult(FName InNextMode) { NextMode = InNextMode; }

	// Mode name that should be transitioned to. NAME_None indicates no transition.
	UPROPERTY(BlueprintReadWrite, Category=Mover)
	FName NextMode = NAME_None;


	static UE_API const FTransitionEvalResult NoTransition;
};


/**
 * Base class for all transitions
 */
UCLASS(MinimalAPI, Abstract, Blueprintable, BlueprintType, EditInlineNew, DefaultToInstanced)
class UBaseMovementModeTransition : public UObject
{
	GENERATED_BODY()

public:
	UE_API virtual UWorld* GetWorld() const override;
	
	UE_API virtual void OnRegistered();
	UE_API virtual void OnUnregistered();

	/** Gets the MoverComponent that ultimately owns this transition */
	UFUNCTION(BlueprintCallable, Category = Mover, meta = (DisplayName = "Get Mover Component"))
	UE_API UMoverComponent* K2_GetMoverComponent() const;

	template<typename MoverT = UMoverComponent UE_REQUIRES(std::is_base_of_v<UMoverComponent, MoverT>)>
	MoverT* GetMoverComponent() const
	{
		return Cast<MoverT>(K2_GetMoverComponent());
	}
	
	template<typename MoverT = UMoverComponent UE_REQUIRES(std::is_base_of_v<UMoverComponent, MoverT>)>
	MoverT& GetMoverComponentChecked() const
	{
		return *CastChecked<MoverT>(K2_GetMoverComponent());
	}

	UFUNCTION(BlueprintNativeEvent, Category = MovementModeTransition)
	UE_API FTransitionEvalResult Evaluate(const FSimulationTickParams& Params) const;

	UFUNCTION(BlueprintNativeEvent, Category = MovementModeTransition)
	UE_API void Trigger(const FSimulationTickParams& Params);

#if WITH_EDITOR
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif // WITH_EDITOR

	/** Settings object type that this mode depends on. May be shared with other transitions and movement modes. When the transition is added to a Mover Component, it will create a shared instance of this settings class. */
	UPROPERTY(EditDefaultsOnly, Category = Mover, meta = (MustImplement = "/Script/Mover.MovementSettingsInterface"))
	TArray<TSubclassOf<UObject>> SharedSettingsClasses;

	/** Whether this transition should reenter a mode if it evaluates true and wants to transition into a mode the actor is already in */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = Mover)
	uint8 bAllowModeReentry : 1 = 0;

	/** Whether this transition should only apply to the first step of the update. If true, modes reached after transitions or mode changes
	* in the current update will not consider this transition
	*/
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = Mover)
	uint8 bFirstSubStepOnly : 1 = 0;

	/** 
	 * Whether this movement mode transition supports being part of an asynchronous movement simulation (running concurrently with the gameplay thread) 
	 * Specifically for the Evaluate and Trigger functions
	 */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = Mover)
	uint8 bSupportsAsync : 1 = 0;

protected:
	UFUNCTION(BlueprintImplementableEvent, Category = MovementModeTransition, meta = (DisplayName = "On Registered", ScriptName = "OnRegistered"))
	UE_API void K2_OnRegistered();
	
	UFUNCTION(BlueprintImplementableEvent, Category = MovementModeTransition, meta = (DisplayName = "On Unregistered", ScriptName = "OnUnregistered"))
	UE_API void K2_OnUnregistered();

	UE_DEPRECATED(5.6, "OnEvaluate() has been replaced with an Evaluate() BlueprintNativeEvent. Rename your override to Evaluate_Implementation().")
	virtual FTransitionEvalResult OnEvaluate(const FSimulationTickParams& Params) const final { return FTransitionEvalResult::NoTransition; }
	UE_DEPRECATED(5.6, "OnTrigger() has been replaced with a Trigger() BlueprintNativeEvent. Rename your override to Trigger_Implementation().")
	virtual void OnTrigger(const FSimulationTickParams& Params) final {}
};

/**
 * Simple transition that evaluates true if a "next mode" is set. Used internally only by the Mover plugin. 
 */
UCLASS(MinimalAPI)
class UImmediateMovementModeTransition : public UBaseMovementModeTransition
{
	GENERATED_UCLASS_BODY()

public:
	UE_API virtual FTransitionEvalResult Evaluate_Implementation(const FSimulationTickParams& Params) const override;
	UE_API virtual void Trigger_Implementation(const FSimulationTickParams& Params) override;

	bool IsSet() const { return !NextMode.IsNone(); }
	UE_API void SetNextMode(FName DesiredModeName, bool bShouldReenter=false, bool bIsFromRollback=false);
	UE_API void Clear();

	FName GetNextModeName() const { return NextMode; }
	bool ShouldReenter() const { return bAllowModeReentry; }
	bool IsCausedByRollback() const { return bIsCausedByRollback; }

private:
	FName NextMode = NAME_None;
	uint8 bIsCausedByRollback : 1 = 0;
};



#undef UE_API
