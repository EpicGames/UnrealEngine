// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayTagContainer.h"
#include "MoveLibrary/MovementUtilsTypes.h"
#include "MovementModifier.generated.h"

#define UE_API MOVER_API

class UMoverComponent;
class UMoverSimulation;
struct FMoverAuxStateContext;
struct FMoverTimeStep;
struct FMoverSyncState;
struct FMoverInputCmdContext;

#define MODIFIER_HANDLE_TYPE uint16
#define MODIFIER_HANDLE_MAX UINT16_MAX
#define MODIFIER_INVALID_HANDLE 0

/**
 * Handle to identify movement modifiers.
 */
USTRUCT(BlueprintType)
struct FMovementModifierHandle
{
	GENERATED_BODY()

	FMovementModifierHandle()
		: Handle(0)
	{
	}

	FMovementModifierHandle(MODIFIER_HANDLE_TYPE InHandle)
		: Handle(InHandle)
	{
	}

	/** Creates a new handle */
	UE_API void GenerateHandle();
	
	bool operator==(const FMovementModifierHandle& Other) const
	{
		return Handle == Other.Handle;
	}

	bool operator!=(const FMovementModifierHandle& Other) const
	{
		return Handle != Other.Handle;
	}

	FString ToString() const
	{
		return FString::Printf(TEXT("%d"), Handle);
	}

	bool IsValid() const
	{
		return Handle != MODIFIER_INVALID_HANDLE;
	}

	void Invalidate()
	{
		Handle = MODIFIER_INVALID_HANDLE;
	}
	
private:
	MODIFIER_HANDLE_TYPE Handle;
};

struct FMovementModifierParams_Async
{
	FMovementModifierParams_Async(UMoverSimulation* InSimulation, const FMoverSyncState* InSyncState, const FMoverTimeStep* InTimeStep)
		: Simulation(InSimulation)
		, SyncState(InSyncState)
		, TimeStep(InTimeStep)
	{
		ensureMsgf(IsValid(), TEXT("Invalid initialization data for FMovementModifierParams_Async"));
	}
	FMovementModifierParams_Async() = delete;

	bool IsValid() const
	{
		return Simulation && SyncState && TimeStep;
	}

	UMoverSimulation* Simulation = nullptr;
	const FMoverSyncState* SyncState = nullptr;
	const FMoverTimeStep* TimeStep = nullptr;
};

/**
 * Movement Modifier: Used to apply changes that indirectly influence the movement simulation,
 * without proposing or executing any movement, but still in sync with the sim.
 * Example usages: changing groups of settings, movement mode re-mappings, etc.
 * 
 * Note: Currently mover expects to only have one type of modifier active at a time.
 *		 This can be fixed by extending the Matches function to check more than just type,
 *		 but make sure anything used to compare is synced through the NetSerialize function.
 */
USTRUCT(BlueprintInternalUseOnly)
struct FMovementModifierBase
{
	GENERATED_BODY()

	UE_API FMovementModifierBase();
	virtual ~FMovementModifierBase() = default;
	
	// This modifier will expire after a set amount of time if > 0. If 0, it will be ticked only once, regardless of time step. It will need to be manually ended if < 0. 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	float DurationMs;

	// The simulation time this move first ticked (< 0 means it hasn't started yet)
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Transient, Category = Mover)
	double StartSimTimeMs;
	
	/** Fired when this modifier is activated. */
	virtual void OnStart(UMoverComponent* MoverComp, const FMoverTimeStep& TimeStep, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState) {}
	
	/** Fired when this modifier is deactivated. */
	virtual void OnEnd(UMoverComponent* MoverComp, const FMoverTimeStep& TimeStep, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState) {}
	
	/**
	 * Fired just before a movement Substep
	 */
	virtual void OnPreMovement(UMoverComponent* MoverComp, const FMoverTimeStep& TimeStep) {}

	/** Fired after a movement Substep */
	virtual void OnPostMovement(UMoverComponent* MoverComp, const FMoverTimeStep& TimeStep, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState) {}

	/** Kicks off this modifier, allowing any initialization to occur. */
	UE_API void StartModifier(UMoverComponent* MoverComp, const FMoverTimeStep& TimeStep, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState);

	/** Ends this move, allowing any cleanup to occur. */
	UE_API void EndModifier(UMoverComponent* MoverComp, const FMoverTimeStep& TimeStep, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState);

	/** Runtime query whether this modifier is finished and can be destroyed. The default implementation is based on DurationMs. */
	UE_API virtual bool IsFinished(double CurrentSimTimeMs) const;
	

	// Begin Async API
	// The async API is used by the async friendly mover simulation class
	// which is currently implemented in ChaosMover for async physics support.
	// The standard Mover update does not currently require these async functions
	// to be implemented
	virtual void OnStart_Async(const FMovementModifierParams_Async& Params) {}
	virtual void OnEnd_Async(const FMovementModifierParams_Async& Params) {}
	virtual void OnPreMovement_Async(const FMovementModifierParams_Async& Params) {}
	virtual void OnPostMovement_Async(const FMovementModifierParams_Async& Params) {}

	UE_API void StartModifier_Async(const FMovementModifierParams_Async& Params);
	UE_API void EndModifier_Async(const FMovementModifierParams_Async& Params);
	// End Async API



	// @return newly allocated copy of this FMovementModifier. Must be overridden by child classes
	UE_API virtual FMovementModifierBase* Clone() const;

	UE_API virtual void NetSerialize(FArchive& Ar);

	UE_API virtual UScriptStruct* GetScriptStruct() const;

	UE_API virtual FString ToSimpleString() const;

	virtual void AddReferencedObjects(class FReferenceCollector& Collector) {}

	/** 
	 * Used to compare modifiers and check if they are the same instance of modifiers.
	 * Doesn't need to be overridden but more specific data to match is best
	 * Note: Current default implementation only checks type and nothing else.
	 */ 
	UE_API virtual bool Matches(const FMovementModifierBase* Other) const;

	UE_API FMovementModifierHandle GetHandle() const;

	UE_API void GenerateHandle();

	/**
	 * Used to write to a valid handle to a invalid handle
	 * Currently used to write a local handle to modifiers that were added from a rollback so they don't have a handle yet
	 * This is done to avoid queueing a modifier again as a local client applies potential input
	 */
	UE_API void OverwriteHandleIfInvalid(const FMovementModifierHandle& ValidModifierHandle);

	/**
  	 * Check modifier for a gameplay tag.
  	 *
  	 * @param TagToFind			Tag to check on the Mover systems
  	 * @param bExactMatch		If true, the tag has to be exactly present, if false then TagToFind will include it's parent tags while matching
  	 * 
  	 * @return True if the TagToFind was found
  	 */
	virtual bool HasGameplayTag(FGameplayTag TagToFind, bool bExactMatch) const
	{
		return false;
	}

	/** Adds all gameplay tags currently active on this modifier to InOutTags. */
	virtual void GetGameplayTags(FGameplayTagContainer& InOutTags) const {}

protected:
	/**
	 * Modifier handle local to this client or server instance. Used to cancel or query for a active/queued modifier.
	 */
	UPROPERTY()
	FMovementModifierHandle LocalModifierHandle;
};

template<>
struct TStructOpsTypeTraits< FMovementModifierBase > : public TStructOpsTypeTraitsBase2< FMovementModifierBase >
{
	enum
	{
		//WithNetSerializer = true,
		WithCopy = true
	};
};


// A collection of movement modifiers affecting a movable actor
USTRUCT(BlueprintType)
struct FMovementModifierGroup
{
	GENERATED_BODY()

	FMovementModifierGroup() {}
	
	bool HasAnyMoves() const { return (!ActiveModifiers.IsEmpty() || !QueuedModifiers.IsEmpty()); }

	/** Serialize all modifiers and their states for this group */
	UE_API void NetSerialize(FArchive& Ar, uint8 MaxNumModifiersToSerialize = MAX_uint8);
	
	UE_API void QueueMovementModifier(TSharedPtr<FMovementModifierBase> Modifier);

	UE_API void CancelModifierFromHandle(const FMovementModifierHandle& HandleToCancel);
	UE_API void CancelModifiersByTag(FGameplayTag Tag, bool bRequiresExactMatch=false);
	
	// Generates active modifier list (by calling FlushModifierArrays) and returns the an array of all currently active modifiers
	UE_API TArray<TSharedPtr<FMovementModifierBase>> GenerateActiveModifiers(UMoverComponent* MoverComp, const FMoverTimeStep& TimeStep, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState);
	
	UE_API TArray<TSharedPtr<FMovementModifierBase>> GenerateActiveModifiers_Async(const FMovementModifierParams_Async& Params);

	/** Copy operator - deep copy so it can be used for archiving/saving off moves */
	UE_API FMovementModifierGroup& operator=(const FMovementModifierGroup& Other);

	/** Comparison operator - needs matching modifier along with identical states in those structs */
	UE_API bool operator==(const FMovementModifierGroup& Other) const;

	/** Comparison operator */
	UE_API bool operator!=(const FMovementModifierGroup& Other) const;

	/** Checks only whether there are matching modifiers, but NOT necessarily identical states of each one */
	UE_API bool HasSameContents(const FMovementModifierGroup& Other) const;

	/** Exposes references to GC system */
	UE_API void AddStructReferencedObjects(FReferenceCollector& Collector) const;

	/** Get a simplified string representation of this group. Typically for debugging. */
	UE_API FString ToSimpleString() const;

	/** Const access to active moves */
	UE_API TArray<TSharedPtr<FMovementModifierBase>>::TConstIterator GetActiveModifiersIterator() const;
	
	/** Const access to queued moves */
	UE_API TArray<TSharedPtr<FMovementModifierBase>>::TConstIterator GetQueuedModifiersIterator() const;

	// Clears active and queued modifiers
	UE_API void Reset();

	// Whether this modifier should reconcile with the authority state
	UE_API bool ShouldReconcile(const FMovementModifierGroup& Other) const;
	
protected:
	/** Movement modifiers that are currently active in this group */
	TArray< TSharedPtr<FMovementModifierBase> > ActiveModifiers;

	/** Movement modifiers that are queued to become active next sim frame */
	TArray< TSharedPtr<FMovementModifierBase> > QueuedModifiers;
	
	// Clears out any finished or invalid modifiers and adds any queued modifiers to the active modifiers
	UE_API void FlushModifierArrays(UMoverComponent* MoverComp, const FMoverTimeStep& TimeStep, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState);
	
	UE_API void FlushModifierArrays_Async(const FMovementModifierParams_Async& Params);

	/** Helper function for serializing array of movement modifiers */
	static UE_API void NetSerializeMovementModifierArray(FArchive& Ar, TArray< TSharedPtr<FMovementModifierBase> >& ModifierArray, uint8 MaxNumModifiersToSerialize = MAX_uint8);
	

};


template<>
struct TStructOpsTypeTraits<FMovementModifierGroup> : public TStructOpsTypeTraitsBase2<FMovementModifierGroup>
{
	enum
	{
		WithCopy = true,		// Necessary so that TSharedPtr<FMovementModifierGroup> Data is copied around
		//WithNetSerializer = true,
		WithIdenticalViaEquality = true,
		WithAddStructReferencedObjects = true,
	};
};

#undef UE_API
