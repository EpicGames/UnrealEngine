// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "MotionWarpingAdapter.h"
#include "MovementMode.h"
#include "MoverTypes.h"
#include "LayeredMove.h"
#include "LayeredMoveBase.h"
#include "MoveLibrary/BasedMovementUtils.h"
#include "MoveLibrary/ConstrainedMoveUtils.h"
#include "MoveLibrary/RollbackBlackboardWrappers.h"
#include "MovementModifier.h"
#include "Backends/MoverBackendLiaison.h"
#include "GameFramework/MovementInterface.h"
#include "DefaultMovementSet/MoverMontageSimulationTypes.h"
#include "UObject/WeakInterfacePtr.h"
#include "Templates/SharedPointer.h"

#include "MoverComponent.generated.h"

struct FMoverTimeStep;
struct FInstantMovementEffect;
struct FMoverSimulationEventData;
struct FMovementBaseTransformDelta;
class UAnimInstance;
class UAnimMontage;
class UMovementModeStateMachine;
class UMovementMixer;
class URollbackBlackboard;

namespace MoverComponentConstants
{
	extern const FVector DefaultGravityAccel;		// Fallback gravity if not determined by the component or world (cm/s^2)
	extern const FVector DefaultUpDir;				// Fallback up direction if not determined by the component or world (normalized)
}

// Fired just before a simulation tick, regardless of being a re-simulated frame or not.
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FMover_OnPreSimTick, const FMoverTimeStep&, TimeStep, const FMoverInputCmdContext&, InputCmd);

// Fired during a simulation tick, after the input is processed but before the actual move calculation.
DECLARE_DYNAMIC_MULTICAST_DELEGATE_FourParams(FMover_OnPreMovement, const FMoverTimeStep&, TimeStep, const FMoverInputCmdContext&, InputCmd, const FMoverSyncState&, SyncState, const FMoverAuxStateContext&, AuxState);

// Fired during a simulation tick, after movement has occurred but before the state is finalized, allowing changes to the output state.
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FMover_OnPostMovement, const FMoverTimeStep&, TimeStep, FMoverSyncState&, SyncState, FMoverAuxStateContext&, AuxState);

// Fired after a simulation tick, regardless of being a re-simulated frame or not.
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FMover_OnPostSimTick, const FMoverTimeStep&, TimeStep);

// Fired after a rollback. First param is the time step we've rolled back to. Second param is when we rolled back from, and represents a later frame that is no longer valid.
// Note: ExpungedTimeStep is called PreRollbackTimeStep in C++ code
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FMover_OnPostSimRollback, const FMoverTimeStep&, CurrentTimeStep, const FMoverTimeStep&, ExpungedTimeStep);

// Fired after changing movement modes. First param is the name of the previous movement mode. Second is the name of the new movement mode. 
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FMover_OnMovementModeChanged, const FName&, PreviousMovementModeName, const FName&, NewMovementModeName);

// Fired when a teleport has succeeded
DECLARE_DYNAMIC_MULTICAST_DELEGATE_FourParams(FMover_OnTeleportSucceeded, const FVector&, FromLocation, const FQuat&, FromRotation, const FVector&, ToLocation, const FQuat&, ToRotation);

// Fired when a teleport has failed
DECLARE_DYNAMIC_MULTICAST_DELEGATE_FiveParams(FMover_OnTeleportFailed, const FVector&, FromLocation, const FQuat&, FromRotation, const FVector&, ToLocation, const FQuat&, ToRotation, ETeleportFailureReason, TeleportFailureReason);

// Fired when a gameplay tag is added to the simulation (e.g. Mover_IsCrouching becomes set). Context carries bIsCausedByRollback and bIsDuringResimulation.
// bIsExternalTag is true if the tag was applied via AddGameplayTag rather than from within the simulation.
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FMover_OnGameplayTagAdded, const FGameplayTag&, Tag, const FMoverEventContext&, EventContext, bool, bIsExternalTag);

// Fired when a gameplay tag is removed from the simulation. Context carries bIsCausedByRollback and bIsDuringResimulation.
// bIsExternalTag is true if the tag was removed via RemoveGameplayTag rather than from within the simulation.
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FMover_OnGameplayTagRemoved, const FGameplayTag&, Tag, const FMoverEventContext&, EventContext, bool, bIsExternalTag);

// Fired after a transition has been triggered.
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FMover_OnTransitionTriggered, const UBaseMovementModeTransition*, ModeTransition);

// Fired after a frame has been finalized. This may be a resimulation or not. No changes to state are possible. Guaranteed to be on the game thread.
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FMover_OnPostFinalize, const FMoverSyncState&, SyncState, const FMoverAuxStateContext&, AuxState);

// Fired after proposed movement has been generated (i.e. after movement modes and layered moves have generated movement and mixed together).
DECLARE_DYNAMIC_DELEGATE_ThreeParams(FMover_ProcessGeneratedMovement, const FMoverTickStartData&, StartState, const FMoverTimeStep&, TimeStep, FProposedMove&, OutProposedMove);

// Fired when a new event has been received from the simulation. This is a C++ only dispatch of the generic event. To dispatch the event to BP, prefer exposing the concrete event
// via a dedicated dynamic delegate (like OnMovementModeChanged).
DECLARE_MULTICAST_DELEGATE_OneParam(FMover_OnPostSimEventReceived, const FMoverSimulationEventData&);

// Fired after based movement is applied to this MoverComponent. TransformDelta is this component's cumulative transform change in world space during the most recent movement simulation tick.  (See @bSupportsKinematicBasedMovement)
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FMover_OnBasedMovementApplied, const FTransform&, TransformDelta, const FMoverTimeStep&, TimeStep);


/**
 * 
 */
UCLASS(MinimalAPI, Blueprintable, BlueprintType, meta = (BlueprintSpawnableComponent))
class UMoverComponent : public UActorComponent, public IMovementInterface
{
	GENERATED_BODY()


public:	
	MOVER_API UMoverComponent();

	MOVER_API virtual void InitializeComponent() override;
	MOVER_API virtual void UninitializeComponent() override;
	MOVER_API virtual void OnRegister() override;
	MOVER_API virtual void RegisterComponentTickFunctions(bool bRegister) override;
	MOVER_API virtual void PostLoad() override;
	MOVER_API virtual void BeginPlay() override;
	MOVER_API virtual void EndPlay(const EEndPlayReason::Type Reason) override;
	
	// Broadcast before each simulation tick.
	// Note - Guaranteed to run on the game thread (even in async simulation).
	UPROPERTY(BlueprintAssignable, Category = Mover)
	FMover_OnPreSimTick OnPreSimulationTick;

	// Broadcast at the end of a simulation tick after movement has occurred, but allowing additions/modifications to the state. Not assignable as a BP event due to it having mutable parameters.
	UPROPERTY()
	FMover_OnPostMovement OnPostMovement;

	// Broadcast after each simulation tick and the state is finalized
	UPROPERTY(BlueprintAssignable, Category = Mover)
	FMover_OnPostSimTick OnPostSimulationTick;

	// Broadcast when a rollback has occurred, just before the next simulation tick occurs
	UPROPERTY(BlueprintAssignable, Category = Mover)
	FMover_OnPostSimRollback OnPostSimulationRollback;

	// Broadcast when a MovementMode has changed. Happens during a simulation tick if the mode changed. 
	UPROPERTY(BlueprintAssignable, Category = Mover)
	FMover_OnMovementModeChanged OnMovementModeChanged;

	// Broadcast when a teleport has succeeded
	UPROPERTY(BlueprintAssignable, Category = Mover)
	FMover_OnTeleportSucceeded OnTeleportSucceeded;

	// Broadcast when a teleport has failed
	UPROPERTY(BlueprintAssignable, Category = Mover)
	FMover_OnTeleportFailed OnTeleportFailed;

	// Broadcast after any based movement has been finished appling to the owner of this MoverComponent during a single 
	// movement simulation tick. This is always on the game thread, after the base has moved. (see @bSupportsKinematicBasedMovement)
	UPROPERTY(BlueprintAssignable, Category = Mover)
	FMover_OnBasedMovementApplied OnBasedMovementApplied;

	// Broadcast when a gameplay tag becomes active, whether driven by the simulation internally or applied externally via AddGameplayTag. Fired on the game thread.
	UPROPERTY(BlueprintAssignable, Category = Mover)
	FMover_OnGameplayTagAdded OnGameplayTagAdded;

	// Broadcast when a gameplay tag is no longer active, whether driven by the simulation internally or removed externally via RemoveGameplayTag. Fired on the game thread.
	UPROPERTY(BlueprintAssignable, Category = Mover)
	FMover_OnGameplayTagRemoved OnGameplayTagRemoved;

	// Broadcast when a Transition has been triggered.
	UPROPERTY(BlueprintAssignable, Category = Mover)
	FMover_OnTransitionTriggered OnMovementTransitionTriggered;

	// Broadcast after each finalized simulation frame, after the state is finalized. (Game thread only)
	UPROPERTY(BlueprintAssignable, Category = Mover)
	FMover_OnPostFinalize OnPostFinalize;

	// Fired when a new event has been received from the simulation
	// This happens after the event had been processed at the mover component level, which may
	// have caused a dedicated delegate, e.g. OnMovementModeChanged, to fire prior to this broadcast.
	FMover_OnPostSimEventReceived OnPostSimEventReceived;

	/**
	 * Broadcast after proposed movement has been generated. After movement modes and layered moves have generated movement and mixed together.
	 * This allows for final modifications to proposed movement before it's executed.
	 */
	FMover_ProcessGeneratedMovement ProcessGeneratedMovement;
	
	// Binds event for processing movement after it has been generated. Allows for final modifications to proposed movement before it's executed.
	UFUNCTION(BlueprintCallable, Category = Mover)
	MOVER_API void BindProcessGeneratedMovement(FMover_ProcessGeneratedMovement ProcessGeneratedMovementEvent);
	// Clears current bound event for processing movement after it has been generated.
	UFUNCTION(BlueprintCallable, Category = Mover)
	MOVER_API void UnbindProcessGeneratedMovement();
	
	// Callbacks
	UFUNCTION()
	virtual void OnBeginOverlap(UPrimitiveComponent* OverlappedComp, AActor* Other, UPrimitiveComponent* OtherComp, int32 OtherBodyIndex, bool bFromSweep, const FHitResult& SweepResult) { }

	// --------------------------------------------------------------------------------
	// NP Driver
	// --------------------------------------------------------------------------------

	// Get latest local input prior to simulation step. Called by backend system on owner's instance (autonomous or authority).
	MOVER_API virtual void ProduceInput(const int32 DeltaTimeMS, FMoverInputCmdContext* Cmd);

	// Take output for simulation. Called by backend system.
	MOVER_API void FinalizeFrame(const FMoverSyncState* SyncState, const FMoverAuxStateContext* AuxState);

	// Take output for simulation when no simulation state changes have occurred. Called by backend system.
	MOVER_API void FinalizeUnchangedFrame();

	// Take smoothed simulation state. Called by backend system, if supported.
	MOVER_API void FinalizeSmoothingFrame(const FMoverSyncState* SyncState, const FMoverAuxStateContext* AuxState);

	// This is an opportunity to run code on the code on the simproxy in interpolated mode - currently used to help activate and deactivate modifiers on the simproxy in interpolated mode
	MOVER_API void TickInterpolatedSimProxy(const FMoverTimeStep& TimeStep, const FMoverInputCmdContext& InputCmd, UMoverComponent* MoverComp, const FMoverSyncState& CachedSyncState, const FMoverSyncState& SyncState, const FMoverAuxStateContext& AuxState);
	
	// Seed initial values based on component's state. Called by backend system.
	MOVER_API void OnSimulationStateInitialized(const FMoverSyncState& InitialSyncState, const FMoverAuxStateContext& InitialAuxState);


	// Specifies which supporting back end class should drive this Mover actor
	UPROPERTY(EditDefaultsOnly, Category = Mover, meta = (MustImplement = "/Script/Mover.MoverBackendLiaisonInterface"))
	TSubclassOf<UActorComponent> BackendClass;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Instanced, Category = Mover, meta=(FullyExpand=true))
	TMap<FName, TObjectPtr<UBaseMovementMode>> MovementModes;

	// Name of the first mode to start in when simulation begins. Must have a mapping in MovementModes. Only used during initialization.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Mover, meta=(GetOptions=GetStartingMovementModeNames))
	FName StartingMovementMode = NAME_None;

	// Transition checks that are always evaluated regardless of mode. Evaluated in order, stopping at the first successful transition check. Mode-owned transitions take precedence. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Instanced, Category=Mover)
	TArray<TObjectPtr<UBaseMovementModeTransition>> Transitions;

	/** List of types that should always be present in this actor's sync state */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Mover)
	TArray<FMoverDataPersistence> PersistentSyncStateDataTypes;

	/** Optional object for producing input cmds. Typically set at BeginPlay time. If not specified, defaulted input will be used.
	*   Note that any other actor component implementing MoverInputProducerInterface on this component's owner will also be able
	*   to produce input commands if bGatherInputFromAllInputProducerComponents is true. @see bGatherInputFromAllInputProducerComponents
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover, meta = (ObjectMustImplement = "/Script/Mover.MoverInputProducerInterface"))
	TObjectPtr<UObject> InputProducer;

	/** If true, any actor component implementing MoverInputProducerInterface on this component's owner will be able to produce input commands */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Mover)
	bool bGatherInputFromAllInputProducerComponents = true;

	/* All MoverInputProducerInterface objects producing input for this mover component. If bGatherInputFromAllInputProducerComponents
	*  is true, all components implementing MoverInputProducerInterface on this component's owner will be added to 
	*  this array at BeginPlay time, and IMoverInputProducerInterface::ProduceInput will be called on each within UMoverComponent::ProduceInput.
	*  The order shouldn't matter, as this is for input commands independent of each other, driving different movement modes.
	*  If order is important, set bGatherInputFromAllInputProducerComponents to false and implement a dedicated input component instead,
	*  gathering input from different sources in a custom order and set it as the InputProducer.
	*/
	UPROPERTY(Transient)
	TArray<TObjectPtr<UObject>> InputProducers;

	/** Optional object for mixing proposed moves.Typically set at BeginPlay time. If not specified, UDefaultMovementMixer will be used. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	TObjectPtr<UMovementMixer> MovementMixer;

	MOVER_API const TArray<TObjectPtr<ULayeredMoveLogic>>* GetRegisteredMoves() const;

	// Flushes MovesPendingRegistration and MovesPendingUnregistration into RegisteredMoves.
	// Extracted so async backends (e.g. ChaosMover) can drive registration without calling
	// the full PreSimulationTick, which is intended for the synchronous path only.
	MOVER_API void FlushPendingMoveRegistrations();
	
	/** Registers layered move logic */
	template <typename MoveT UE_REQUIRES(std::is_base_of_v<ULayeredMoveLogic, MoveT>)>
	void RegisterMove(TSubclassOf<MoveT> MoveClass = MoveT::StaticClass())
	{
		K2_RegisterMove(MoveClass);
	}

	/** Registers layered move logic */
	UFUNCTION(BlueprintCallable, Category = Mover, DisplayName = "Register Move")
	void K2_RegisterMove(TSubclassOf<ULayeredMoveLogic> MoveClass);

	/** Registers an array of layered move logic classes */
	UFUNCTION(BlueprintCallable, Category = Mover, DisplayName = "Register Moves")
	void K2_RegisterMoves(TArray<TSubclassOf<ULayeredMoveLogic>> MoveClasses);
	
	/** Unregisters layered move logic */
	template <typename MoveT UE_REQUIRES(std::is_base_of_v<ULayeredMoveLogic, MoveT>)>
	void UnregisterMove(TSubclassOf<MoveT> MoveClass = MoveT::StaticClass())
	{
		K2_UnregisterMove(MoveClass);
	}

	/** Unregisters layered move logic */
	UFUNCTION(BlueprintCallable, Category = Mover, DisplayName = "Unregister Move")
	void K2_UnregisterMove(TSubclassOf<ULayeredMoveLogic> MoveClass);
	
	template <typename MoveT, typename ActivationParamsT UE_REQUIRES(std::is_base_of_v<ULayeredMoveLogic, MoveT> && std::is_base_of_v<typename MoveT::MoveDataType::ActivationParamsType, ActivationParamsT>)>
	bool QueueLayeredMoveActivationWithContext(const ActivationParamsT& ActivationParams, TSubclassOf<MoveT> MoveClass = MoveT::StaticClass())
	{
		return MakeAndQueueLayeredMove(MoveClass, &ActivationParams);
	}
	
	/**
	 * Queues a layered move for activation.
	 * Takes a Activation Context which provides context to set Layered Move Data.
	 * Make sure Activation Context type matches layered Move Data
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = Mover, meta = (CustomStructureParam = "MoveAsRawData", AllowAbstract = "false"), DisplayName = "Queue Layered Move Activation With Context")
	bool K2_QueueLayeredMoveActivationWithContext(TSubclassOf<ULayeredMoveLogic> MoveLogicClass, UPARAM(DisplayName="Layered Move Activation Context") const int32& MoveAsRawData);
	DECLARE_FUNCTION(execK2_QueueLayeredMoveActivationWithContext);

	/**
 	 * Queues a layered move for activation.
 	 * Takes NO Activation Context meaning the layered move will be activated using default Move Data.
 	 * Note: Changing Move Data is still possible in the layered move logic itself
 	 * See QueueLayeredMoveActivationWithContext for activating a layered move with context
 	 */
	UFUNCTION(BlueprintCallable, Category = Mover, meta = (AllowAbstract = "false"))
	bool QueueLayeredMoveActivation(TSubclassOf<ULayeredMoveLogic> MoveLogicClass);
	
	/**
	 * Queue a layered move to start during the next simulation frame. This will clone whatever move you pass in, so you'll need to fully set it up before queuing.
	 * @param LayeredMove			The move to queue, which must be a LayeredMoveBase sub-type. 
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = Mover, meta = (CustomStructureParam = "MoveAsRawData", AllowAbstract = "false", DisplayName = "Queue Layered Move"))
	MOVER_API void K2_QueueLayeredMove(UPARAM(DisplayName="Layered Move") const int32& MoveAsRawData);
	DECLARE_FUNCTION(execK2_QueueLayeredMove);

	// Queue a layered move to start during the next simulation frame
	MOVER_API void QueueLayeredMove(TSharedPtr<FLayeredMoveBase> Move);

	/**
  	 * Advance Usage. This will only become relevant to Mover in general when Layered Moves are automatically networked (pending, only Chaos Mover networks them by default)
	 * Queue a scheduled move to take place after delay (tunable in the NetworkPhysicsSettingsComponent)
	 * ensuring it gets executed on the same frame on all networked end points. @see UNetworkPhysicsSettingsComponent, @see FNetworkPhysicsSettings
	 * Examples:
	 * 1. For an Autonomous Proxy Actor: An autonomous proxy might want to schedule a move that can be dodged by other players, for instance and air stomp, so that it starts executing on the same frame for sim proxies
	 *    giving them a fair chance to dodge it. The server might also need to schedule 
	 * 2. For an actor moved on the Server authoritatively. Since the server runs behind clients, if it wants to avoid corrections it needs to schedule execution of moves in the future so they get
	 *    executed on the same frame on all end points. For instance, a platform changing direction as a result of some unpredictable event (such as colliding with a player character) 
	 *    should be scheduled if we want to guarantee correction-free physical interactions with that platform
	 * 
	 * Pros: Everybody executes on the same frame (or at roughly the same time in variable dt), less correction, no surprises.
     * Cons: Delay in execution, needs to be masked with anticipation anim. Significant lag (several frames).
	 * 
	 * Default to using QueueLayeredMove instead, except for advanced use cases where it is important the move get executed on the same frame on all end points.
	 * 
	 * @param Move			The layered move to queue, which must be a FLayeredMoveBase sub - type.
	 */
	MOVER_API void ScheduleLayeredMove(TSharedPtr<FLayeredMoveBase> Move);

protected:
	// Queue a scheduled layered move. Thread safe, can be used outside the game thread.
	void QueueScheduledLayeredMove(const FScheduledLayeredMove& ScheduledLayeredMove);

public:
	
	/**
 	 * Queue a Movement Modifier to start during the next simulation frame. This will clone whatever move you pass in, so you'll need to fully set it up before queuing.
 	 * @param MovementModifier The modifier to queue, which must be a LayeredMoveBase sub-type.
 	 * @return Returns a Modifier handle that can be used to query or cancel the movement modifier
 	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = Mover, meta = (CustomStructureParam = "MoveAsRawData", AllowAbstract = "false", DisplayName = "Queue Movement Modifier"))
	MOVER_API FMovementModifierHandle K2_QueueMovementModifier(UPARAM(DisplayName="Movement Modifier") const int32& MoveAsRawData);
	DECLARE_FUNCTION(execK2_QueueMovementModifier);

	// Queue a Movement Modifier to start during the next simulation frame.
	MOVER_API FMovementModifierHandle QueueMovementModifier(TSharedPtr<FMovementModifierBase> Modifier);

	/**
	 * Cancel any active or queued Modifiers with the handle passed in.
	 */
	UFUNCTION(BlueprintCallable, Category = Mover)
	MOVER_API void CancelModifierFromHandle(FMovementModifierHandle ModifierHandle);

	/**
	 * Cancel any active or queued movement features (layered moves, modifiers, etc.) that have a matching gameplay tag. Does not affect the active movement mode.
	 */
	UFUNCTION(BlueprintCallable, Category = Mover)
	MOVER_API void CancelFeaturesWithTag(FGameplayTag TagToCancel, bool bRequireExactMatch=false);

	/**
	 * Check any active or queued movement features (layered moves, modifiers, etc.) for a matching gameplay tag.
	 */
	UFUNCTION(BlueprintCallable, Category = Mover)
	MOVER_API bool HasFeaturesWithTag(FGameplayTag TagToCancel, bool bRequireExactMatch = false) const;

	/**
	 * Queue an Instant Movement Effect to start at the end of this frame or start of the next subtick - whichever happens first. This will clone whatever move you pass in, so you'll need to fully set it up before queuing.
	 * @param InstantMovementEffect			The effect to queue, which must be a FInstantMovementEffect sub-type. 
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = Mover, meta = (CustomStructureParam = "EffectAsRawData", AllowAbstract = "false", DisplayName = "Queue Instant Movement Effect"))
	MOVER_API void K2_QueueInstantMovementEffect(UPARAM(DisplayName="Instant Movement Effect") const int32& EffectAsRawData);
	DECLARE_FUNCTION(execK2_QueueInstantMovementEffect);

	/**
	 * Schedule an Instant Movement Effect to be applied as early as possible while ensuring it gets executed on the same frame on all networked end points.
	 * This adds a delay to the application of the effect, tunable in the NetworkPhysicsSettingsComponent. @see UNetworkPhysicsSettingsComponent, @see FNetworkPhysicsSettings, @see EventSchedulingMinDelaySeconds
	 * This will clone whatever move you pass in, so you'll need to fully set it up before queuing.
	 * @param InstantMovementEffect			The effect to queue, which must be a FInstantMovementEffect sub-type. 
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = Mover, meta = (CustomStructureParam = "EffectAsRawData", AllowAbstract = "false", DisplayName = "Schedule Instant Movement Effect"))
	MOVER_API void K2_ScheduleInstantMovementEffect(UPARAM(DisplayName="Instant Movement Effect") const int32& EffectAsRawData);
	DECLARE_FUNCTION(execK2_ScheduleInstantMovementEffect);

	/**
	 *  Queue a Instant Movement Effect to take place at the end of this frame or start of the next subtick - whichever happens first
	 *  @param InstantMovementEffect			The effect to queue, which must be a FInstantMovementEffect sub - type.
	 */ 
	MOVER_API void QueueInstantMovementEffect(TSharedPtr<FInstantMovementEffect> InstantMovementEffect);
	/** 
	 * Advance Usage. This will only become relevant to Mover in general when Instant Movement Effects are automatically networked (pending, only Chaos Mover networks them by default)
	 * Queue a scheduled Instant Movement Effect to take place after delay (tunable in the NetworkPhysicsSettingsComponent)
	 * ensuring it gets executed on the same frame on all networked end points. @see UNetworkPhysicsSettingsComponent, @see FNetworkPhysicsSettings
	 * Default to using QueueInstantMovementEffect instead, except for advanced use cases where it is important the effect get executed on the same frame on all end points.
	 * @param InstantMovementEffect			The effect to queue, which must be a FInstantMovementEffect sub - type.
	 */
	MOVER_API void ScheduleInstantMovementEffect(TSharedPtr<FInstantMovementEffect> InstantMovementEffect);

	// Queue an instant movement effect in async mode. Do not use on the game thread.
	void QueueInstantMovementEffect_Internal(const FMoverTimeStep& TimeStep, TSharedPtr<FInstantMovementEffect> InstantMovementEffect);

protected:
	// Queue a scheduled instant movement effect. Thread safe, can be used outside the game thread.
	void QueueScheduledInstantMovementEffect(const FScheduledInstantMovementEffect& ScheduledInstantMovementEffect);

public:	
	// Queue a movement mode change to occur during the next simulation frame. If bShouldReenter is true, then a mode change will occur even if already in that mode.
	UFUNCTION(BlueprintCallable, Category = Mover, DisplayName="Queue Next Movement Mode")
	MOVER_API void QueueNextMode(FName DesiredModeName, bool bShouldReenter=false);

	// Add a movement mode to available movement modes. Returns true if the movement mode was added successfully. Returns the mode that was made.
	UFUNCTION(BlueprintCallable, Category = Mover, meta=(DeterminesOutputType="MovementMode"))
	MOVER_API UBaseMovementMode* AddMovementModeFromClass(FName ModeName, UPARAM(meta = (AllowAbstract = "false"))TSubclassOf<UBaseMovementMode> MovementMode);

	// Add a movement mode to available movement modes. Returns true if the movement mode was added successfully
	UFUNCTION(BlueprintCallable, Category = Mover)
	MOVER_API bool AddMovementModeFromObject(FName ModeName, UBaseMovementMode* MovementMode);
	
	// Removes a movement mode from available movement modes. Returns number of modes removed from the available movement modes.
	UFUNCTION(BlueprintCallable, Category = Mover)
	MOVER_API bool RemoveMovementMode(FName ModeName);
	
public:
	// Set gravity override, as a directional acceleration in worldspace.  Gravity on Earth would be {x=0,y=0,z=-980}
	UFUNCTION(BlueprintCallable, Category = Mover)
	MOVER_API void SetGravityOverride(bool bOverrideGravity, FVector GravityAcceleration=FVector::ZeroVector);
	
	// Get the current acceleration due to gravity (cm/s^2) in worldspace
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = Mover)
	MOVER_API FVector GetGravityAcceleration() const;

	/** Returns a quaternion transforming from world to gravity space. */
	FQuat GetWorldToGravityTransform() const { return WorldToGravityTransform; }

	/** Returns a quaternion transforming from gravity to world space. */
	FQuat GetGravityToWorldTransform() const { return GravityToWorldTransform; }

	// Set UpDirection override. This is a fixed direction that overrides the gravity-derived up direction.
	UFUNCTION(BlueprintCallable, Category = Mover)
	MOVER_API void SetUpDirectionOverride(bool bOverrideUpDirection, FVector UpDirection=FVector::UpVector);

	// Get the normalized direction considered "up" in worldspace. Typically aligned with gravity, and typically determines the plane an actor tries to move along.
	UFUNCTION(BlueprintPure = false, Category = Mover)
	MOVER_API FVector GetUpDirection() const;

	// Access the planar constraint that may be limiting movement direction
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = Mover)
	MOVER_API const FPlanarConstraint& GetPlanarConstraint() const;

	// Sets planar constraint that can limit movement direction
	UFUNCTION(BlueprintCallable, Category = Mover)
	MOVER_API void SetPlanarConstraint(const FPlanarConstraint& InConstraint);

	// Sets BaseVisualComponentTransform used for cases where we want to move the visual component away from the root component. See @BaseVisualComponentTransform
	UFUNCTION(BlueprintCallable, Category = Mover)
	MOVER_API void SetBaseVisualComponentTransform (const FTransform& ComponentTransform);

	// Gets BaseVisualComponentTransform used for cases where we want to move the visual component away from the root component. See @BaseVisualComponentTransform
	UFUNCTION(BlueprintCallable, Category = Mover)
	MOVER_API FTransform GetBaseVisualComponentTransform() const;

	/** Sets whether this mover component can use grouped movement updates, which improve performance but can cause attachments to update later than expected */
	UFUNCTION(BlueprintCallable, Category = Mover)
	MOVER_API void SetUseDeferredGroupMovement(bool bEnable);

	/** Returns true if this component is actually using grouped movement updates, which checks the flag and any global settings */
	UFUNCTION(BlueprintCallable, Category = Mover)
	MOVER_API bool IsUsingDeferredGroupMovement() const;
	
public:

	/**
	 *  Converts a local root motion transform to worldspace. 
	 * @param AlternateActorToWorld   allows specification of a different actor root transform, for cases when root motion isn't directly being applied to this actor (async simulations)
	 * @param OptionalWarpingContext   allows specification of a warping context, for use with root motion that is asynchronous from the actor (async simulations)
	 */
	MOVER_API virtual FTransform ConvertLocalRootMotionToWorld(const FTransform& LocalRootMotionTransform, float DeltaSeconds, const FTransform* AlternateActorToWorld=nullptr, const FMotionWarpingUpdateContext* OptionalWarpingContext=nullptr) const;

	/** delegates used when converting local root motion to worldspace, allowing external systems to influence it (such as motion warping) */
	FOnWarpLocalspaceRootMotionWithContext ProcessLocalRootMotionDelegate;
	FOnWarpWorldspaceRootMotionWithContext ProcessWorldRootMotionDelegate;

public:	// Queries

	// Get the transform of the root component that our Mover simulation is moving
	MOVER_API FTransform GetUpdatedComponentTransform() const;

	// Sets which component we're using as the root of our movement
	UFUNCTION(BlueprintCallable, Category = Mover)
	MOVER_API void SetUpdatedComponent(USceneComponent* NewUpdatedComponent);

	// Access the root component of the actor that our Mover simulation is moving
	UFUNCTION(BlueprintCallable, Category = Mover)
	MOVER_API USceneComponent* GetUpdatedComponent() const;

	// Typed accessor to root moving component
	template<class T>
	T* GetUpdatedComponent() const
	{
		static_assert(TPointerIsConvertibleFromTo<T, const USceneComponent>::Value, "'T' template parameter to GetUpdatedComponent must be derived from USceneComponent");
		return Cast<T>(GetUpdatedComponent());
	}

	// Access the primary visual component of the actor
	UFUNCTION(BlueprintCallable, Category = Mover)
	MOVER_API USceneComponent* GetPrimaryVisualComponent() const;

	// Typed accessor to primary visual component
	template<class T>
	T* GetPrimaryVisualComponent() const
	{
		return Cast<T>(GetPrimaryVisualComponent());
	}

	// Sets this Mover actor's primary visual component. Must be a descendant of the updated component that acts as our movement root. 
	UFUNCTION(BlueprintCallable, Category=Mover)
	MOVER_API void SetPrimaryVisualComponent(USceneComponent* SceneComponent);

	// Get the current velocity (units per second, worldspace)
	UFUNCTION(BlueprintPure, Category = Mover)
	MOVER_API virtual FVector GetVelocity() const override;

	// Get the intended movement direction in worldspace with magnitude (range 0-1)
	UFUNCTION(BlueprintPure, Category = Mover)
	MOVER_API FVector GetMovementIntent() const;

	// Get the orientation that the actor is moving towards
	UFUNCTION(BlueprintPure, Category = Mover)
	MOVER_API FRotator GetTargetOrientation() const;

	/** Get a sampling of where the actor is projected to be in the future, based on a current state. Note that this is projecting ideal movement without doing full simulation and collision. */
	UE_DEPRECATED(5.5, "Use GetPredictedTrajectory instead.")
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = Mover)
	MOVER_API TArray<FTrajectorySampleInfo> GetFutureTrajectory(float FutureSeconds, float SamplesPerSecond);

	/** Get a sampling of where the actor is projected to be in the future, based on a current state. Note that this is projecting ideal movement without doing full simulation and collision.
	 * The first sample info of the returned array corresponds to the current state of the mover. */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = Mover)
	MOVER_API virtual TArray<FTrajectorySampleInfo> GetPredictedTrajectory(FMoverPredictTrajectoryParams PredictionParams);

	// Get the current movement mode name
	UFUNCTION(BlueprintPure, Category = Mover)
	MOVER_API FName GetMovementModeName() const;

	// Gets the name of the next movement mode, if a change is queued. Returns the current mode name otherwise.
	UFUNCTION(BlueprintPure, Category = Mover)
	MOVER_API virtual FName GetNextMovementModeName() const;

	// Get the current movement mode 
	UFUNCTION(BlueprintPure, Category = Mover)
	MOVER_API const UBaseMovementMode* GetMovementMode() const;

	// Get the current movement base. Null if there isn't one.
	UFUNCTION(BlueprintPure, Category = Mover)
	MOVER_API UPrimitiveComponent* GetMovementBase() const;

	// Get the current movement base bone, NAME_None if there isn't one.
	UFUNCTION(BlueprintPure, Category = Mover)
	MOVER_API FName GetMovementBaseBoneName() const;

	// Signals whether we have a sync state saved yet. If not, most queries will not be meaningful.
	UE_DEPRECATED(5.6, "HasValidCachedState has been deprecated, and is not needed since we no longer wait until movement simulation begins before providing a valid sync state.")
	UFUNCTION(BlueprintPure, Category = Mover, meta=(DeprecatedFunction, DeprecationMessage="HasValidCachedState has been deprecated, and is not needed since we no longer wait until movement simulation begins before providing a valid sync state."))
	MOVER_API bool HasValidCachedState() const;

	// Access the most recent captured sync state.
	UFUNCTION(BlueprintPure, Category = Mover)
	MOVER_API const FMoverSyncState& GetSyncState() const;

	// Signals whether we have input data saved yet. If not, input queries will not be meaningful.
	UE_DEPRECATED(5.6, "HasValidCachedInputCmd has been deprecated, and is not needed since we no longer wait until movement simulation begins before providing a valid input cmd.")
	UFUNCTION(BlueprintPure, Category = Mover, meta = (DeprecatedFunction, DeprecationMessage = "HasValidCachedInputCmd has been deprecated, and is not needed since we no longer wait until movement simulation begins before providing a valid input cmd."))
	MOVER_API bool HasValidCachedInputCmd() const;

	// Access the most recently-used inputs.
	UFUNCTION(BlueprintPure, Category = Mover)
	MOVER_API const FMoverInputCmdContext& GetLastInputCmd() const;

	// Get the most recent TimeStep
	UFUNCTION(BlueprintPure, Category = Mover)
	MOVER_API const FMoverTimeStep& GetLastTimeStep() const;

	// Access the most recent floor check hit result.
	UFUNCTION(BlueprintPure, Category = Mover)
	MOVER_API virtual bool TryGetFloorCheckHitResult(FHitResult& OutHitResult) const;

	// Access the read-only version of the Mover's Blackboard
	UFUNCTION(BlueprintPure, Category=Mover)
	MOVER_API const UMoverBlackboard* GetSimBlackboard() const;

	MOVER_API UMoverBlackboard* GetSimBlackboard_Mutable() const;

	// Access the version of the movement blackboard for users outside of the movement simulation
	FRollbackBlackboardExternalWrapper GetRollbackBlackboardExternal() const { return RollbackBlackboardExternalWrapper; }

	/** Find settings object by type. Returns null if there is none of that type */
	const IMovementSettingsInterface* FindSharedSettings(const UClass* ByType) const { return FindSharedSettings_Mutable(ByType); }
	template<typename SettingsT = IMovementSettingsInterface UE_REQUIRES(std::is_base_of_v<IMovementSettingsInterface, SettingsT>)>
	const SettingsT* FindSharedSettings() const { return Cast<const SettingsT>(FindSharedSettings(SettingsT::StaticClass())); }

	/** Find mutable settings object by type. Returns null if there is none of that type */
	MOVER_API IMovementSettingsInterface* FindSharedSettings_Mutable(const UClass* ByType) const;
	template<typename SettingsT = IMovementSettingsInterface UE_REQUIRES(std::is_base_of_v<IMovementSettingsInterface, SettingsT>)>
	SettingsT* FindSharedSettings_Mutable() const { return Cast<SettingsT>(FindSharedSettings_Mutable(SettingsT::StaticClass())); }

	/** Find mutable settings object by type. Returns null if there is none of that type */
	UFUNCTION(BlueprintPure, Category = Mover,  meta=(DeterminesOutputType="SharedSetting", DisplayName="Find Shared Settings Mutable"))
	MOVER_API UObject* FindSharedSettings_Mutable_BP(UPARAM(meta = (MustImplement = "MovementSettingsInterface")) TSubclassOf<UObject> SharedSetting) const;

	/** Find settings object by type. Returns null if there is none of that type */
	UFUNCTION(BlueprintPure, Category = Mover,  meta=(DeterminesOutputType="SharedSetting", DisplayName="Find Shared Settings"))
	MOVER_API const UObject* FindSharedSettings_BP(UPARAM(meta = (MustImplement = "MovementSettingsInterface")) TSubclassOf<UObject> SharedSetting) const;

	/** Gets the currently active movement mode, provided it is of the given type. Returns nullptr if there is no active mode yet, or if it's of a different type. */
	template<typename ModeT = UBaseMovementMode UE_REQUIRES(std::is_base_of_v<UBaseMovementMode, ModeT>)>
	const ModeT* GetActiveMode(bool bRequireExactClass = false) const { return Cast<ModeT>(GetActiveModeInternal(ModeT::StaticClass(), bRequireExactClass)); }

	/** Gets the currently active movement mode, provided it is of the given type. Returns nullptr if there is no active mode yet, or if it's of a different type. */
	template<typename ModeT = UBaseMovementMode UE_REQUIRES(std::is_base_of_v<UBaseMovementMode, ModeT>)>
	ModeT* GetActiveMode_Mutable(bool bRequireExactClass = false) const { return Cast<ModeT>(GetActiveModeInternal(ModeT::StaticClass(), bRequireExactClass)); }

	/** Find the first movement mode on this component with the given type, optionally of the given type exactly. Returns null if there is none of that type */
	template<typename ModeT = UBaseMovementMode UE_REQUIRES(std::is_base_of_v<UBaseMovementMode, ModeT>)>
	ModeT* FindMode_Mutable(bool bRequireExactClass = false) const { return Cast<ModeT>(FindMode_Mutable(ModeT::StaticClass(), bRequireExactClass)); }
	MOVER_API UBaseMovementMode* FindMode_Mutable(TSubclassOf<UBaseMovementMode> ModeType, bool bRequireExactClass = false) const;

	/** Find the movement mode on this component the given name and type, optionally of the given type exactly. Returns null if there is no mode by that name, or if it's of a different type. */
	template<typename ModeT = UBaseMovementMode UE_REQUIRES(std::is_base_of_v<UBaseMovementMode, ModeT>)>
	ModeT* FindMode_Mutable(FName MovementModeName, bool bRequireExactClass = false) const { return Cast<ModeT>(FindMode_Mutable(ModeT::StaticClass(), MovementModeName, bRequireExactClass)); }
	MOVER_API UBaseMovementMode* FindMode_Mutable(TSubclassOf<UBaseMovementMode> ModeType, FName ModeName, bool bRequireExactClass = false) const;
	
	UFUNCTION(BlueprintPure, Category = Mover,  meta=(DeterminesOutputType="MovementMode"))
	MOVER_API UBaseMovementMode* FindMovementMode(TSubclassOf<UBaseMovementMode> MovementMode) const;

	UFUNCTION(BlueprintPure, Category = Mover)
	MOVER_API UBaseMovementMode* FindMovementModeByName(FName MovementModeName) const;
	
	/**
	 * Retrieves an active layered move, by writing to a target instance if it is the matching type. Note: Writing to the struct returned will not modify the active struct.
	 * @param DidSucceed			Flag indicating whether data was actually written to target struct instance
	 * @param TargetAsRawBytes		The data struct instance to write to, which must be a FLayeredMoveBase sub-type
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = Mover, meta = (CustomStructureParam = "TargetAsRawBytes", AllowAbstract = "false", DisplayName = "Find Active Layered Move"))
	MOVER_API void K2_FindActiveLayeredMove(bool& DidSucceed, UPARAM(DisplayName = "Out Layered Move") int32& TargetAsRawBytes) const;
	DECLARE_FUNCTION(execK2_FindActiveLayeredMove);

	// Find an active layered move by type. Returns null if one wasn't found 
	MOVER_API const FLayeredMoveBase* FindActiveLayeredMoveByType(const UScriptStruct* DataStructType) const;

	/** Find a layered move of a specific type in this components active layered moves. If not found, null will be returned. */
	template <typename MoveT = FLayeredMoveBase UE_REQUIRES(std::is_base_of_v<FLayeredMoveBase, MoveT>)>
	const MoveT* FindActiveLayeredMoveByType() const { return static_cast<const MoveT*>(FindActiveLayeredMoveByType(MoveT::StaticStruct())); }
	
	/**
	 * Retrieves Movement modifier by writing to a target instance if it is the matching type. Note: Writing to the struct returned will not modify the active struct.
	 * @param ModifierHandle		Handle of the modifier we're trying to cancel
	 * @param bFoundModifier		Flag indicating whether modifier was found and data was actually written to target struct instance
	 * @param TargetAsRawBytes		The data struct instance to write to, which must be a FMovementModifierBase sub-type
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = Mover, meta = (CustomStructureParam = "TargetAsRawBytes", AllowAbstract = "false", DisplayName = "Find Movement Modifier"))
	MOVER_API void K2_FindMovementModifier(FMovementModifierHandle ModifierHandle, bool& bFoundModifier, UPARAM(DisplayName = "Out Movement Modifier") int32& TargetAsRawBytes) const;
	DECLARE_FUNCTION(execK2_FindMovementModifier);

	// Checks if the modifier handle passed in is active or queued on this mover component
	UFUNCTION(BlueprintPure, Category = Mover)
	MOVER_API bool IsModifierActiveOrQueued(const FMovementModifierHandle& ModifierHandle) const;
	
	// Find movement modifier by it's handle. Returns nullptr if the modifier couldn't be found
	MOVER_API const FMovementModifierBase* FindMovementModifier(const FMovementModifierHandle& ModifierHandle) const;

	// Find movement modifier by type (returns the first modifier it finds). Returns nullptr if the modifier couldn't be found
	MOVER_API const FMovementModifierBase* FindMovementModifierByType(const UScriptStruct* DataStructType) const;
	
	/** Find a movement modifier of a specific type in this components movement modifiers. If not found, null will be returned. */
	template <typename ModifierT = FMovementModifierBase UE_REQUIRES(std::is_base_of_v<FMovementModifierBase, ModifierT>)>
	const ModifierT* FindMovementModifierByType() const { return static_cast<const ModifierT*>(FindMovementModifierByType(ModifierT::StaticStruct())); }
	
	// Returns true if the current backend runs the simulation asynchronously on the worker thread.
	// Use this to decide whether simulation output (e.g. montage position) is the sole authority
	// or whether the game thread is also advancing state independently.
	UFUNCTION(BlueprintPure, Category = Mover)
	bool IsBackendAsync() const { return BackendLiaisonComp && BackendLiaisonComp->IsAsync(); }

	// Get the gameplay tags currently active, combining simulation-reported tags and externally-added tags.
	// In the async backend (ChaosMover), or when mover.UseSnapshotGameplayTagsForSyncBackend is set,
	// returns the cached AllGameplayTags snapshot. Otherwise performs a live walk of the current sync state.
	UFUNCTION(BlueprintPure, Category = Mover)
	MOVER_API FGameplayTagContainer GetGameplayTags() const;

	/**
 	 * Check Mover systems for a gameplay tag.
 	 *
 	 * @param TagToFind			Tag to check on the Mover systems
 	 * @param bExactMatch		If true, the tag has to be exactly present, if false then TagToFind will include it's parent tags while matching
 	 *
 	 * @return True if the TagToFind was found
 	 */
	UFUNCTION(BlueprintPure, Category = Mover, meta = (Keywords = "HasTag"))
	MOVER_API bool HasGameplayTag(FGameplayTag TagToFind, bool bExactMatch) const;

	/**
	 * Check Mover systems for a gameplay tag. Use the given state, as well as any loose tags on the MoverComponent.
	 *
	 * @param TagToFind			Tag to check on the MoverComponent or state
	 * @param bExactMatch		If true, the tag has to be exactly present, if false then TagToFind will include it's parent tags while matching
	 *
	 * @return True if the TagToFind was found
	 */
	UFUNCTION(BlueprintPure, Category = Mover, meta = (Keywords = "HasTag"))
	MOVER_API bool HasGameplayTagInState(const FMoverSyncState& SyncState, FGameplayTag TagToFind, bool bExactMatch) const;

	/**
  	 * Adds a gameplay tag to this Mover Component.
  	 * Note: Duplicate tags will not be added
  	 * @param TagToAdd			Tag to add to the Mover Component
  	 */
	UFUNCTION(BlueprintCallable, Category = Mover, meta = (Keywords = "Add Tag"))
	MOVER_API void AddGameplayTag(FGameplayTag TagToAdd);

	/**
   	 * Adds a series of gameplay tags to this Mover Component
   	 * Note: Duplicate tags will not be added
   	 * @param TagsToAdd			Tags to add/append to the Mover Component
   	 */
	UFUNCTION(BlueprintCallable, Category = Mover, meta = (Keywords = "Add Tag"))
	MOVER_API void AddGameplayTags(const FGameplayTagContainer& TagsToAdd);
	
	/**
   	 * Removes a gameplay tag from this Mover Component
   	 * @param TagToRemove			Tag to remove from the Mover Component
   	 */
	UFUNCTION(BlueprintCallable, Category = Mover, meta = (Keywords = "Remove Tag"))
	MOVER_API void RemoveGameplayTag(FGameplayTag TagToRemove);

	/**
	 * Removes gameplay tags from this Mover Component
	 * @param TagsToRemove			Tags to remove from the Mover Component
	 */
	UFUNCTION(BlueprintCallable, Category = Mover, meta = (Keywords = "Remove Tag"))
	MOVER_API void RemoveGameplayTags(const FGameplayTagContainer& TagsToRemove);
	
	// IMovementInterface
	MOVER_API virtual UObject* GetMovementOwner() const override;
	MOVER_API virtual UObject* GetUpdatedObject() const override;
	MOVER_API virtual FTransform GetTransform() const override;
	MOVER_API virtual void RequestStopMovement() override;
	
public: 
	// Functions that are driven by external systems as the movement simulation advances.
	
	// Called before each simulation tick. Broadcasts OnPreSimulationTick delegate.
	MOVER_API virtual void OnPreSimulate(const FMoverTimeStep& TimeStep, const FMoverTickStartData& StartingData);
	
	// Called after each simulation tick. Broadcasts OnPostMovement and OnPostSimulationTick delegates.
	MOVER_API virtual void OnPostSimulate(const FMoverTimeStep& TimeStep, const FMoverTickStartData& StartingData, OUT FMoverTickEndData& EndingData);

	// Called when a rollback occurs, with the simulation state that is being restored. NewBaseTimeStep represents the new time and frame we'll be resimulating next.
	MOVER_API void OnSimulationRollback(const FMoverSyncState* SyncState, const FMoverAuxStateContext* AuxState, const FMoverTimeStep& NewBaseTimeStep);

protected:
	/** Makes this component and owner actor reflect the state of a particular frame snapshot. This occurs after simulation ticking, as well as during a rollback before we resimulate forward.
	 * @param bDueToRollback	Signifies if this is a forward-advancing frame or rollback. Affects how transform changes are applied.
	 */
	MOVER_API void SetFrameStateFromContext(const FMoverSyncState* SyncState, const FMoverAuxStateContext* AuxState, bool bDueToRollback);

	/** Update cached frame state if it has changed */
	MOVER_API void UpdateCachedFrameState(const FMoverSyncState* SyncState, const FMoverAuxStateContext* AuxState);

public:
	// Author input and state based on the current location of the actor. Typically used at initialization time.
	MOVER_API virtual void GetDefaultInputAndState(FMoverInputCmdContext& OutInputCmd, FMoverSyncState& OutSyncState, FMoverAuxStateContext& OutAuxState) const;

	MOVER_API void SetSimulation(UMoverSimulation& Simulation);		// Temporary support function. Will be removed once simulations are used consistently.

	/** Handle a blocking impact.*/
	UFUNCTION(BlueprintCallable, Category = Mover)
	MOVER_API void HandleImpact(FMoverOnImpactParams& ImpactParams);

protected:
	MOVER_API void FindDefaultUpdatedComponent();
	MOVER_API void FindAndRegisterDefaults();
	MOVER_API void UpdateTickRegistration();

	MOVER_API virtual void DoQueueNextMode(FName DesiredModeName, bool bShouldReenter=false);

	#if WITH_EDITOR
	MOVER_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	MOVER_API virtual void PostCDOCompiled(const FPostCDOCompiledContext& Context) override;
	MOVER_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	MOVER_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
	MOVER_API bool ValidateSetup(class FDataValidationContext& ValidationErrors) const;
	MOVER_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;

	UFUNCTION()
	MOVER_API TArray<FString> GetStartingMovementModeNames();
	#endif // WITH_EDITOR

	UFUNCTION()
	MOVER_API virtual void PhysicsVolumeChanged(class APhysicsVolume* NewVolume);

	MOVER_API virtual void OnHandleImpact(const FMoverOnImpactParams& ImpactParams);

	/** internal function to perform post-sim scheduling to optionally support simple based movement */
	MOVER_API void UpdateBasedMovementScheduling(const FMoverTickEndData& SimOutput);

	MOVER_API UBaseMovementMode* GetActiveModeInternal(TSubclassOf<UBaseMovementMode> ModeType, bool bRequireExactClass = false) const;

	TObjectPtr<UPrimitiveComponent> MovementBaseDependency;	// used internally for based movement scheduling management
	
	/** internal function to ensure SharedSettings array matches what's needed by the list of Movement Modes */
	MOVER_API void RefreshSharedSettings();

	/** This is the component that's actually being moved. Typically it is the Actor's root component and often a collidable primitive. */
	UPROPERTY(Transient)
	TObjectPtr<USceneComponent> UpdatedComponent = nullptr;

	/** UpdatedComponent, cast as a UPrimitiveComponent. May be invalid if UpdatedComponent was null or not a UPrimitiveComponent. */
	UPROPERTY(Transient)
	TObjectPtr<UPrimitiveComponent> UpdatedCompAsPrimitive = nullptr;

	/** The main visual component associated with this Mover actor, typically a mesh and typically parented to the UpdatedComponent. */
	UPROPERTY(Transient)
	TObjectPtr<USceneComponent> PrimaryVisualComponent;


	/** Cached original offset from the visual component, used for cases where we want to move the visual component away from the root component (for smoothing, corrections, etc.) */
	FTransform BaseVisualComponentTransform = FTransform::Identity;

	// TODO: Look at possibility of replacing this with a FGameplayTagCountContainer that could possibly represent both internal and external tags
	/** Gameplay tags currently active on the simulation. */
	FGameplayTagContainer InternalGameplayTags;

	/** A list of gameplay tags associated with this Mover Component added from sources outside of Mover */
	FGameplayTagContainer ExternalGameplayTags;

	/** Combined cache of InternalGameplayTags and ExternalGameplayTags. Used by GetGameplayTags() and HasGameplayTag() in the async backend path, and in the sync backend path when mover.UseSnapshotGameplayTagsForSyncBackend is set. */
	FGameplayTagContainer AllGameplayTags;

	/** Rebuilds AllGameplayTags from InternalGameplayTags and ExternalGameplayTags. Must be called after either source changes. */
	void RebuildAllGameplayTags();

	/** Collects all gameplay tags currently active on the given sync state (movement mode, modifiers, layered moves) into OutTags. */
	void CollectGameplayTagsFromSyncState(const FMoverSyncState& SyncState, FGameplayTagContainer& OutTags) const;

	FMoverInputCmdContext CachedLastProducedInputCmd;
	
	FMoverInputCmdContext CachedLastUsedInputCmd;
	
	FMoverDoubleBuffer<FMoverSyncState> MoverSyncStateDoubleBuffer;
	
	const FMoverDefaultSyncState* LastMoverDefaultSyncState = nullptr;

	FMoverTimeStep CachedLastSimTickTimeStep;	// Saved timestep info from our last simulation tick, used during rollback handling. This will rewind during corrections.
	FMoverTimeStep CachedNewestSimTickTimeStep;	// Saved timestep info from the newest (farthest-advanced) simulation tick. This will not rewind during corrections.

	UPROPERTY(Transient)
	TScriptInterface<IMoverBackendLiaisonInterface> BackendLiaisonComp;

	/** Tick function that may be called anytime after this actor's movement step, useful as a way to support based movement on objects that are not */
	FMoverDynamicBasedMovementTickFunction BasedMovementTickFunction;

	/** Used to store cached data & computations between decoupled systems, that can be referenced by name */
	UPROPERTY(Transient)
	TObjectPtr<UMoverBlackboard> SimBlackboard = nullptr;

	/** This is what all movement mechanisms should be interacting with during simulation, rather than relying on actors/components. External parties should generally not see it. */
	UPROPERTY(Transient)
	TObjectPtr<UMoverSimulation> Simulation = nullptr;

	// Accessors used to interface with the simulation's rollback blackboard
	UPROPERTY(Transient)
	FRollbackBlackboardExternalWrapper RollbackBlackboardExternalWrapper;


	/**
	 * Layered moves registered on this component that can be activated regardless of the current mode
	 * Changes to this array or its contents occur ONLY during PreSimulationTick to ensure threadsafe access during async simulations.
	 */
	UPROPERTY(Transient)
	TArray<TObjectPtr<ULayeredMoveLogic>> RegisteredMoves;

	UPROPERTY(Transient)
	TArray<TSubclassOf<ULayeredMoveLogic>> MovesPendingRegistration;

	UPROPERTY(Transient)
	TArray<TSubclassOf<ULayeredMoveLogic>> MovesPendingUnregistration;
	
	/** Helper function for making Queueing and making Layered Moves */
	bool MakeAndQueueLayeredMove(const TSubclassOf<ULayeredMoveLogic>& MoveLogicClass, const FLayeredMoveActivationParams* ActivationParams);

	/**
	 * Fires OnGameplayTagAdded/OnGameplayTagRemoved for every tag that is in NewTags but not PreviousTags (added),
	 * or in PreviousTags but not NewTags (removed). Tags already covered by an entry in EventsToSkip are suppressed
	 * to avoid double-firing alongside explicit simulation events. Tags also present in ExternalGameplayTags are
	 * suppressed with a log, as concurrent simulation and external ownership of the same tag is unexpected.
	 */
	void DispatchGameplayTagTransitionEvents(const FGameplayTagContainer& PreviousTags, const FGameplayTagContainer& NewTags, const TArray<TSharedPtr<FMoverSimulationEventData>>& EventsToSkip, const FMoverTimeStep& TimeStep);

	/**
	 * Called after applying a new tag snapshot from simulation output. In principle, every queued simulation tag event
	 * should account for every change between PreviousTags and NewTags. If that invariant is violated (i.e. a tag
	 * changed state without a corresponding event), this is a bug - we recover by emitting the missing events and
	 * logging a warning for each discrepancy. Delegates to DispatchGameplayTagTransitionEvents for the actual dispatch.
	 */
	void DispatchUnexpectedTagDriftEvents(const FGameplayTagContainer& PreviousTags, const FGameplayTagContainer& NewTags, const TArray<TSharedPtr<FMoverSimulationEventData>>& Events, const FMoverTimeStep& TimeStep);

private:
	/** Collection of settings objects that are shared between movement modes. This list is automatically managed based on the @MovementModes contents. */
	UPROPERTY(EditDefaultsOnly, EditFixedSize, Instanced, Category = Mover, meta = (NoResetToDefault, ObjectMustImplement = "/Script/Mover.MovementSettingsInterface"))
	TArray<TObjectPtr<UObject>> SharedSettings;
	
	/** cm/s^2, only meaningful if @bHasGravityOverride is enabled.Set @SetGravityOverride */
	UPROPERTY(EditDefaultsOnly, Category="Mover|Gravity", meta=(ForceUnits = "cm/s^2"))
	FVector GravityAccelOverride;

	/** Settings that can lock movement to a particular plane */
	UPROPERTY(EditDefaultsOnly, Category = "Mover|Constraints")
	FPlanarConstraint PlanarConstraint;

public:

	// If enabled, the movement of the primary visual component will be smoothed via an offset from the root moving component. This is useful in fixed-tick simulations with variable rendering rates.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mover")
	EMoverSmoothingMode SmoothingMode = EMoverSmoothingMode::VisualComponentOffset;

	// Whether to warn when we detect that an external system has moved our object, outside of movement simulation control
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mover", AdvancedDisplay)
	uint8 bWarnOnExternalMovement : 1 = 1;

	// If enabled, we'll accept any movements from an external system in the next simulation state update
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mover", AdvancedDisplay)
	uint8 bAcceptExternalMovement : 1 = 0;

	// If enabled, we'll send inputs along with to sim proxy via the sync state, and they'll be available via GetLastInputCmd. This may be useful for cases where input is used to hint at object state, such as an anim graph. This option is intended to be temporary until all networking backends allow this.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mover", AdvancedDisplay, Experimental)
	uint8 bSyncInputsForSimProxy : 1 = 0;

	MOVER_API void SetSimulationOutput(const FMoverTimeStep& TimeStep, const UE::Mover::FSimulationOutputData& OutputData);

	// Dispatch a simulation event. It will be processed immediately.
	MOVER_API void DispatchSimulationEvent(const FMoverSimulationEventData& EventData);

protected:
	MOVER_API virtual void DispatchSimulationEvents(const UE::Mover::FSimulationOutputData& OutputData);
	MOVER_API virtual void ProcessSimulationEvent(const FMoverSimulationEventData& EventData);
	MOVER_API virtual void SetAdditionalSimulationOutput(const FMoverDataCollection& Data);

	// Flush the simulation-authoritative montage state accumulated during SetAdditionalSimulationOutput
	// calls for this frame. Drives GT montage lifecycle (position, blend-out, finish) from sim output.
	MOVER_API void FlushMontageStates(const FMoverSimDrivenMontageData& MontageData);

	// Apply a single montage entry: start, position, blend-out, or finish the GT montage instance.
	MOVER_API void ApplyMontageEntry(const FMoverSimDrivenMontageEntry& Entry, UAnimInstance* AnimInstance);

	// Stop a GT montage instance that disappeared from simulation output without a normal finish signal.
	MOVER_API void StopDrivenMontage(const FMoverSimDrivenMontageEntry& LastEntry, UAnimInstance* AnimInstance);

private:
	// Whether to override the up direction with a fixed value instead of using gravity to deduce it
	UPROPERTY(EditDefaultsOnly, Category="Mover|UpDirection")
	bool bHasUpDirectionOverride = false;

	// A fixed up direction to use if bHasUpDirectionOverride is true
	UPROPERTY(EditDefaultsOnly, Category="Mover|UpDirection", meta = (EditCondition = "bHasUpDirectionOverride"))
	FVector UpDirectionOverride = FVector::UpVector;

	/** Whether or not gravity is overridden on this actor. Otherwise, fall back on world settings. See @SetGravityOverride */
	UPROPERTY(EditDefaultsOnly, Category="Mover|Gravity")
	bool bHasGravityOverride = false;

	/**
     * If true, then the transform updates applied in UMoverComponent::SetFrameStateFromContext will use a "deferred group move"
     * to improve performance.
     *
     * It is not recommended that you enable this when you need exact, high fidelity characters such as your player character.
     * This is mainly a benefit for scenarios with large amounts of NPCs or lower fidelity characters where it is acceptable
     * to not have immediately applied transforms.
     *
     * This only does something if the "s.GroupedComponentMovement.Enable" CVar is set to true.
     */
    UPROPERTY(EditDefaultsOnly, Category="Mover", meta = (EditCondition = "Engine.SceneComponent.IsGroupedComponentMovementEnabled"))
    bool bUseDeferredGroupMovement = false;

	/** Transient flag indicating whether we are executing OnRegister(). */
	bool bInOnRegister = false;

	/** Transient flag indicating whether we are executing InitializeComponent(). */
	bool bInInitializeComponent = false;

	/**
	 * A cached quaternion representing the rotation from world space to gravity relative space defined by GravityAccelOverride.
	 */
	UPROPERTY(Category="Mover|Gravity", VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = true))
	FQuat WorldToGravityTransform;
	
	/**
	 * A cached quaternion representing the inverse rotation from world space to gravity relative space defined by GravityAccelOverride.
	 */
	UPROPERTY(Category="Mover|Gravity", VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = true))
	FQuat GravityToWorldTransform;

	// Simulation-authoritative montage driving state.
	// Tracks each active move (keyed by montage asset) to detect blend-out edge transitions
	// and abrupt cancellations across dispatch cycles.
	TMap<UAnimMontage*, FMoverSimDrivenMontageEntry> DrivenMontageStateMap;


protected:

	/** If enabled, this actor will be moved to follow a base actor that it's standing on. Typically disabled for physics-based movement, which handles based movement internally. */
	UPROPERTY(EditDefaultsOnly, Category = "Mover")
	bool bSupportsKinematicBasedMovement = true;

	FMoverAuxStateContext CachedLastAuxState;

	friend class UBaseMovementMode;
	friend class UMoverDebugComponent;
	friend class UBasedMovementUtils;
};
