// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoverTypes.h"
#include "LayeredMove.h"
#include "MoverDataModelTypes.generated.h"

#define UE_API MOVER_API





// Used to identify how to interpret a movement input vector's values
UENUM(BlueprintType)
enum class EMoveInputType : uint8
{
	Invalid,

	/** Move with intent, as a per-axis magnitude [-1,1] (E.g., "move straight forward as fast as possible" would be (1, 0, 0) and "move straight left at half speed" would be (0, -0.5, 0) regardless of frame time). Zero vector indicates intent to stop. */
	DirectionalIntent,

	/** Move with a given velocity (units per second) */
	Velocity,

	/** No move input of any type */
	None,
};


// Data block containing all inputs that need to be authored and consumed for the default Mover character simulation
USTRUCT(BlueprintType)
struct FCharacterDefaultInputs : public FMoverDataStructBase
{
	GENERATED_BODY()

	// Sets the directional move inputs for a simulation frame
	UE_API void SetMoveInput(EMoveInputType InMoveInputType, const FVector& InMoveInput);

	const FVector& GetMoveInput() const { return MoveInput; }
	EMoveInputType GetMoveInputType() const { return MoveInputType; }

protected:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Mover)
	EMoveInputType MoveInputType;

	/**
	 * Representing the directional move input for this frame. Must be interpreted according to MoveInputType. Relative to MovementBase if set, world space otherwise. Will be truncated to match network serialization precision.
	 * Note: Use SetDirectionalInput or SetVelocityInput to set MoveInput and MoveInputType
	 */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Mover)
	FVector MoveInput;

public:
	// Facing direction intent, as a normalized forward-facing direction. A zero vector indicates no intent to change facing direction. Relative to MovementBase if set, world space otherwise.
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = Mover)
	FVector OrientationIntent;

	// World space orientation that the controls were based on. This is commonly a player's camera rotation.
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = Mover)
	FRotator ControlRotation;

	// Used to force the Mover actor into a different movement mode
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = Mover)
	FName SuggestedMovementMode;

	// Specifies whether we are using a movement base, which will affect how move inputs are interpreted
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = Mover)
	bool bUsingMovementBase;

	// Optional: when moving on a base, input may be relative to this object
	UPROPERTY(BlueprintReadWrite, Category = Mover)
	TObjectPtr<UPrimitiveComponent> MovementBase;

	// Optional: for movement bases that are skeletal meshes, this is the bone we're based on. Only valid if MovementBase is set.
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = Mover)
	FName MovementBaseBoneName;

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = Mover)
	bool bIsJumpJustPressed;
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = Mover)
	bool bIsJumpPressed;

	UE_API FVector GetMoveInput_WorldSpace() const;
	UE_API FVector GetOrientationIntentDir_WorldSpace() const;


	FCharacterDefaultInputs()
		: MoveInputType(EMoveInputType::None)
		, MoveInput(ForceInitToZero)
		, OrientationIntent(ForceInitToZero)
		, ControlRotation(ForceInitToZero)
		, SuggestedMovementMode(NAME_None)
		, bUsingMovementBase(false)
		, MovementBase(nullptr)
		, MovementBaseBoneName(NAME_None)
		, bIsJumpJustPressed(false)
		, bIsJumpPressed(false)
	{
	}

	virtual ~FCharacterDefaultInputs() = default;

	bool operator==(const FCharacterDefaultInputs& Other) const
	{
		return MoveInputType == Other.MoveInputType
			&& MoveInput == Other.MoveInput
			&& OrientationIntent == Other.OrientationIntent
			&& ControlRotation == Other.ControlRotation
			&& SuggestedMovementMode == Other.SuggestedMovementMode
			&& bUsingMovementBase == Other.bUsingMovementBase
			&& MovementBase == Other.MovementBase
			&& MovementBaseBoneName == Other.MovementBaseBoneName
			&& bIsJumpJustPressed == Other.bIsJumpJustPressed
			&& bIsJumpPressed == Other.bIsJumpPressed;
	}
	
	bool operator!=(const FCharacterDefaultInputs& Other) const { return !operator==(Other); }
	
	// @return newly allocated copy of this FCharacterDefaultInputs. Must be overridden by child classes
	UE_API virtual FMoverDataStructBase* Clone() const override;
	UE_API virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;
	virtual UScriptStruct* GetScriptStruct() const override { return StaticStruct(); }
	UE_API virtual void ToString(FAnsiStringBuilderBase& Out) const override;
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override { Super::AddReferencedObjects(Collector); }
	UE_API virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override;
	UE_API virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override;
	UE_API virtual void Merge(const FMoverDataStructBase& From) override;
	UE_API virtual void Decay(float DecayAmount) override;
};

template<>
struct TStructOpsTypeTraits< FCharacterDefaultInputs > : public TStructOpsTypeTraitsBase2< FCharacterDefaultInputs >
{
	enum
	{
		WithNetSerializer = true,
		WithCopy = true
	};
};




// Data block containing basic sync state information
USTRUCT(BlueprintType)
struct FMoverDefaultSyncState : public FMoverDataStructBase
{
	GENERATED_BODY()
protected:
	// Position relative to MovementBase if set, world space otherwise
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FVector Location;

	// Forward-facing rotation relative to MovementBase if set, world space otherwise.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FRotator Orientation;

	// Linear velocity, units per second, relative to MovementBase if set, world space otherwise.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FVector Velocity;

	// Angular velocity, degrees per second, relative to MovementBase if set, world space otherwise.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FVector AngularVelocityDegrees;

public:
	// Movement intent direction relative to MovementBase if set, world space otherwise. Magnitude of range (0-1)
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Mover)
	FVector MoveDirectionIntent;

protected:
	// Optional: when moving on a base, input may be relative to this object
	UPROPERTY(BlueprintReadOnly, Category = Mover)
	TWeakObjectPtr<UPrimitiveComponent> MovementBase;

	// Optional: for movement bases that are skeletal meshes, this is the bone we're based on. Only valid if MovementBase is set.
	UPROPERTY(BlueprintReadOnly, Category = Mover)
	FName MovementBaseBoneName;

	UPROPERTY(BlueprintReadOnly, Category = Mover)
	FVector MovementBasePos;

	UPROPERTY(BlueprintReadOnly, Category = Mover)
	FQuat MovementBaseQuat;

public:

	FMoverDefaultSyncState()
		: Location(ForceInitToZero)
		, Orientation(ForceInitToZero)
		, Velocity(ForceInitToZero)
		, AngularVelocityDegrees(ForceInitToZero)
		, MoveDirectionIntent(ForceInitToZero)
		, MovementBase(nullptr)
		, MovementBaseBoneName(NAME_None)
		, MovementBasePos(ForceInitToZero)
		, MovementBaseQuat(FQuat::Identity)
	{
	}

	virtual ~FMoverDefaultSyncState() = default;

	// @return newly allocated copy of this FMoverDefaultSyncState. Must be overridden by child classes
	UE_API virtual FMoverDataStructBase* Clone() const override;

	UE_API virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;

	virtual UScriptStruct* GetScriptStruct() const override { return StaticStruct(); }

	UE_API virtual void ToString(FAnsiStringBuilderBase& Out) const override;

	UE_API virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override;

	UE_API virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override;

	UE_DEPRECATED(5.7, "Use the SetTransforms_WorldSpace with an angular velocity")
	UE_API void SetTransforms_WorldSpace(FVector WorldLocation, FRotator WorldOrient, FVector WorldVelocity, UPrimitiveComponent* Base=nullptr, FName BaseBone = NAME_None);
	
	UE_API void SetTransforms_WorldSpace(FVector WorldLocation, FRotator WorldOrient, FVector WorldVelocity, FVector WorldAngularVelocityDegrees, UPrimitiveComponent* Base=nullptr, FName BaseBone = NAME_None);

	// Returns whether the base setting succeeded
	UE_API bool SetMovementBase(UPrimitiveComponent* Base, FName BaseBone=NAME_None);

	// Refreshes captured movement base transform based on its current state, while maintaining the same base-relative transforms
	UE_API bool UpdateCurrentMovementBase();

	// Queries
	bool IsNearlyEqual(const FMoverDefaultSyncState& Other) const;

	UPrimitiveComponent* GetMovementBase() const { return MovementBase.Get(); }
	FName GetMovementBaseBoneName() const { return MovementBaseBoneName; }
	FVector GetCapturedMovementBasePos() const { return MovementBasePos; }
	FQuat GetCapturedMovementBaseQuat() const { return MovementBaseQuat; }

	UE_API FVector GetLocation_WorldSpace() const;
	UE_API FVector GetLocation_BaseSpace() const;	// If there is no movement base set, these will be the same as world space

	UE_API FVector GetIntent_WorldSpace() const;
	UE_API FVector GetIntent_BaseSpace() const;

	UE_API FVector GetVelocity_WorldSpace() const;
	UE_API FVector GetVelocity_BaseSpace() const;

	UE_API FRotator GetOrientation_WorldSpace() const;
	UE_API FRotator GetOrientation_BaseSpace() const;

	UE_API FTransform GetTransform_WorldSpace() const;
	UE_API FTransform GetTransform_BaseSpace() const;

	UE_API FVector GetAngularVelocityDegrees_WorldSpace() const;
	UE_API FVector GetAngularVelocityDegrees_BaseSpace() const;
};

template<>
struct TStructOpsTypeTraits< FMoverDefaultSyncState > : public TStructOpsTypeTraitsBase2< FMoverDefaultSyncState >
{
	enum
	{
		WithNetSerializer = true,
		WithCopy = true
	};
};


/**
 * Blueprint function library to make it easier to work with Mover data structs, since we can't add UFUNCTIONs to structs
 */
UCLASS(MinimalAPI)
class UMoverDataModelBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:	// FCharacterDefaultInputs

	/** Sets move input from a unit length vector representing directional intent */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API void SetDirectionalInput(UPARAM(Ref) FCharacterDefaultInputs& Inputs, const FVector& DirectionInput);
	
	/** Sets move input from a vector representing desired velocity c/m^2 */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API void SetVelocityInput(UPARAM(Ref) FCharacterDefaultInputs& Inputs, const FVector& VelocityInput = FVector::ZeroVector);

	/** Returns the move direction intent, if any, in world space */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API FVector GetMoveDirectionIntentFromInputs(const FCharacterDefaultInputs& Inputs);


public:	// FMoverDefaultSyncState

	/** Returns the location in world space */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API FVector GetLocationFromSyncState(const FMoverDefaultSyncState& SyncState);

	/** Returns the move direction intent, if any, in world space */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API FVector GetMoveDirectionIntentFromSyncState(const FMoverDefaultSyncState& SyncState);

	/** Returns the velocity in world space */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API FVector GetVelocityFromSyncState(const FMoverDefaultSyncState& SyncState);

	/** Returns the angular velocity in world space, in degrees per second */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API FVector GetAngularVelocityDegreesFromSyncState(const FMoverDefaultSyncState& SyncState);

	/** Returns the orientation in world space */
	UFUNCTION(BlueprintCallable, Category = Mover)
	static UE_API FRotator GetOrientationFromSyncState(const FMoverDefaultSyncState& SyncState);
};


// AI-related movement inputs (e.g., RVO avoidance)
USTRUCT(BlueprintType)
struct FMoverAIInputs : public FMoverDataStructBase
{
	GENERATED_BODY()

public:
	// RVO Velocity calculated on the Server
	UPROPERTY(BlueprintReadWrite, Category = Mover)
	FVector RVOVelocityDelta;

	FMoverAIInputs()
		: RVOVelocityDelta(ForceInitToZero)
	{
	}

	virtual ~FMoverAIInputs() = default;

	UE_API virtual FMoverDataStructBase* Clone() const override;
	UE_API virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;
	virtual UScriptStruct* GetScriptStruct() const override { return StaticStruct(); }
	UE_API virtual void ToString(FAnsiStringBuilderBase& Out) const override;
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override { Super::AddReferencedObjects(Collector); }
	UE_API virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override;
	UE_API virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct) override;
	UE_API virtual void Merge(const FMoverDataStructBase& From) override;
};

template<>
struct TStructOpsTypeTraits<FMoverAIInputs> : public TStructOpsTypeTraitsBase2<FMoverAIInputs>
{
	enum
	{
		WithNetSerializer = true,
		WithCopy = true,
	};
};


#undef UE_API
