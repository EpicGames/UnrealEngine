// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettingsBackedByCVars.h"

#include "MoverDeveloperSettings.generated.h"

#define UE_API MOVER_API

/** Developer settings for the Mover plugin */
UCLASS(MinimalAPI, config = Engine, defaultconfig, meta = (DisplayName = "Mover Settings"))
class UMoverDeveloperSettings : public UDeveloperSettingsBackedByCVars
{
	GENERATED_BODY()
public:
	UE_API UMoverDeveloperSettings();
	
	/**
     * This specifies the number of times a movement mode can refund all of the time in a substep before we back out to avoid freezing the game/editor
     */
    UPROPERTY(config, EditAnywhere, Category = "Mover")
    int32 MaxTimesToRefundSubstep;
	
};

#undef UE_API
