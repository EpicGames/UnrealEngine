// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoverSimulationTypes.h"

// Set mover.debug.EnableGameplayTagLog to enable Mover gameplay tag event logging
// across the worker thread (WT:*) and game thread (GT:*) steps.
namespace MoverCVars
{
	MOVER_API extern bool bEnableGameplayTagLog;
}

// Logs the [GT:DriftDetect] header line and a [GT:DriftDetect:Event] line per queued event.
// The CVar check is included; safe to call unconditionally.
MOVER_API void LogGameplayTagDriftDetect(
	const FString& ActorName,
	const TCHAR* RoleStr,
	const FMoverTimeStep& TimeStep,
	const FGameplayTag& Tag,
	bool bWasAdded,
	const TArray<TSharedPtr<FMoverSimulationEventData>>& Events,
	const FGameplayTagContainer& PreviousTags,
	const FGameplayTagContainer& NewTags);

// Conditional log macro: gates on MoverCVars::EnableGameplayTagLog and prepends
// the [GameplayTagLog] prefix. Accepts any UE log category.
//   MOVER_TAG_LOG(LogMover, "[WT:TagDiff] Actor=%s Role=%s ...", *Name, *Role, ...)
// Under NO_LOGGING the macro expands to a true no-op that discards all arguments,
// so call sites do not need to guard themselves and DebugOwnerName-bearing members
// can be stripped with a single #if !NO_LOGGING at the declaration.
#if NO_LOGGING
	#define MOVER_TAG_LOG(Category, Format, ...) do {} while (0)
#else
	#define MOVER_TAG_LOG(Category, StepAndFormat, ...) \
		UE_CLOG(MoverCVars::bEnableGameplayTagLog, Category, Log, TEXT("[GameplayTagLog]" StepAndFormat), ##__VA_ARGS__)
#endif
