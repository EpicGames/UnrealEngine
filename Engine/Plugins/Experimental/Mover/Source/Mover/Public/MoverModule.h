// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

struct IConsoleCommand;
class UWorld;

namespace UE::Mover
{
	extern int32 DisableDataCopyInPlace;
}

class FMoverModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	void ShowTrajectory(const TArray<FString>& Args, UWorld* World);
	void ShowTrail(const TArray<FString>& Args, UWorld* World);
	void ShowCorrections(const TArray<FString>& Args, UWorld* World);
	
private:
	TArray<IConsoleCommand*> ConsoleCommands;
};
