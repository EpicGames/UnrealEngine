// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"
#include "PoseSearch/PoseSearchTrajectoryPredictor.h"
#include "MoverPoseSearchTrajectoryPredictor.generated.h"

#define UE_API MOVER_API


class UMoverComponent;
struct FTransformTrajectory;

/**
 * Trajectory predictor that can query from a Mover-driven actor, for use with Pose Search
 */
UCLASS(MinimalAPI, BlueprintType, EditInlineNew)
class UMoverTrajectoryPredictor : public UObject, public IPoseSearchTrajectoryPredictorInterface
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "Animation|PoseSearch|Experimental")
	void Setup(UMoverComponent* InMoverComponent) { MoverComponent = InMoverComponent; }

	UE_API virtual void Predict(FTransformTrajectory& InOutTrajectory, int32 NumPredictionSamples, float SecondsPerPredictionSample, int32 NumHistorySamples) override;

	UE_API virtual void GetGravity(FVector& OutGravityAccel) override;

	UE_API virtual void GetCurrentState(FVector& OutPosition, FQuat& OutFacing, FVector& OutVelocity) override;

	UE_API virtual void GetVelocity(FVector& OutVelocity) override;

	UE_API virtual void GetAngularVelocity(FVector& OutAngularVelocityDegrees) override;

	// Share code in static helper functions so we can call it from AnimNext, which doesn't support interfaces
	static UE_API void Predict(UMoverComponent& Mover, FTransformTrajectory& InOutTrajectory, int32 NumPredictionSamples, float SecondsPerPredictionSample, int32 NumHistorySamples, float MoverSamplingInterval, bool bDisableGravity = true);
	static UE_API void GetCurrentState(UMoverComponent& Mover, FVector& OutPosition, FQuat& OutFacing, FVector& OutVelocity);


protected:
	TObjectPtr<UMoverComponent> MoverComponent;

	// If true, the prediction will not take gravity into account
	// If false, mover prediction will account for gravity (e.g. in falling state)
	UPROPERTY(EditAnywhere, Category = "Animation|PoseSearch|Experimental", BlueprintReadWrite)
	bool bDisableGravity = false;

	// Sampling frame rate to query the mover, doesn't necessarily match the SecondsPerPredictionSample which is used to write to the trajectory
	// This allows sampling of mover at a high frequency for accuracy but then we can downsample the results to a more coarse grained trajectory
	UPROPERTY(EditAnywhere, Category = "Animation|PoseSearch|Experimental", BlueprintReadWrite)
	FFrameRate MoverSamplingFrameRate = FFrameRate(60, 1);
};

#undef UE_API
