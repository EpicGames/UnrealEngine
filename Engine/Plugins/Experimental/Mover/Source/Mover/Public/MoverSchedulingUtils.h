// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoverSimulationTypes.h"
#include "Backends/MoverBackendLiaison.h"

namespace UE
{
namespace Mover
{
	/** Turns a FInstantMovementEffect into a scheduled one (FScheduledInstantMovementEffect)
	*	The effect can be scheduled to apply immediately, or scheduled to apply with a delay
	*   @param Backend the simulation backend which implements the details of whether the simulation advances in fixed dt and how long to schedule in the future for networked events
	*   @param TimeStep the time step of the current or upcoming tick
	*   @param InstantMovementEffect the effect to schedule
	*/
	MOVER_API FScheduledInstantMovementEffect MakeScheduledEffect(const TScriptInterface<IMoverBackendLiaisonInterface>& Backend, const FMoverTimeStep& TimeStep, TSharedPtr<FInstantMovementEffect> InstantMovementEffect);

	// Same as above but schedules the effect to run on the next frame. This is best used for fixed tick mode, it will add a small SchedulingDelaySeconds in the variable tick case
	MOVER_API FScheduledInstantMovementEffect MakeScheduledEffectForNextFrame(const TScriptInterface<IMoverBackendLiaisonInterface>& Backend, const FMoverTimeStep& TimeStep, TSharedPtr<FInstantMovementEffect> InstantMovementEffect);

	/** Turns a FLayeredMove into a scheduled one (FScheduledLayeredMove)
	*	The move can be scheduled to apply immediately, or scheduled to apply with a delay
	*   @param Backend the simulation backend which implements the details of whether the simulation advances in fixed dt and how long to schedule in the future for networked events
	*   @param TimeStep the time step of the current or upcoming tick
	*   @param LayeredMove the move to schedule
	*/
	MOVER_API FScheduledLayeredMove MakeScheduledMove(const TScriptInterface<IMoverBackendLiaisonInterface>& Backend, const FMoverTimeStep& TimeStep, TSharedPtr<FLayeredMoveBase> LayeredMove);
	// Same as above but schedules the effect to run on the next frame. This is best used for fixed tick mode, it will add a small SchedulingDelaySeconds in the variable tick case
	MOVER_API FScheduledLayeredMove MakeScheduledMoveForNextFrame(const TScriptInterface<IMoverBackendLiaisonInterface>& Backend, const FMoverTimeStep& TimeStep, TSharedPtr<FLayeredMoveBase> LayeredMove);
}
}
