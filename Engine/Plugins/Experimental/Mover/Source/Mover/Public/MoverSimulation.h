// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "MoveLibrary/RollbackBlackboardWrappers.h"
#include "MovementModifier.h"
#include "MoverSimulationTypes.h"
#include "MoverSimulation.generated.h"

#define UE_API MOVER_API

class UMoverBlackboard;
struct FLayeredMoveInstance;
struct FMoverSyncState;

/**
* WIP Base class for a Mover simulation.
* The simulation is intended to be the thing that updates the Mover
* state and should be safe to run on an async thread
*/
UCLASS(Abstract, MinimalAPI, BlueprintType)
class UMoverSimulation : public UObject
{
	GENERATED_BODY()

public:
	UE_API UMoverSimulation();

	// Warning: the regular blackboard will be fully replaced by the rollback blackboard in the future
	UFUNCTION(BlueprintPure, Category = Mover)
	UE_API const UMoverBlackboard* GetBlackboard() const;

	// Warning: the regular blackboard will be fully replaced by the rollback blackboard in the future
	UFUNCTION(BlueprintPure, Category = Mover)
	UE_API UMoverBlackboard* GetBlackboard_Mutable();

	UFUNCTION(BlueprintPure, Category = Mover)
	UE_API FRollbackBlackboardSimWrapper GetRollbackBlackboardSimWrapper() const;

	// Get accessor for the blackboard to be used predictively. You can read from it, but any writes will be temporary and thrown away once prediction is finished.
	UFUNCTION(BlueprintPure, Category = Mover)
	UE_API FRollbackBlackboardSimWrapper GetRollbackBlackboardPredictionWrapper() const;

	// Queues a mode change for the next opportunity. This function is intended to be threadsafe and can be called from inside or outside of the simulation.
	UFUNCTION(BlueprintCallable, Category = Mover, meta=(BlueprintThreadSafe))
	virtual void QueueNextMode(FModeChangeParams ModeChangeParams) {}

	// Gets the anticipated next mode, either the next queued one or the current mode.
	UFUNCTION(BlueprintPure, Category = Mover, meta = (BlueprintThreadSafe))
	virtual FName GetNextModeName() const { return NAME_None; }

	virtual bool HasFeaturesWithTag(FGameplayTag TagToMatch, bool bRequireExactMatch) const { return false; }
	virtual void CancelFeaturesWithTag(FGameplayTag TagToCancel, bool bRequireExactMatch = false) {}

	// Registers a new movement mode for use in the simulation.
	UFUNCTION(BlueprintCallable, Category = Mover, meta = (BlueprintThreadSafe))
	virtual void RegisterMovementMode(FModeRegistrationParams ModeRegisterParams) {}

	// Unregisters a movement mode from the simulation
	UFUNCTION(BlueprintCallable, Category = Mover, meta = (BlueprintThreadSafe))
	virtual void UnregisterMovementMode(FModeRegistrationParams ModeRegisterParams) {}

	// Modifier management functions
	virtual const FMovementModifierBase* FindQueuedModifier(FMovementModifierHandle ModifierHandle) const { return nullptr; }
	virtual const FMovementModifierBase* FindQueuedModifierByType(const UScriptStruct* ModifierType) const { return nullptr; }
	virtual FMovementModifierHandle QueueMovementModifier(TSharedPtr<FMovementModifierBase> Modifier) { return FMovementModifierHandle(MODIFIER_INVALID_HANDLE); }
	virtual void CancelModifierFromHandle(FMovementModifierHandle ModifierHandle) {}

	// -------------------------------------------------------------------------
	// Thread-safe simulation-level staging queues.
	// GT queues here; backends/FSM consume with ConsumeQueuedXxx on the simulation thread.
	// -------------------------------------------------------------------------

	/** Queue a struct-based layered move (GT staging, write-locked). */
	UE_API void QueueLayeredMove(const FScheduledLayeredMove& Move);

	/** Queue a scheduled instant movement effect (GT staging, write-locked). */
	UE_API void QueueInstantMovementEffect(const FScheduledInstantMovementEffect& Effect);

	/** Queue an instanced layered move (GT staging, write-locked). */
	UE_API void QueueLayeredMoveInstance(TSharedPtr<FLayeredMoveInstance> MoveInstance);

	/** Consume all queued struct-based layered moves. Empties the queue under a write lock. */
	UE_API TArray<FScheduledLayeredMove> ConsumeQueuedLayeredMoves();

	/** Consume all queued instant movement effects. Empties the queue under a write lock. */
	UE_API TArray<FScheduledInstantMovementEffect> ConsumeQueuedInstantMovementEffects();

	/** Consume all queued instanced layered moves. Empties the queue under a write lock. */
	UE_API TArray<TSharedPtr<FLayeredMoveInstance>> ConsumeQueuedLayeredMoveInstances();

	/**
	* Attempt to teleport to TargetTransform. The teleport is not guaranteed to happen. This function is meant to be called by an instant movement effect as part of its effect application.
	* If it succeeds a FTeleportSucceededEventData will be emitted, if it fails a FTeleportFailedEventData will be sent.
	* @param TimeStep The time step of the current step or substep being simulated. This will come from the ApplyMovementEffect function.
	* @param TargetTransform The transform to teleport to. In the case bUseActorRotation is true, the rotation of this transform will be ignored.
	* @param bUseActorRotation If true, the rotation will not be modified upon teleportation. If false, the rotation in TargetTransform will be used to orient the teleported.
	* @param OutputState This is the sync state that me modified as a result of the application of this effect. Like TimeStep, this should come from the ApplyMovementEffect function.
	*/
	UFUNCTION(BlueprintCallable, Category = Mover)
	virtual void AttemptTeleport(const FMoverTimeStep& TimeStep, const FTransform& TargetTransform, bool bUseActorRotation, FMoverSyncState& OutputState) {}


protected:
	UPROPERTY(Transient)
	TObjectPtr<UMoverBlackboard> Blackboard = nullptr;

	UPROPERTY(Transient)
	TObjectPtr<URollbackBlackboard> RollbackBlackboard = nullptr;

	FRollbackBlackboardSimWrapper RollbackBlackboardSimWrapper;

// TODO: These friend clases are temporary until the control flow changes to include a Simulation object consistently
friend class UMoverComponent;

private:
	/** Protects the three simulation-level staging queues below. */
	mutable FTransactionallySafeRWLock SimQueueLock;

	/** Struct-based layered moves staged by the GT, consumed by the FSM or ChaosMover backend each tick. */
	TArray<FScheduledLayeredMove> QueuedLayeredMoves;

	/** Instant movement effects staged by the GT, consumed by the FSM or ChaosMover backend each tick. */
	TArray<FScheduledInstantMovementEffect> QueuedInstantMovementEffects;

	/** Instanced layered moves staged by the GT (via MakeAndQueueLayeredMove), consumed each tick. */
	TArray<TSharedPtr<FLayeredMoveInstance>> QueuedLayeredMoveInstances;
};

#undef UE_API
