// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/StringBuilder.h"
#include "NetworkPredictionReplicationProxy.h"
#include "Engine/NetSerialization.h"
#include "MoverTypes.h"
#include "MoveLibrary/MovementRecord.h"
#include "MoveLibrary/RollbackBlackboardWrappers.h"
#include "LayeredMove.h"
#include "LayeredMoveGroup.h"
#include "MovementModifier.h"
#include "MoverDataModelTypes.h"
#include "InstantMovementEffect.h"
#include "UObject/Interface.h"
#include <functional>

#include "MoverSimulationTypes.generated.h"

class UBaseMovementMode;


// Names for our default modes
namespace DefaultModeNames
{
	extern MOVER_API const FName Walking;
	extern MOVER_API const FName Falling;
	extern MOVER_API const FName Flying;
	extern MOVER_API const FName Swimming;
}

// Commonly-used blackboard object keys
namespace CommonBlackboard
{
	extern MOVER_API const FName LastFloorResult;
	extern MOVER_API const FName LastWaterResult;

	extern MOVER_API const FName LastFoundDynamicMovementBase;
	extern MOVER_API const FName AccumulatedBasedTransformDelta;
	extern MOVER_API const FName LastBasedMovementAppliedEventTime;

	extern MOVER_API const FName TimeSinceSupported;

	extern MOVER_API const FName LastModeChangeRecord;
}

/**
 * Filled out by a MovementMode during simulation tick to indicate its ending state, allowing for a residual time step and switching modes mid-tick
 */
USTRUCT(BlueprintType)
struct FMovementModeTickEndState
{
	GENERATED_BODY()
	
	FMovementModeTickEndState() 
	{ 
		ResetToDefaults(); 
	}

	void ResetToDefaults()
	{
		RemainingMs = 0.f;
		NextModeName = NAME_None;
		bEndedWithNoChanges = false;
	}

	// Any unused tick time
	UPROPERTY(BlueprintReadWrite, Category=Mover)
	float RemainingMs;

	UPROPERTY(BlueprintReadWrite, Category=Mover)
	FName NextModeName = NAME_None;

	// Affirms that no state changes were made during this simulation tick. Can help optimizations if modes set this during sim tick.
	UPROPERTY(BlueprintReadWrite, Category=Mover)
	bool bEndedWithNoChanges = false;
};

USTRUCT()
struct FMoverSchedulingInfo
{
	GENERATED_BODY()

	bool ShouldExecuteAtTimeStep(const FMoverTimeStep& TimeStep) const
	{
		return bIsFixedDt ? ShouldExecuteAtFrame(TimeStep.ServerFrame) : ShouldExecuteAtTimeMs(TimeStep.BaseSimTimeMs);
	}

	bool ShouldExecuteAtTime(const FMoverTime& ServerTime) const
	{
		return bIsFixedDt ? ShouldExecuteAtFrame(ServerTime.FrameCount) : ShouldExecuteAtTimeMs(ServerTime.TimeMs);
	}

	bool ShouldExecuteAtFrame(int32 CurrentServerFrame) const
	{
		ensureMsgf(bIsFixedDt, TEXT("In variable delta time mode, use ShouldExecuteAtTimeMs"));
		return (ensure(ServerExecutionTime.FrameCount != INDEX_NONE) && CurrentServerFrame >= ServerExecutionTime.FrameCount);
	}

	bool ShouldExecuteAtTimeMs(double CurrentServerTimeMs) const
	{
		ensureMsgf(!bIsFixedDt, TEXT("In fixed delta time mode, use ShouldExecuteAtFrame"));
		return (CurrentServerTimeMs >= ServerExecutionTime.TimeMs);
	}

	bool ShouldRollBackAtTimeStep(const FMoverTimeStep& NewTimeStep) const
	{
		return bIsFixedDt ? ShouldRollBackAtFrame(NewTimeStep.ServerFrame) : ShouldRollBackAtTimeMs(NewTimeStep.BaseSimTimeMs);
	}

	bool ShouldRollBackAtFrame(int32 NewServerFrame) const
	{
		ensureMsgf(bIsFixedDt, TEXT("In variable delta time mode, use ShouldRollBackAtTimeMs"));
		return (ensure(NewServerFrame != INDEX_NONE) && ServerIssuanceTime.FrameCount >= NewServerFrame);
	}

	bool ShouldRollBackAtTimeMs(double NewServerTimeMs) const
	{
		ensureMsgf(!bIsFixedDt, TEXT("In fixed delta time mode, use ShouldRollBackAtFrame"));
		return (ServerIssuanceTime.TimeMs >= NewServerTimeMs);
	}

	FString ToString() const
	{
		return FString::Printf(TEXT("Issuance: %s | Execution: %s | bIsFixedDt = %d "), *ServerIssuanceTime.ToString(), *ServerExecutionTime.ToString(), bIsFixedDt ? 1 : 0);
	}

	// Server time at which this scheduled event was issued
	// bIsFixedDt TRUE: 
	//	   - ServerIssuanceTime.FrameCount will be valid and set to the backend's GetCurrentSimFrame, i.e. that of the server frame count of the first async step that will be processing this.
	//     - ServerIssuanceTime.TimeMs MAY be set to ServerIssuanceTime.TimeMs * AsyncFixedDt * 1000, is only here for reference, and it NOT an interpolated time
	// bIsFixedDt FALSE:
	//     - ServerIssuanceTime.FrameCount will be invalid, do not use
	//     - ServerIssuanceTime.TimeMs will be set to the backend's GetCurrentSimTimeMs
	UPROPERTY(VisibleAnywhere, Category = "Mover")
	FMoverTime ServerIssuanceTime;

	// Server time at which this scheduled event should take effect
	// bIsFixedDt TRUE: 
	//	   - ServerExecutionTime.FrameCount will be valid and set relative to the backend's GetCurrentSimFrame, i.e. that of the server frame count of the first async step that will be processing this.
	//     - ServerExecutionTime.TimeMs MAY be set to ServerExecutionTime.FrameCount * AsyncFixedDt * 1000, is only here for reference, and it NOT an interpolated time
	// bIsFixedDt FALSE:
	//     - ServerExecutionTime.FrameCount will be invalid, do not use
	//     - ServerExecutionTime.TimeMs will be set relative to the backend's GetCurrentSimTimeMs
	UPROPERTY(VisibleAnywhere, Category = "Mover")
	FMoverTime ServerExecutionTime;

	// Whether this time was set in the context of a fixed or variable time advance scheme. This should correspond to the backend's IsFixedDt()
	UPROPERTY(VisibleAnywhere, Category = "Mover")
	bool bIsFixedDt = true;
};

USTRUCT()
struct FScheduledInstantMovementEffect
{
	GENERATED_BODY()

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		Out.Appendf("Scheduling = %s | Effect = %s", TCHAR_TO_ANSI(*SchedulingInfo.ToString()), Effect.IsValid() ? TCHAR_TO_ANSI(*Effect->ToSimpleString()) : "Invalid");
	}

	FMoverSchedulingInfo SchedulingInfo;

	TSharedPtr<FInstantMovementEffect> Effect;
};

USTRUCT()
struct FScheduledLayeredMove
{
	GENERATED_BODY()

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		Out.Appendf("Scheduling = %s | Move = %s", TCHAR_TO_ANSI(*SchedulingInfo.ToString()), Move.IsValid() ? TCHAR_TO_ANSI(*Move->ToSimpleString()) : "Invalid");
	}

	FMoverSchedulingInfo SchedulingInfo;

	TSharedPtr<FLayeredMoveBase> Move;
};

/**
 * The client generates this representation of "input" to the simulated actor for one simulation frame. This can be direct mapping
 * of controls, or more abstract data. It is composed of a collection of typed structs that can be customized per project.
 */
USTRUCT(BlueprintType)
struct FMoverInputCmdContext
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category = Mover)
	FMoverDataCollection InputCollection;

	UScriptStruct* GetStruct() const
	{
		return StaticStruct();
	}

	void NetSerialize(const FNetSerializeParams& P)
	{
		bool bIgnoredResult(false);
		InputCollection.NetSerialize(P.Ar, P.Map, bIgnoredResult);
	}

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		InputCollection.ToString(Out);
	}

	void Interpolate(const FMoverInputCmdContext* From, const FMoverInputCmdContext* To, float Pct)
	{
		InputCollection.Interpolate(From->InputCollection, To->InputCollection, Pct);
	}

	void Reset()
	{
		InputCollection.Empty();
	}
};


/** State we are evolving frame to frame and keeping in sync (frequently changing). It is composed of a collection of typed structs 
 *  that can be customized per project. Mover actors are required to have FMoverDefaultSyncState as one of these structs.
 */
USTRUCT(BlueprintType)
struct FMoverSyncState
{
	GENERATED_BODY()

public:

	// The mode we ended up in from the prior frame, and which we'll start in during the next frame
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Mover)
	FName MovementMode;

	// Additional moves influencing our proposed motion
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Mover)
	FLayeredMoveGroup LayeredMoves;

	// Additional moves influencing our proposed motion
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Mover)
	FLayeredMoveInstanceGroup LayeredMoveInstances;

	// Additional modifiers influencing our simulation
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Mover)
	FMovementModifierGroup MovementModifiers;

	UPROPERTY(BlueprintReadWrite, Category = Mover)
	FMoverDataCollection SyncStateCollection;

	FMoverSyncState()
	{
		MovementMode = NAME_None;
	}

	bool HasSameContents(const FMoverSyncState& Other) const
	{
		return MovementMode == Other.MovementMode &&
			LayeredMoves.HasSameContents(Other.LayeredMoves) &&
			LayeredMoveInstances.HasSameContents(Other.LayeredMoveInstances) &&
			MovementModifiers.HasSameContents(Other.MovementModifiers) &&
			SyncStateCollection.HasSameContents(Other.SyncStateCollection);
	}

	UScriptStruct* GetStruct() const { return StaticStruct(); }


	void NetSerialize(const FNetSerializeParams& P)
	{
		P.Ar << MovementMode;
		LayeredMoves.NetSerialize(P.Ar);
		LayeredMoveInstances.NetSerialize(P.Ar);
		MovementModifiers.NetSerialize(P.Ar);

		bool bIgnoredResult(false);
		SyncStateCollection.NetSerialize(P.Ar, P.Map, bIgnoredResult);
	}

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		Out.Appendf("MovementMode: %s\n", TCHAR_TO_ANSI(*MovementMode.ToString()));
		Out.Appendf("Layered Moves: %s\n", TCHAR_TO_ANSI(*LayeredMoves.ToSimpleString()));
		Out.Appendf("Layered Moves: %s\n", TCHAR_TO_ANSI(*LayeredMoveInstances.ToSimpleString()));
		Out.Appendf("Movement Modifiers: %s\n", TCHAR_TO_ANSI(*MovementModifiers.ToSimpleString()));
		SyncStateCollection.ToString(Out);
	}

	bool ShouldReconcile(const FMoverSyncState& AuthorityState) const
	{
		return (MovementMode != AuthorityState.MovementMode) || 
			   SyncStateCollection.ShouldReconcile(AuthorityState.SyncStateCollection) ||
			   MovementModifiers.ShouldReconcile(AuthorityState.MovementModifiers);
	}

	void Interpolate(const FMoverSyncState* From, const FMoverSyncState* To, float Pct)
	{
		MovementMode = To->MovementMode;
		LayeredMoves = To->LayeredMoves;
		LayeredMoveInstances = To->LayeredMoveInstances;
		MovementModifiers = To->MovementModifiers;

		SyncStateCollection.Interpolate(From->SyncStateCollection, To->SyncStateCollection, Pct);
	}

	// Resets the sync state to its default configuration and removes any
	// active or queued layered modes and modifiers
	void Reset()
	{
		MovementMode = NAME_None;
		SyncStateCollection.Empty();
		LayeredMoves.Reset();
		LayeredMoveInstances.Reset();
		MovementModifiers.Reset();
	}
};

/** 
 *  Double Buffer struct for various Mover data. 
 */
template<typename T>
struct FMoverDoubleBuffer
{
	// Sets all buffered data - usually used for initializing data
	void SetBufferedData(const T& InDataToCopy)
	{
		Buffer[0] = InDataToCopy;
		Buffer[1] = InDataToCopy;
	}
	
	// Gets data that is safe to read and is not being written to
	const T& GetReadable() const
	{
		return Buffer[ReadIndex];
	}

	// Gets data that is being written to and is expected to change
	T& GetWritable()
	{
		return Buffer[(ReadIndex + 1) % 2];
	}

	// Flips which data in the buffer we return for reading and writing
	void Flip()
	{
		ReadIndex = (ReadIndex + 1) % 2;
	}
	
private:
	uint32 ReadIndex = 0;
	T Buffer[2];
};

// Auxiliary state that is input into the simulation (changes rarely)
USTRUCT(BlueprintType)
struct FMoverAuxStateContext
{
	GENERATED_BODY()

public:
	UScriptStruct* GetStruct() const { return StaticStruct(); }

	bool ShouldReconcile(const FMoverAuxStateContext& AuthorityState) const
	{ 
		return AuxStateCollection.ShouldReconcile(AuthorityState.AuxStateCollection); 
	}

	void NetSerialize(const FNetSerializeParams& P)
	{
		bool bIgnoredResult(false);
		AuxStateCollection.NetSerialize(P.Ar, P.Map, bIgnoredResult);
	}

	void ToString(FAnsiStringBuilderBase& Out) const
	{
		AuxStateCollection.ToString(Out);
	}

	void Interpolate(const FMoverAuxStateContext* From, const FMoverAuxStateContext* To, float Pct)
	{
		AuxStateCollection.Interpolate(From->AuxStateCollection, To->AuxStateCollection, Pct);
	}

	UPROPERTY(BlueprintReadWrite, Category = Mover)
	FMoverDataCollection AuxStateCollection;
};


/**
 * Contains all state data for the start of a simulation tick
 */
USTRUCT(BlueprintType)
struct FMoverTickStartData
{
	GENERATED_BODY()

	FMoverTickStartData() {}
	FMoverTickStartData(
			const FMoverInputCmdContext& InInputCmd,
			const FMoverSyncState& InSyncState,
			const FMoverAuxStateContext& InAuxState)
		:  InputCmd(InInputCmd), SyncState(InSyncState), AuxState(InAuxState)
	{
	}

	UPROPERTY(BlueprintReadOnly, Category=Mover)
	FMoverInputCmdContext InputCmd;
	UPROPERTY(BlueprintReadOnly, Category=Mover)
	FMoverSyncState SyncState;
	UPROPERTY(BlueprintReadOnly, Category=Mover)
	FMoverAuxStateContext AuxState;
};

/**
 * Contains all state data produced by a simulation tick, including new simulation state
 */
USTRUCT(BlueprintType)
struct FMoverTickEndData
{
	GENERATED_BODY()

	FMoverTickEndData() {}
	FMoverTickEndData(
		const FMoverSyncState* SyncState,
		const FMoverAuxStateContext* AuxState)
	{
		this->SyncState = *SyncState;
		this->AuxState = *AuxState;
	}

	void InitForNewFrame()
	{
		MovementEndState.ResetToDefaults();
		MoveRecord.Reset();
	}

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Mover)
	FMoverSyncState SyncState;
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Mover)
	FMoverAuxStateContext AuxState;
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Mover)
	FMovementModeTickEndState MovementEndState;

	FMovementRecord MoveRecord;
};



/**
 * FMoverSimContext: contains information about the systems supporting the active simulation. This is commonly
 * passed in to in-simulation functions and events.
 * All objects are guaranteed to exist for the lifetime of any function called with FMoverSimContext as an argument.
 * Do not cache references to them, as their lifetime is not guaranteed outside of the simulation.
 */
USTRUCT(BlueprintType)
struct FMoverSimContext
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, Category = Mover)
	TObjectPtr<UMoverSimulation> Simulation = nullptr;

	UPROPERTY(BlueprintReadOnly, Category = Mover)
	mutable FRollbackBlackboardSimWrapper Blackboard;

	/** 
	 * Owner of the simulation. True type is dependent on the situation (Actor, MoverComp, SG entity, etc.)
	 * May be null when its lifetime can't be guaranteed
	 */
	UPROPERTY(BlueprintReadOnly, Category=Mover) 
	TObjectPtr<UObject> SimulationOwner = nullptr;
};


// Input parameters to provide context for SimulationTick functions
USTRUCT(BlueprintType)
struct FSimulationTickParams
{
	GENERATED_BODY()

	// Information about the systems supporting the active simulation
	UPROPERTY(BlueprintReadOnly, Category = Mover)
	FMoverSimContext SimContext;

	// Components involved in movement by the simulation
	// This will be empty when the simulation is ticked asynchronously
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Mover)
	FMovingComponentSet MovingComps;

	// Blackboard (old style, soon to be removed)
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Mover)
	TObjectPtr<UMoverBlackboard> SimBlackboard;

	// Simulation state data at the start of the tick, including Input Cmd
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Mover)
	FMoverTickStartData StartState;

	// Time and frame information for this tick
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Mover)
	FMoverTimeStep TimeStep;

	// Proposed movement for this tick
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Mover)
	FProposedMove ProposedMove;
};

UINTERFACE(MinimalAPI, BlueprintType)
class UMoverInputProducerInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * MoverInputProducerInterface: API for any object that can produce input for a Mover simulation frame
 */
class IMoverInputProducerInterface : public IInterface
{
	GENERATED_BODY()

public:
	/** Contributes additions to the input cmd for this simulation frame. Typically this is translating accumulated user input (or AI state) into parameters that affect movement. */
	UFUNCTION(BlueprintNativeEvent)
	MOVER_API void ProduceInput(int32 SimTimeMs, FMoverInputCmdContext& InputCmdResult);
};


/** 
 * FMoverPredictTrajectoryParams: parameter block for querying future trajectory samples based on a starting state
 * See UMoverComponent::GetPredictedTrajectory
 */
USTRUCT(BlueprintType)
struct FMoverPredictTrajectoryParams
{
	GENERATED_BODY()

	/** How many samples to predict into the future, including the first sample, which is always a snapshot of the
	 *  starting state with 0 accumulated time. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover, meta = (ClampMin = 1))
	int32 NumPredictionSamples = 1;

	/* How much time between predicted samples */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover, meta = (ClampMin = 0.00001))
	float SecondsPerSample = 0.333f;

	/** If true, samples are based on the visual component transform, rather than the 'updated' movement root.
	 *  Typically, this is a mesh with its component location at the bottom of the collision primitive.
	 *  If false, samples are from the movement root. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	bool bUseVisualComponentRoot = false;

	/** If true, gravity will not taken into account during prediction */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	bool bDisableGravity = false;

	/** Optional starting time. If not set, prediction will begin from the current sim time. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	TOptional<FMoverTime> OptionalStartTime;

 	/** Optional starting sync state. If not set, prediction will begin from the current state. */
 	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
 	TOptional<FMoverSyncState> OptionalStartSyncState;
 
 	/** Optional starting aux state. If not set, prediction will begin from the current state. */
 	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
 	TOptional<FMoverAuxStateContext> OptionalStartAuxState;

 	/** Optional input cmds to use, one per sample. If none are specified, prediction will begin with last-used inputs. 
 	 *  If too few are specified for the number of samples, the final input in the array will be used repeatedly to cover remaining samples. */
 	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
 	TArray<FMoverInputCmdContext> OptionalInputCmds;

};

USTRUCT()
struct FMoverSimEventGameThreadContext
{
	GENERATED_BODY()

public:
	UMoverComponent* MoverComp = nullptr;
};

USTRUCT()
struct FMoverSimulationEventData
{
	GENERATED_BODY()

	using FEventProcessedCallbackPtr = std::function<void(const FMoverSimulationEventData& Data, const FMoverSimEventGameThreadContext& GameThreadContext)>;

	FMoverSimulationEventData(const FMoverTimeStep& InEventTime, FEventProcessedCallbackPtr InEventProcessedCallback = nullptr)
		: EventProcessedCallback(InEventProcessedCallback)
		, Context(InEventTime)
	{
	}

	FMoverSimulationEventData() {}
	virtual ~FMoverSimulationEventData() = default;

	// User must override
	MOVER_API virtual UScriptStruct* GetScriptStruct() const;

	template<typename T>
	T* CastTo_Mutable()
	{
		return T::StaticStruct() == GetScriptStruct() ? static_cast<T*>(this) : nullptr;
	}

	template<typename T>
	const T* CastTo() const
	{
		return const_cast<const T*>(const_cast<FMoverSimulationEventData*>(this)->CastTo_Mutable<T>());
	}

	void OnEventProcessed(const FMoverSimEventGameThreadContext& GameThreadContext) const
	{
		if (EventProcessedCallback)
		{
			EventProcessedCallback(*this, GameThreadContext);
		}
	}

	void SetEventProcessedCallback(FEventProcessedCallbackPtr Callback)
	{
		EventProcessedCallback = Callback;
	}

	virtual bool IsEqual(const FMoverSimulationEventData& Other, int32 ServerFrameDiffTolerance = 0) const
	{
		return (GetScriptStruct() == Other.GetScriptStruct()) &&
			(Context.ServerFrame <= Other.Context.ServerFrame + ServerFrameDiffTolerance) &&
			(Context.ServerFrame >= Other.Context.ServerFrame - ServerFrameDiffTolerance);
	}

private:
	// This callback is fired when the event is processed on the game thread
	// This is called before and in addition to any type based handling
	FEventProcessedCallbackPtr EventProcessedCallback = nullptr;

public:
	UPROPERTY(VisibleAnywhere, Category = Mover)
	FMoverEventContext Context;

	// If true, this event will always re-emit during resimulation and is never added to the
	// ProcessedEvents dedup list. Set this on event types that represent state changes which
	// must be faithfully replayed on resim (e.g. gameplay tag adds/removes). Leave false for
	// punctual one-shot events that the Game Thread has already received and must not see again.
	UPROPERTY(VisibleAnywhere, Category = Mover)
	bool bReEmitOnResim = false;
};

// WithCopy = true: EventProcessedCallback is a std::function. FInstancedStruct::InitializeAs falls back to
// FMemory::Memcpy when WithCopy is absent, which is UB for std::function (shallow copy of internal heap storage).
// operator= must be used instead so std::function copies correctly.
template<>
struct TStructOpsTypeTraits<FMoverSimulationEventData> : public TStructOpsTypeTraitsBase2<FMoverSimulationEventData>
{
	enum { WithCopy = true };
};

USTRUCT()
struct FMovementModeChangedEventData : public FMoverSimulationEventData
{
	GENERATED_BODY()

	FMovementModeChangedEventData(const FMoverTimeStep& InEventTime, const FName InPreviousModeName, const FName InNewModeName, FEventProcessedCallbackPtr InEventProcessedCallback = nullptr)
		: FMoverSimulationEventData(InEventTime, InEventProcessedCallback)
		, PreviousModeName(InPreviousModeName)
		, NewModeName(InNewModeName)
	{
	}
	FMovementModeChangedEventData() {}

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return FMovementModeChangedEventData::StaticStruct();
	}

	virtual bool IsEqual(const FMoverSimulationEventData& Other, int32 ServerFrameDiffTolerance = 0) const override
	{
		if (Super::IsEqual(Other, ServerFrameDiffTolerance))
		{
			const FMovementModeChangedEventData& TypedOther = static_cast<const FMovementModeChangedEventData&>(Other);
			return (PreviousModeName == TypedOther.PreviousModeName) && (NewModeName == TypedOther.NewModeName);
		}
		else
		{
			return false;
		}
	}

	UPROPERTY(VisibleAnywhere, Category = Mover)
	FName PreviousModeName = NAME_None;

	UPROPERTY(VisibleAnywhere, Category = Mover)
	FName NewModeName = NAME_None;
};

// WithCopy = true: inherited std::function in FMoverSimulationEventData requires operator= for safe copying via FInstancedStruct.
template<>
struct TStructOpsTypeTraits<FMovementModeChangedEventData> : public TStructOpsTypeTraitsBase2<FMovementModeChangedEventData>
{
	enum { WithCopy = true };
};

USTRUCT()
struct FTeleportSucceededEventData : public FMoverSimulationEventData
{
	GENERATED_BODY()

	FTeleportSucceededEventData(const FMoverTimeStep& InEventTime, const FVector& InFromLocation, const FQuat& InFromRotation, const FVector& InToLocation, const FQuat& InToRotation)
		: FMoverSimulationEventData(InEventTime)
		, FromLocation(InFromLocation)
		, FromRotation(InFromRotation)
		, ToLocation(InToLocation)
		, ToRotation(InToRotation)
	{
	}
	FTeleportSucceededEventData() {}

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return FTeleportSucceededEventData::StaticStruct();
	}

	virtual bool IsEqual(const FMoverSimulationEventData& Other, int32 ServerFrameDiffTolerance = 0) const override
	{
		if (Super::IsEqual(Other, ServerFrameDiffTolerance))
		{
			const FTeleportSucceededEventData& TypedOther = static_cast<const FTeleportSucceededEventData&>(Other);
			return FromLocation.Equals(TypedOther.FromLocation) && FromRotation.Equals(TypedOther.FromRotation) &&
				ToLocation.Equals(TypedOther.ToLocation) && ToRotation.Equals(TypedOther.ToRotation);
		}
		else
		{
			return false;
		}
	}

	UPROPERTY(VisibleAnywhere, Category = Mover)
	FVector FromLocation = FVector::ZeroVector;

	UPROPERTY(VisibleAnywhere, Category = Mover)
	FQuat FromRotation = FQuat::Identity;

	UPROPERTY(VisibleAnywhere, Category = Mover)
	FVector ToLocation = FVector::ZeroVector;

	UPROPERTY(VisibleAnywhere, Category = Mover)
	FQuat ToRotation = FQuat::Identity;
};

// WithCopy = true: inherited std::function in FMoverSimulationEventData requires operator= for safe copying via FInstancedStruct.
template<>
struct TStructOpsTypeTraits<FTeleportSucceededEventData> : public TStructOpsTypeTraitsBase2<FTeleportSucceededEventData>
{
	enum { WithCopy = true };
};

UENUM(BlueprintType)
enum class ETeleportFailureReason : uint8
{
	Reason_NotAvailable UMETA(DisplayName = "Reason Not Available", Tooltip = "A reason for the teleport failure was not indicated"),
};

USTRUCT()
struct FTeleportFailedEventData : public FMoverSimulationEventData
{
	GENERATED_BODY()

	FTeleportFailedEventData(const FMoverTimeStep& InEventTime, const FVector& InFromLocation, const FQuat& InFromRotation, const FVector& InToLocation, const FQuat& InToRotation, ETeleportFailureReason InTeleportFailureReason)
		: FMoverSimulationEventData(InEventTime)
		, FromLocation(InFromLocation)
		, FromRotation(InFromRotation)
		, ToLocation(InToLocation)
		, ToRotation(InToRotation)
		, TeleportFailureReason(InTeleportFailureReason)
	{
	}
	FTeleportFailedEventData() {}

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return FTeleportFailedEventData::StaticStruct();
	}

	virtual bool IsEqual(const FMoverSimulationEventData& Other, int32 ServerFrameDiffTolerance = 0) const override
	{
		if (Super::IsEqual(Other, ServerFrameDiffTolerance))
		{
			const FTeleportFailedEventData& TypedOther = static_cast<const FTeleportFailedEventData&>(Other);
			return FromLocation.Equals(TypedOther.FromLocation) && FromRotation.Equals(TypedOther.FromRotation) &&
				ToLocation.Equals(TypedOther.ToLocation) && ToRotation.Equals(TypedOther.ToRotation);
		}
		else
		{
			return false;
		}
	}

	UPROPERTY(VisibleAnywhere, Category = Mover)
	FVector FromLocation = FVector::ZeroVector;

	UPROPERTY(VisibleAnywhere, Category = Mover)
	FQuat FromRotation = FQuat::Identity;

	UPROPERTY(VisibleAnywhere, Category = Mover)
	FVector ToLocation = FVector::ZeroVector;

	UPROPERTY(VisibleAnywhere, Category = Mover)
	FQuat ToRotation = FQuat::Identity;

	UPROPERTY(VisibleAnywhere, Category = Mover)
	ETeleportFailureReason TeleportFailureReason = ETeleportFailureReason::Reason_NotAvailable;
};

// WithCopy = true: inherited std::function in FMoverSimulationEventData requires operator= for safe copying via FInstancedStruct.
template<>
struct TStructOpsTypeTraits<FTeleportFailedEventData> : public TStructOpsTypeTraitsBase2<FTeleportFailedEventData>
{
	enum { WithCopy = true };
};

USTRUCT()
struct FMoverGameplayTagChangeEventData : public FMoverSimulationEventData
{
	GENERATED_BODY()

	FMoverGameplayTagChangeEventData(const FMoverTimeStep& InEventTime, const FGameplayTag& InTag, bool bInWasAdded, FEventProcessedCallbackPtr InEventProcessedCallback = nullptr)
		: FMoverSimulationEventData(InEventTime, InEventProcessedCallback)
		, Tag(InTag)
		, bWasAdded(bInWasAdded)
	{
		bReEmitOnResim = true;
	}
	FMoverGameplayTagChangeEventData()
	{
		bReEmitOnResim = true;
	}

	virtual UScriptStruct* GetScriptStruct() const override
	{
		return FMoverGameplayTagChangeEventData::StaticStruct();
	}

	virtual bool IsEqual(const FMoverSimulationEventData& Other, int32 ServerFrameDiffTolerance = 0) const override
	{
		if (Super::IsEqual(Other, ServerFrameDiffTolerance))
		{
			const FMoverGameplayTagChangeEventData& TypedOther = static_cast<const FMoverGameplayTagChangeEventData&>(Other);
			return Tag == TypedOther.Tag && bWasAdded == TypedOther.bWasAdded;
		}
		return false;
	}

	UPROPERTY(VisibleAnywhere, Category = Mover)
	FGameplayTag Tag;

	UPROPERTY(VisibleAnywhere, Category = Mover)
	bool bWasAdded = false;

	// Was this event triggered by an externally-added tag (via AddGameplayTag/RemoveGameplayTag), rather than from within the simulation?
	UPROPERTY(VisibleAnywhere, Category = Mover)
	bool bIsExternalTag = false;
};

// WithCopy = true: inherited std::function in FMoverSimulationEventData requires operator= for safe copying via FInstancedStruct.
template<>
struct TStructOpsTypeTraits<FMoverGameplayTagChangeEventData> : public TStructOpsTypeTraitsBase2<FMoverGameplayTagChangeEventData>
{
	enum { WithCopy = true };
};

namespace UE::Mover
{
	struct FSimulationOutputData
	{
		MOVER_API void Reset();
		MOVER_API void Interpolate(const FSimulationOutputData& From, const FSimulationOutputData& To, float Alpha, double SimTimeMs);
		bool IsValid() const
		{
			return bIsValid && (SyncState.SyncStateCollection.GetDataArray().Num() > 0);
		}

		FMoverSyncState SyncState;
		FMoverInputCmdContext LastUsedInputCmd;
		FMoverDataCollection AdditionalOutputData;
		TArray<TSharedPtr<FMoverSimulationEventData>> Events;
		/** Snapshot of all gameplay tags active on the simulation at the end of this tick, as collected from the simulation. */
		FGameplayTagContainer InternalGameplayTags;
		bool bIsValid = false;
	};

	class FSimulationOutputRecord
	{
	public:
		struct FData
		{
			MOVER_API void Reset();

			FMoverTimeStep TimeStep;
			FSimulationOutputData SimOutputData;
		};

		MOVER_API void Add(const FMoverTimeStep& InTimeStep, const FSimulationOutputData& InData);

		MOVER_API const FSimulationOutputData& GetLatest() const;

		/** This will create an interpolated output and extract events from the stored data with time stamps up until the input time */
		MOVER_API void CreateInterpolatedResult(double AtBaseTimeMs, FMoverTimeStep& OutTimeStep, FSimulationOutputData& OutData);

		MOVER_API void Clear();

		/** Remove all queued events whose ServerFrame is >= FromServerFrame. Call this on the first resim frame after a rollback to discard stale predicted events. */
		MOVER_API void PruneEventsFromFrame(int32 FromServerFrame);

#if !NO_LOGGING
		// Mover Gameplay Tag Logging
		FString DebugOwnerName;
		FString DebugOwnerRole;
#endif

	private:
		FData Data[2];
		TArray<TSharedPtr<FMoverSimulationEventData>> Events;
		uint8 CurrentIndex = 1;

		// Tracks accumulated post-event InternalGameplayTags across multiple calls to
		// CreateInterpolatedResult for the same interpolation window. Incremented by
		// Add() whenever new simulation data arrives; compared in CreateInterpolatedResult to
		// detect whether the Prev..Curr window has changed since the last call.
		int32 NextInterpolatedGeneration = 0;
		int32 LastInterpolatedGeneration = INDEX_NONE;
		FGameplayTagContainer LastInterpolatedTags;

		// Mover Gameplay Tag Logging
		void LogTagInterpolate(uint8 PrevIndex, double AtBaseTimeMs, bool bIsInterpolated) const;
	};

} // namespace UE::Mover


// Params used when initiating a change in movement mode
USTRUCT(BlueprintType)
struct FModeChangeParams
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category = Mover)
	FName DesiredModeName = NAME_None;

	UPROPERTY(BlueprintReadWrite, Category = Mover)
	bool bShouldReenter = false;
};



// Params used when initiating a change in movement mode registration
USTRUCT(BlueprintType)
struct FModeRegistrationParams
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category = Mover)
	FName ModeName = NAME_None;

	UPROPERTY(BlueprintReadWrite, Category = Mover)
	TObjectPtr<UBaseMovementMode> ModeObject = nullptr;

	UPROPERTY(BlueprintReadWrite, Category = Mover)
	bool bIsTheDefaultMode = false;

};
