// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/StringBuilder.h"
#include "Engine/HitResult.h"
#include "MoverLog.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "NativeGameplayTags.h"
#include "MoverTypes.generated.h"

#define UE_API MOVER_API

// Gameplay tags
MOVER_API UE_DECLARE_GAMEPLAY_TAG_EXTERN(Mover_IsOnGround);
MOVER_API UE_DECLARE_GAMEPLAY_TAG_EXTERN(Mover_IsInAir);
MOVER_API UE_DECLARE_GAMEPLAY_TAG_EXTERN(Mover_IsFalling);
MOVER_API UE_DECLARE_GAMEPLAY_TAG_EXTERN(Mover_IsFlying);
MOVER_API UE_DECLARE_GAMEPLAY_TAG_EXTERN(Mover_IsSwimming);
MOVER_API UE_DECLARE_GAMEPLAY_TAG_EXTERN(Mover_IsCrouching);
MOVER_API UE_DECLARE_GAMEPLAY_TAG_EXTERN(Mover_IsNavWalking);
MOVER_API UE_DECLARE_GAMEPLAY_TAG_EXTERN(Mover_AnimRootMotion);
MOVER_API UE_DECLARE_GAMEPLAY_TAG_EXTERN(Mover_SkipAnimRootMotion);
MOVER_API UE_DECLARE_GAMEPLAY_TAG_EXTERN(Mover_SkipVerticalAnimRootMotion);
MOVER_API UE_DECLARE_GAMEPLAY_TAG_EXTERN(Mover_DisableLanding);

/** List of Mover tick phases where different parts of work are being performed */
UENUM(BlueprintType)
enum class EMoverTickPhase : uint8
{
	Invalid = 0             UMETA(Hidden),

	/** This tick is where an input for the next movement step is authored */
	ProduceInput = 1        UMETA(DisplayName = "ProduceInput"),

	/** This tick is where movement based on {input, state} is simulated, to produce a new state */
	SimulateMovement = 2    UMETA(DisplayName = "SimulateMovement"),

	/** This tick is where the newest simulation state is applied to the actor and its components */
	ApplyState = 3          UMETA(DisplayName = "ApplyState")
};

/** List of tick dependency order of execution relative to the Mover tick function. */
UENUM(BlueprintType)
enum class EMoverTickDependencyOrder : uint8
{
	Before = 0,
	After = 1,
};

/** Options for how to handle smoothing frame data from the backend. Typically this is for advancing the simulation at a lower or fixed rate versus the game thread/render rate. */
UENUM(BlueprintType)
enum class EMoverSmoothingMode : uint8
{
	/** Smoothed frames will be ignored */
	None,

	/** Use the smoothed state data to offset the visual root component only, without smoothing the root moving component or any other state data */
	VisualComponentOffset,
};


// Struct to hold params for when an impact happens. This contains all of the data for impacts including what gets passed to the FMover_OnImpact delegate
USTRUCT(BlueprintType, meta = (DisplayName = "Impact Data"))
struct FMoverOnImpactParams
{
	GENERATED_BODY()
	
	UE_API FMoverOnImpactParams();
	
	UE_API FMoverOnImpactParams(const FName& ModeName, const FHitResult& Hit, const FVector& Delta);
	
	// Name of the movement mode this actor is currently in at the time of the impact
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FName MovementModeName;
	
	// The hit result of the impact
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FHitResult HitResult;

	// The original move that was being performed when the impact happened
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	FVector AttemptedMoveDelta;
};

// Struct containing additional context about an event occurrence, such as what led to it
USTRUCT(BlueprintType)
struct FMoverEventContext
{
	GENERATED_BODY()

	UE_API FMoverEventContext();
	UE_API FMoverEventContext(const FMoverTimeStep& InEventTime);

	// Simulation time that this event occurred (in server simulation timespace)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	double EventTimeMs = -1.0;

	// Server frame that this event occurred
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	int32 ServerFrame = INDEX_NONE;

	// Did this event occur while resimulating a past sim frame?
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	uint8 bIsDuringResimulation : 1 = 0;

	// Did this event occur during the instant that the simulation rolled back to an earlier state? Note this is different than during resim. See also @bIsDuringResimulation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	uint8 bIsCausedByRollback : 1 = 0;
};

// A time that can be a frame count or a floating point value.
// When time advances in fixed time steps a frame count will be provided to allow for exact time arithmetic
// A floating point time will also be provided in milliseconds
// When time advances in variable time steps, the frame count will be INDEX_NONE (-1) and only the floating point time will be valid
USTRUCT(BlueprintType)
struct FMoverTime
{
	GENERATED_BODY()

	FString ToString() const
	{
		return (FrameCount != INDEX_NONE) ? FString::Printf(TEXT("FrameCount = %d, TimeMs = %.3f"), FrameCount, TimeMs) : FString::Printf(TEXT("TimeMs = %.3f"), TimeMs);
	}

	// Returns elapsed time in seconds from this time to Other.
	// In fixed time steps (FrameCount != INDEX_NONE), uses integer frame arithmetic for exactness.
	// (but intra step time differences, smaller than FixedFrameDtSeconds, will have to use other functions)
	// In variable time steps (FrameCount == INDEX_NONE), uses floating-point ms arithmetic.
	double ElapsedSecondsTo(const FMoverTime& Other, double FixedFrameDtSeconds) const
	{
		if (FrameCount != INDEX_NONE)
			return (double)(Other.FrameCount - FrameCount) * FixedFrameDtSeconds;
		else
			return (Other.TimeMs - TimeMs) * 0.001;
	}

	// Number of fixed time steps that have occurred since time 0. Only valid when time advances in fixed time steps.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Mover)
	int32 FrameCount=INDEX_NONE;

	// Time in milliseconds. In fixed time steps this should always be equal to FrameCount * Duration of a Fixed Time Step
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Mover)
	double TimeMs=-1.0;
};


USTRUCT(BlueprintType)
struct FMoverTimeStep
{
	GENERATED_BODY()
	

	// The server simulation frame this timestep is associated with. This is the frame that's being worked on now.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Mover)
	int32 ServerFrame=INDEX_NONE;

	// Starting simulation time (in server simulation timespace)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Mover)
	double BaseSimTimeMs=-1.0;

	// The delta time step for this tick
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Mover)
	float StepMs=1.0;

	// Indicates whether this time step is re-simulating based on prior inputs, such as during a correction
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Mover)
	uint8 bIsResimulating : 1 = 0;

	// If resimulating, is this the first frame after the rollback?
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Mover)
	uint8 bIsFirstResimFrame : 1 = 0;

	// Returns an FMoverTime representing the start of this time step.
	// Both FrameCount and TimeMs are always populated; ElapsedSecondsTo uses whichever is appropriate
	// based on the reference FMoverTime's FrameCount (INDEX_NONE = variable dt, otherwise fixed dt).
	FMoverTime ToStartTime() const
	{
		FMoverTime T;
		T.FrameCount = ServerFrame;
		T.TimeMs = BaseSimTimeMs;
		return T;
	}

	// Returns an FMoverTime representing the start of the next step.
	// With regard to time difference calculation, it also represents the time at the end of the current 
	// step, but with a frame count incremented by one. Otherwise the distance between ToStartTime and
	// ToNextTime in async mode would be 0, since it base time arithmetic on frame counts.
	FMoverTime ToNextTime() const
	{
		FMoverTime T;
		T.FrameCount = ServerFrame + 1;
		T.TimeMs = BaseSimTimeMs + (double)StepMs;
		return T;
	}
};



// Base type for all data structs used to compose Mover simulation model definition dynamically (input cmd, sync state, aux state)
// NOTE: for simulation state data (sync/aux), derive from FMoverStateData instead 
USTRUCT(BlueprintType)
struct FMoverDataStructBase
{
	GENERATED_BODY()

	UE_API FMoverDataStructBase();
	virtual ~FMoverDataStructBase() = default;

	/** return newly allocated copy of this FMoverDataStructBase. MUST be overridden by derived types. */
	UE_API virtual FMoverDataStructBase* Clone() const;

	virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
	{
		bOutSuccess = true;
		return true;
	}

	/** Gets the type info of this FMoverDataStructBase. MUST be overridden by derived types. */
	UE_API virtual UScriptStruct* GetScriptStruct() const;

	/** Get string representation of this struct instance */
	virtual void ToString(FAnsiStringBuilderBase& Out) const {}

	/** If derived classes hold any object references, override this function and add them to the collector. */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) {}

	/**
	 * Checks if the contained data is equal, within reason.
	 * AuthorityState is guaranteed to be the same concrete type as 'this'.
	 * Override for: All types that compose STATE data (sync or aux) and types that compose INPUT data for physics-based movement
	 */
	UE_API virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const;

	/**
	 * Interpolates contained data between a starting and ending block.
	 * From and To are guaranteed to be the same concrete type as 'this'.
	 * Override for: All types that compose STATE data (sync or aux) and types that compose INPUT data for physics-based movement  
	 */
	UE_API virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float Pct);

	/**
	 * Merges contained data from a previous frame with that of the current frame. For more information, see the comments on FNetworkPhysicsData::MergeData.
	 * From is guaranteed to be the same concrete type as 'this'.
	 * Override for: Types that compose INPUT data for physics-based movement
	 */
	UE_API virtual void Merge(const FMoverDataStructBase& From);
	
	/**
	 * Decays contained data during resimulation if data is forward predicted. For more information, see the comments on FNetworkPhysicsData::DecayData.
	 * @param DecayAmount = Total amount of decay as a multiplier. 10% decay = 0.1.
	 * Override for: Types that compose INPUT data for physics-based movement
	 */
	virtual void Decay(float DecayAmount) {};


	/** Used to differentiate between this type, and a proxy data type. RARELY overridden by derived types. */
	UE_API virtual const UScriptStruct* GetDataScriptStruct() const;
};

template<>
struct TStructOpsTypeTraits< FMoverDataStructBase > : public TStructOpsTypeTraitsBase2< FMoverDataStructBase >
{
	enum
	{
		WithNetSerializer = true,
		WithCopy = true
	};
};




// Contains a group of different FMoverDataStructBase-derived data, and supports net serialization of them. Note that
//	each contained data must have a unique type.  This is to support dynamic composition of Mover simulation model
//  definitions (input cmd, sync state, aux state).
USTRUCT(BlueprintType)
struct FMoverDataCollection
{
	GENERATED_BODY()

	UE_API FMoverDataCollection();

	void Empty() { DataArray.Empty(); }

	/** Serialize all data in this collection */
	UE_API bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess);

	/** Serializes data in this collection for debug purposes.
	*   This is currently only usable in the context of sending mover info to the Chaos Visual Debugger
	*/
	UE_API bool SerializeDebugData(FArchive& Ar);

	/** Copy operator - deep copy so it can be used for archiving/saving off data */
	UE_API FMoverDataCollection& operator=(const FMoverDataCollection& Other);

	/** Comparison operator (deep) - needs matching struct types along with identical states in those structs. See also ShouldReconcile */
	UE_API bool operator==(const FMoverDataCollection& Other) const;

	/** Comparison operator */
	UE_API bool operator!=(const FMoverDataCollection& Other) const;

	/** Checks if the collections are significantly different enough (piece-wise) to need reconciliation. NOT an equality check. */
	UE_API bool ShouldReconcile(const FMoverDataCollection& Other) const;

	/** Make this collection a piece-wise interpolation between 2 collections */
	UE_API void Interpolate(const FMoverDataCollection& From, const FMoverDataCollection& To, float Pct);

	/** Merge a previous frame's collection with this collection */
	UE_API void Merge(const FMoverDataCollection& From);

	/** Decay input based on DecayAmount for resimulation and forward prediction */
	UE_API void Decay(float DecayAmount);

	/** Checks only whether there are matching structs inside, but NOT necessarily identical states of each one */
	UE_API bool HasSameContents(const FMoverDataCollection& Other) const;

	/** Exposes references to GC system */
	UE_API void AddStructReferencedObjects(FReferenceCollector& Collector) const;

	/** Get string representation of all elements in this collection */
	UE_API void ToString(FAnsiStringBuilderBase& Out) const;

	/** Const access to data array of collections */
	UE_API TArray<TSharedPtr<FMoverDataStructBase>>::TConstIterator GetCollectionDataIterator() const;
	
	/** Find data of a specific type in the collection (mutable version). If not found, null will be returned. */
	template <typename T>
	T* FindMutableDataByType() const
	{
		if (FMoverDataStructBase* FoundData = FindDataByType(T::StaticStruct()))
		{
			return static_cast<T*>(FoundData);
		}

		return nullptr;
	}

	/** Find data of a specific type in the collection. If not found, null will be returned. */
	template <typename T>
	const T* FindDataByType() const
	{
		if (const FMoverDataStructBase* FoundData = FindDataByType(T::StaticStruct()))
		{
			return static_cast<const T*>(FoundData);
		}

		return nullptr;
	}

	/** Find data of a specific type in the collection. If not found, a new default instance will be added. */
	template <typename T>
	const T& FindOrAddDataByType()
	{
		if (const T* ExistingData = FindDataByType<T>())
		{
			return *ExistingData;
		}

		return *(static_cast<const T*>(AddDataByType(T::StaticStruct())));
	}

	/** Find data of a specific type in the collection (mutable version). If not found, a new default instance will be added. */
	template <typename T>
	T& FindOrAddMutableDataByType()
	{
		if (T* ExistingData = FindMutableDataByType<T>())
		{
			return *ExistingData;
		}

		return *(static_cast<T*>(AddDataByType(T::StaticStruct())));
	}

	/** Find data of a specific type in the collection (mutable version). If not found, a new default instance will be added. Outputs if the state was added or not. */
	template <typename T>
	T& FindOrAddMutableDataByType(bool& bOutAdded)
	{
		if (T* ExistingData = FindMutableDataByType<T>())
		{
			bOutAdded = false;
			return *ExistingData;
		}

		bOutAdded = true;
		return *(static_cast<T*>(AddDataByType(T::StaticStruct())));
	}

	/** Adds this data instance to the collection, taking ownership of it. If an existing data struct of the same type is already there, it will be removed first. */
	UE_API void AddOrOverwriteData(const TSharedPtr<FMoverDataStructBase> DataInstance);

	/** Adds data to the collection by copying over an existing struct or cloning the provided struct if no matching struct exists. 
	 *  This is different than AddOrOverwriteData because the instance isn't touched, avoiding memory allocation & array changing if a matching struct exists.
	 */
	UE_API void AddDataByCopy(const FMoverDataStructBase* DataInstanceToCopy);


	const TArray< TSharedPtr<FMoverDataStructBase> >& GetDataArray() const
	{
		return DataArray;
	}

	/** Find data of a specific type in the collection. */
	UE_API FMoverDataStructBase* FindDataByType(const UScriptStruct* DataStructType) const;

	/** Find data of a specific type in the collection. If not found, a new default instance will be added. */
	UE_API FMoverDataStructBase* FindOrAddDataByType(const UScriptStruct* DataStructType);

	/** Removes data of a specific type in the collection. Returns true if data was removed. */
	UE_API bool RemoveDataByType(const UScriptStruct* DataStructType);

protected:
	UE_API FMoverDataStructBase* AddDataByType(const UScriptStruct* DataStructType);
	static UE_API TSharedPtr<FMoverDataStructBase> CreateDataByType(const UScriptStruct* DataStructType);
	
	/** Helper function for serializing array of data */
	static UE_API void NetSerializeDataArray(FArchive& Ar, UPackageMap* Map, TArray<TSharedPtr<FMoverDataStructBase>>& DataArray);

	/** All data in this collection */
	TArray< TSharedPtr<FMoverDataStructBase> > DataArray;


friend class UMoverDataCollectionLibrary;
friend class UMoverComponent;
};


template<>
struct TStructOpsTypeTraits<FMoverDataCollection> : public TStructOpsTypeTraitsBase2<FMoverDataCollection>
{
	enum
	{
		WithCopy = true,		// Necessary so that DataArray is copied around
		WithNetSerializer = true,
		WithIdenticalViaEquality = true,
		WithAddStructReferencedObjects = true,
	};
};

/** Info about data collection types that should always be present, and how they should propagate from one frame to the next */
USTRUCT(BlueprintType, meta = (DisplayName = "Persistent Data Settings"))
struct FMoverDataPersistence
{
	GENERATED_BODY()

	FMoverDataPersistence() {}

	FMoverDataPersistence(UScriptStruct* TypeToPersist, bool bShouldCopyBetweenFrames) 
	{
		RequiredType = TypeToPersist;
		bCopyFromPriorFrame = bShouldCopyBetweenFrames;
	}

	// The type that should propagate between frames
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover, meta=(MetaStruct="/Script/Mover.MoverDataStructBase"))
	TObjectPtr<UScriptStruct> RequiredType = nullptr;

	// If true, values will be copied from the prior frame. Otherwise, they will be default-initialized.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mover)
	bool bCopyFromPriorFrame = true;
};

// Information about a change in movement mode
struct FMovementModeChangeRecord
{
	FName ModeName;
	FName PrevModeName;
	uint32 Frame;
	double SimTimeMs;
};



// Blueprint helper functions for working with a Mover data collection
UCLASS(MinimalAPI)
class UMoverDataCollectionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	* Add a data struct to the collection, overwriting an existing one with the same type
	* @param SourceAsRawBytes		The data struct instance to add by copy, which must be a FMoverDataStructBase sub-type
	*/
	UFUNCTION(BlueprintCallable, CustomThunk, Category = "Mover|Data", meta = (CustomStructureParam = "SourceAsRawBytes", AllowAbstract = "false", DisplayName = "Add Data To Collection"))
	static UE_API void K2_AddDataToCollection(UPARAM(Ref) FMoverDataCollection& Collection, UPARAM(DisplayName="Struct To Add") const int32& SourceAsRawBytes);

	/**
	 * Retrieves data from a collection, by writing to a target instance if it contains one of the matching type.  Changes must be written back using AddDataToCollection.
	 * @param DidSucceed			Flag indicating whether data was actually written to target struct instance
	 * @param TargetAsRawBytes		The data struct instance to write to, which must be a FMoverDataStructBase sub-type
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = "Mover|Data", meta = (CustomStructureParam = "TargetAsRawBytes", AllowAbstract = "false", DisplayName = "Get Data From Collection"))
	static UE_API void K2_GetDataFromCollection(bool& DidSucceed, UPARAM(Ref) const FMoverDataCollection& Collection, UPARAM(DisplayName = "Out Struct") int32& TargetAsRawBytes);

	/**
	* Clears all data from a collection
	*/
	UFUNCTION(BlueprintCallable, Category = "Mover|Data", meta=(DisplayName = "Clear Data From Collection"))
	static UE_API void ClearDataFromCollection(UPARAM(Ref) FMoverDataCollection& Collection);

private:
	DECLARE_FUNCTION(execK2_AddDataToCollection);
	DECLARE_FUNCTION(execK2_GetDataFromCollection);
};

#undef UE_API
