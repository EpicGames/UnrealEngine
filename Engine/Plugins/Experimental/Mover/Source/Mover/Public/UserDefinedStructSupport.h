// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoverTypes.h"
#include "StructUtils/InstancedStruct.h"
#include "UserDefinedStructSupport.generated.h"

class UUserDefinedStruct;


#define UE_API MOVER_API

/** Wrapper class that's used to add User-Defined Struct instances to Mover Data Collections (input or state).
 * This allows devs to add custom data to inputs and/or state without requiring native code.
 * Note that these are typically less efficient than natively-defined structs, and the logic of operations
 * like interpolation, merging, and serialization may be simplistic for a project's needs.
 * At present:
 * - any differences between any struct contents will trigger reconciliation, even small floating point number differences
 * - only boolean values can be merged
 * - interpolation will take the entire struct instance from the highest weight frame
 */
USTRUCT()
struct FMoverUserDefinedDataStruct : public FMoverDataStructBase
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FInstancedStruct StructInstance;	// this always represents a User-Defined Struct from Blueprint

	// Implementation of FMoverDataStructBase
	virtual bool ShouldReconcile(const FMoverDataStructBase& AuthorityState) const override;
	virtual void Interpolate(const FMoverDataStructBase& From, const FMoverDataStructBase& To, float LerpFactor) override;
	virtual void Merge(const FMoverDataStructBase& From) override;
	virtual FMoverDataStructBase* Clone() const override;
	virtual bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess) override;
	virtual void ToString(FAnsiStringBuilderBase& Out) const override;

	// Note: this is the FMoverUserDefinedDataStruct type, NOT the User-Defined Struct type
	virtual UScriptStruct* GetScriptStruct() const override { return StaticStruct(); }
	// This returns the User-Defined Struct type
	virtual const UScriptStruct* GetDataScriptStruct() const override;

	static bool NetSerializeUserDefinedStruct(FInstancedStruct& UDSStructInstance, FArchive& Ar, UPackageMap* Map);
};


#undef UE_API
