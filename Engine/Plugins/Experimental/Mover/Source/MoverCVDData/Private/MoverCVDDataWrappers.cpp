// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoverCVDDataWrappers.h"

#include "Logging/LogMacros.h"
#include UE_INLINE_GENERATED_CPP_BY_NAME(MoverCVDDataWrappers)

DEFINE_LOG_CATEGORY_STATIC(LogMoverCVDData, Log, All);

FStringView FMoverCVDSimDataWrapper::WrapperTypeName = TEXT("FMoverCVDSimDataWrapper");

bool FMoverCVDSimDataWrapper::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;

	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << SolverID;
	Ar << ParticleID;
	Ar << SyncStateBytes;
	Ar << SyncStateDataCollectionBytes;
	Ar << InputCmdBytes;
	Ar << InputMoverDataCollectionBytes;

	// Serialize named sections individually so section keys (FNames) are tracked by the CVD name table
	// when the outer archive is an FChaosVDMemoryWriter/Reader.
	int32 NumSections = LocalSimDataSections.Num();
	Ar << NumSections;
	if (Ar.IsLoading())
	{
		// A single particle's local sim data will only ever have a small number of named sections
		// (e.g. LocalSimInput, InternalSimData, DebugSimData). Guard against corrupt/malicious data
		// that could cause a huge or negative allocation.
		constexpr int32 MaxExpectedSections = 256;
		if (NumSections < 0 || NumSections > MaxExpectedSections)
		{
			UE_LOGF(LogMoverCVDData, Warning, "FMoverCVDSimDataWrapper::Serialize - Invalid NumSections (%d), expected 0..%d. Data may be corrupt.", NumSections, MaxExpectedSections);
			Ar.SetError();
			return false;
		}
		LocalSimDataSections.SetNum(NumSections);
	}
	for (TPair<FName, TArray<uint8>>& Section : LocalSimDataSections)
	{
		Ar << Section.Key;
		Ar << Section.Value;
	}

	return !Ar.IsError();
}
