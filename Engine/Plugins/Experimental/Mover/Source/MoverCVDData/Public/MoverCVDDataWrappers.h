// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataWrappers/ChaosVDDataSerializationMacros.h"
#include "DataWrappers/ChaosVDParticleDataWrapper.h"
#include "UObject/ObjectMacros.h"

#include "MoverCVDDataWrappers.generated.h"

USTRUCT(DisplayName="Mover Sim Data")
struct FMoverCVDSimDataWrapper : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()
	
	MOVERCVDDATA_API static FStringView WrapperTypeName;

	UPROPERTY(VisibleAnywhere, Category="Mover Info")
	int32 SolverID = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category="Mover Info")
	int32 ParticleID = INDEX_NONE;

	TArray<uint8> SyncStateBytes;
	TArray<uint8> SyncStateDataCollectionBytes;
	TArray<uint8> InputCmdBytes;
	TArray<uint8> InputMoverDataCollectionBytes;

	/** Named sections of local simulation data (e.g. LocalSimInput, InternalSimData, DebugSimData).
	 *  Each section is serialized independently so the origin of each struct is preserved in the CVD viewer. */
	TArray<TPair<FName, TArray<uint8>>> LocalSimDataSections;

	MOVERCVDDATA_API bool Serialize(FArchive& Ar);
};

CVD_IMPLEMENT_SERIALIZER(FMoverCVDSimDataWrapper)

USTRUCT()
struct FMoverCVDSimDataContainer
{
	GENERATED_BODY()

	TMap<int32, TArray<TSharedPtr<FMoverCVDSimDataWrapper>>> SimDataBySolverID;
};