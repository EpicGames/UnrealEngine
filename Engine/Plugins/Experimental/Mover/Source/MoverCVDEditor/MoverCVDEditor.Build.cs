// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MoverCVDEditor : ModuleRules
{
    public MoverCVDEditor(ReadOnlyTargetRules Target) : base(Target)
    {
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core"
			});

		PrivateDependencyModuleNames.AddRange(
		    new string[]
		    {
				"ApplicationCore",
				"ChaosVD",
				"ChaosVDData",
				"ChaosVDRuntime",
				"CoreUObject",
				"EditorWidgets",
				"Engine",
				"Mover",
				"MoverCVDData",
				"Projects",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"TraceServices",
				"TypedElementFramework",
				"UnrealEd",
			}
		);
        
        SetupModulePhysicsSupport(Target);

		SetupModuleChaosVisualDebuggerSupport(Target);
	}
}