// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoverCVDEditor.h"

#include "MoverCVDExtension.h"
#include "ExtensionsSystem/ChaosVDExtensionsManager.h"

#define LOCTEXT_NAMESPACE "FMoverCVDEditorModule"

void FMoverCVDEditorModule::StartupModule()
{
	TSharedRef<FMoverCVDExtension> NewExtension = MakeShared<FMoverCVDExtension>();
	FChaosVDExtensionsManager::Get().RegisterExtension(NewExtension);
	AvailableExtensions.Add(NewExtension);
}

void FMoverCVDEditorModule::ShutdownModule()
{
	for (const TWeakPtr<FChaosVDExtension>& Extension : AvailableExtensions)
	{
		if (const TSharedPtr<FChaosVDExtension>& ExtensionPtr = Extension.Pin())
		{
			FChaosVDExtensionsManager::Get().UnRegisterExtension(ExtensionPtr.ToSharedRef());
		}
	}

	AvailableExtensions.Reset();
}

#undef LOCTEXT_NAMESPACE
    
IMPLEMENT_MODULE(FMoverCVDEditorModule, MoverCVDEditor)