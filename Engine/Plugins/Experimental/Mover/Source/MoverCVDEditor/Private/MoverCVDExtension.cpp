// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoverCVDExtension.h"

#include "MoverCVDSimDataComponent.h"
#include "MoverCVDSimDataProcessor.h"
#include "Widgets/SChaosVDMainTab.h"
#include "MoverCVDTab.h"
#include "MoverCVDStyle.h"

namespace NMoverCVDExtension
{
	static const FName MoverTabName = FName(TEXT("Mover Info"));
	static const FName ExtensionName = FName(TEXT("FMoverCVDExtension"));
};

FMoverCVDExtension::FMoverCVDExtension() : FChaosVDExtension()
{
	DataComponentsClasses.Add(UMoverCVDSimDataComponent::StaticClass());

	ExtensionName = NMoverCVDExtension::ExtensionName;

	FMoverCVDStyle::Initialize();
}

FMoverCVDExtension::~FMoverCVDExtension()
{
	DataComponentsClasses.Reset();

	FMoverCVDStyle::Shutdown();
}

void FMoverCVDExtension::RegisterDataProcessorsInstancesForProvider(const TSharedRef<FChaosVDTraceProvider>& InTraceProvider)
{
	FChaosVDExtension::RegisterDataProcessorsInstancesForProvider(InTraceProvider);

    TSharedPtr<FMoverCVDSimDataProcessor> SimDataProcessor = MakeShared<FMoverCVDSimDataProcessor>();
    SimDataProcessor->SetTraceProvider(InTraceProvider);
    InTraceProvider->RegisterDataProcessor(SimDataProcessor);
}

TConstArrayView<TSubclassOf<UActorComponent>> FMoverCVDExtension::GetSolverDataComponentsClasses()
{
	return DataComponentsClasses;
}

void FMoverCVDExtension::RegisterCustomTabSpawners(const TSharedRef<SChaosVDMainTab>& InParentTabWidget)
{
	InParentTabWidget->RegisterTabSpawner<FMoverCVDTab>(NMoverCVDExtension::MoverTabName);
}
