// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ExtensionsSystem/ChaosVDExtension.h"
#include "Templates/SubclassOf.h"
#include "Templates/SharedPointer.h"

class FChaosVDTraceProvider;
class UActorComponent;
class SChaosVDMainTab;

/** MoverCVDExtension is where we register MoverCVDTab as a displayable tab, register MoverCVDSimDataProcessor and give access to the MoverSimDataComponent */
class FMoverCVDExtension final : public FChaosVDExtension
{
public:
	
	FMoverCVDExtension();
	virtual ~FMoverCVDExtension() override;

	virtual void RegisterDataProcessorsInstancesForProvider(const TSharedRef<FChaosVDTraceProvider>& InTraceProvider) override;
	virtual TConstArrayView<TSubclassOf<UActorComponent>> GetSolverDataComponentsClasses() override;

	// Registers all available Tab Spawner instances in this extension, if any
	virtual void RegisterCustomTabSpawners(const TSharedRef<SChaosVDMainTab>& InParentTabWidget) override;

private:
	TArray<TSubclassOf<UActorComponent>> DataComponentsClasses;
};
