// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoverCVDSimDataComponent.h"

#include "ChaosVDRecording.h"
#include "ChaosVDScene.h"
#include "MoverCVDDataWrappers.h"
#include "Actors/ChaosVDDataContainerBaseActor.h"
#include "MoverCVDTab.h"
#include "MoverSimulationTypes.h"
#include "ChaosVisualDebugger/MoverCVDRuntimeTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoverCVDSimDataComponent)

void UMoverCVDSimDataComponent::UpdateFromSolverFrameData(const FChaosVDSolverFrameData& InSolverFrameData)
{
	Super::UpdateFromSolverFrameData(InSolverFrameData);

	if (TSharedPtr<FMoverCVDSimDataContainer> SimDataContainer = InSolverFrameData.GetCustomData().GetData<FMoverCVDSimDataContainer>())
	{
		if (const TArray<TSharedPtr<FMoverCVDSimDataWrapper>>* RecordedData = SimDataContainer->SimDataBySolverID.Find(SolverID))
		{
			// Load the recorded data into the Physics Mover CVD component
			FrameSimDataArray.Reset(RecordedData->Num());
			FrameSimDataArray = *RecordedData;

			// Also, clear all cached deserialized data, we're starting from scratch at a new frame
			DeserializedStates.Empty();
		}
	}
}

void UMoverCVDSimDataComponent::ClearData()
{
	FrameSimDataArray.Reset();
}

bool UMoverCVDSimDataComponent::FindAndUnwrapSimDataForParticle(uint32 ParticleID, TSharedPtr<FMoverCVDSimDataWrapper>& OutSimDataWrapper, TSharedPtr<FMoverSyncState>& OutSyncState, TSharedPtr<FMoverInputCmdContext>& OutInputCmd, TArray<TPair<FName, TSharedPtr<FMoverDataCollection>>>& OutLocalSimDataSections)
{
	// Look for a sim data corresponding to ParticleID
	TSharedPtr<FMoverCVDSimDataWrapper>* FoundSimData = FrameSimDataArray.FindByPredicate
		(
			[&](const TSharedPtr<FMoverCVDSimDataWrapper>& SimData)
			{
				return (SimData->HasValidData() && (SimData->ParticleID == ParticleID));
			}
		);

	if (!FoundSimData || !*FoundSimData)
	{
		return false;
	}

	// We use the data wrapper pointer as key in the arrays of deserialized structs (input command, sync state)
	FMoverCVDSimDataWrapper* SimDataPtr = (*FoundSimData).Get();
	OutSimDataWrapper = *FoundSimData;

	// Did we previously deserialize?
	TSharedPtr<FDeserializedMoverStates> DeserializedMoverStates = DeserializedStates.FindOrAdd(SimDataPtr);
	if (!DeserializedMoverStates)
	{
		DeserializedMoverStates = MakeShared<FDeserializedMoverStates>();
		// This means that the SimData wasn't deserialized yet, so we do it now
		// Otherwise we use the cached version
		UE::MoverUtils::FMoverCVDRuntimeTrace::UnwrapSimData(*SimDataPtr, DeserializedMoverStates->InputCommand, DeserializedMoverStates->SyncState, DeserializedMoverStates->LocalSimDataSections);
	}
	if (DeserializedMoverStates)
	{
		OutSyncState = DeserializedMoverStates->SyncState;
		OutInputCmd = DeserializedMoverStates->InputCommand;
		OutLocalSimDataSections = DeserializedMoverStates->LocalSimDataSections;
	}

	return true;
}
