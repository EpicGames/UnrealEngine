// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ChaosVDSolverDataComponent.h"
#include "MoverCVDSimDataComponent.generated.h"

struct FMoverSyncState;
struct FMoverCVDSimDataWrapper;
struct FMoverInputCmdContext;
struct FMoverDataCollection;

struct FDeserializedMoverStates
{
	TSharedPtr<FMoverSyncState> SyncState;
	TSharedPtr<FMoverInputCmdContext> InputCommand;
	TArray<TPair<FName, TSharedPtr<FMoverDataCollection>>> LocalSimDataSections;
};

/** Component holding Mover data for the current visualized frame */
UCLASS()
class UMoverCVDSimDataComponent : public UChaosVDSolverDataComponent
{
	GENERATED_BODY()

public:
	// That we chose to implement this function and not UpdateFromNewGameFrameData or UpdateFromNewSolverStageData is tied
	// to the implementation of FMoverCVDSimDataProcessor, which currently add ths information to FChaosVDTraceProvider::GetCurrentSolverFrame()
	// Eventually we will record information at different stages of a solver frame and will be using UpdateFromNewSolverStageData instead,
	// to show the state of the sync state at the beginning of the frame, then at the end
	virtual void UpdateFromSolverFrameData(const FChaosVDSolverFrameData& InSolverFrameData) override;

	virtual void ClearData() override;

	TConstArrayView<TSharedPtr<FMoverCVDSimDataWrapper>> GetFrameSimDataArray() const
	{
		return FrameSimDataArray;
	}

	bool FindAndUnwrapSimDataForParticle(
		uint32 ParticleID,
		TSharedPtr<FMoverCVDSimDataWrapper>& OutSimDataWrapper,
		TSharedPtr<FMoverSyncState>& OutSyncState,
		TSharedPtr<FMoverInputCmdContext>& OutInputCmd,
		TArray<TPair<FName, TSharedPtr<FMoverDataCollection>>>& OutLocalSimDataSections);

private:
	// This is the array of FMoverCVDSimDataWrapper for the current frame (as it is updated in UpdateFromNewGameFrameData)
	// and corresponding to this DataComponent's SolverID
	TArray<TSharedPtr<FMoverCVDSimDataWrapper>> FrameSimDataArray;

	TMap<FMoverCVDSimDataWrapper*, TSharedPtr<FDeserializedMoverStates>> DeserializedStates;
};
