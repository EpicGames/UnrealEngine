// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoverCVDSimDataProcessor.h"

#include "MoverCVDDataWrappers.h"

FMoverCVDSimDataProcessor::FMoverCVDSimDataProcessor() : FChaosVDDataProcessorBase(FMoverCVDSimDataWrapper::WrapperTypeName)
{
	// Uses SerializeBin on raw structs with no explicit versioning  - not safe to read across binary-incompatible builds.
	bBackwardsCompatible = false;
}

bool FMoverCVDSimDataProcessor::ProcessRawData(const TArray<uint8>& InData)
{
	FChaosVDDataProcessorBase::ProcessRawData(InData);

	const TSharedPtr<FChaosVDTraceProvider> ProviderSharedPtr = TraceProvider.Pin();
	if (!ensure(ProviderSharedPtr.IsValid()))
	{
		return false;
	}

	const TSharedPtr<FMoverCVDSimDataWrapper> SimData = MakeShared<FMoverCVDSimDataWrapper>();
	const bool bSuccess = Chaos::VisualDebugger::ReadDataFromBuffer(InData, *SimData, ProviderSharedPtr.ToSharedRef());
	
	if (bSuccess)
	{
		if (FChaosVDSolverFrameData* CurrentSolverFrameData = ProviderSharedPtr->GetCurrentSolverFrame(SimData->SolverID))
		{
			if (TSharedPtr<FMoverCVDSimDataContainer> SimDataContainer = CurrentSolverFrameData->GetCustomData().GetOrAddDefaultData<FMoverCVDSimDataContainer>())
			{
				SimDataContainer->SimDataBySolverID.FindOrAdd(SimData->SolverID).Add(SimData);
			}
		}
	}

	return bSuccess;
}
