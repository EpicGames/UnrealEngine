// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoverCVDSimDataSettings.h"

#include "ChaosVDSettingsManager.h"
#include "Utils/ChaosVDUserInterfaceUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoverCVDSimDataSettings)

void UMoverCVDSimDataSettings::SetDataVisualizationFlags(EMoverCVDSimDataVisualizationFlags NewFlags)
{
	if (UMoverCVDSimDataSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UMoverCVDSimDataSettings>())
	{
		Settings->DebugDrawFlags = static_cast<uint32>(NewFlags);
		Settings->BroadcastSettingsChanged();
	}
}

EMoverCVDSimDataVisualizationFlags UMoverCVDSimDataSettings::GetDataVisualizationFlags()
{
	if (UMoverCVDSimDataSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UMoverCVDSimDataSettings>())
	{
		return static_cast<EMoverCVDSimDataVisualizationFlags>(Settings->DebugDrawFlags);
	}

	return EMoverCVDSimDataVisualizationFlags::None;
}

bool UMoverCVDSimDataSettings::CanVisualizationFlagBeChangedByUI(uint32 Flag)
{
	return Chaos::VisualDebugger::Utils::ShouldVisFlagBeEnabledInUI(Flag, DebugDrawFlags, EMoverCVDSimDataVisualizationFlags::EnableDraw);
}
