// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Settings/ChaosVDCoreSettings.h"
#include "MoverCVDSimDataSettings.generated.h"

/** Set of visualization flags options for Mover Sim Data */
UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EMoverCVDSimDataVisualizationFlags : uint32
{
	None					= 0 UMETA(Hidden),
	EnableDraw				= 1 << 0,
};
ENUM_CLASS_FLAGS(EMoverCVDSimDataVisualizationFlags);

UCLASS(config=ChaosVD, PerObjectConfig)
class UMoverCVDSimDataSettings : public UChaosVDVisualizationSettingsObjectBase
{
	GENERATED_BODY()

public:

	/** If true, any debug draw text available will be drawn */
	UPROPERTY(EditAnywhere, Category=DebugDraw)
	bool bShowDebugText = false;

	/** The depth priority used for while drawing data. Can be World or Foreground (with this one the shapes will be drawn on top of the geometry and be always visible) */
	UPROPERTY(EditAnywhere, Category=DebugDraw)
	TEnumAsByte<ESceneDepthPriorityGroup> DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground;

	/** Thickness to apply to any debug draw shape controlled by this setting. */
	UPROPERTY(EditAnywhere, config, Category=DebugDraw)
	float BaseThickness = 2.0f;

	static void SetDataVisualizationFlags(EMoverCVDSimDataVisualizationFlags NewFlags);
	static EMoverCVDSimDataVisualizationFlags GetDataVisualizationFlags();

	virtual bool CanVisualizationFlagBeChangedByUI(uint32 Flag) override;

private:

	/** Set of flags to enable/disable visualization of debug draw data shapes */
	UPROPERTY(config, meta = (Bitmask, BitmaskEnum = "/Script/MoverCVDEditor.EMoverCVDSimDataVisualizationFlags"))
	uint32 DebugDrawFlags = static_cast<uint32>(EMoverCVDSimDataVisualizationFlags::None);
};