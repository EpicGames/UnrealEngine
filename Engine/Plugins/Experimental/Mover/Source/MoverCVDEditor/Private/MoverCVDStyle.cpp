// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoverCVDStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IPluginManager.h"
#include "Slate/SlateGameResources.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

#define RootToContentDir Style->RootToContentDir

TSharedPtr<FSlateStyleSet> FMoverCVDStyle::StyleInstance = nullptr;

void FMoverCVDStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FMoverCVDStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

FName FMoverCVDStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("MoverCVDStyle"));
	return StyleSetName;
}

const FVector2D Icon16x16(16.0f, 16.0f);

TSharedRef< FSlateStyleSet > FMoverCVDStyle::Create()
{
	TSharedRef< FSlateStyleSet > Style = MakeShareable(new FSlateStyleSet("MoverCVDStyle"));
	Style->SetContentRoot(IPluginManager::Get().FindPlugin("Mover")->GetBaseDir() / TEXT("Resources"));

	Style->Set("TabIconMoverInfoPanel",  new IMAGE_BRUSH_SVG(TEXT("MoverInfo"), Icon16x16));

	return Style;
}

void FMoverCVDStyle::ReloadTextures()
{
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
	}
}

const ISlateStyle& FMoverCVDStyle::Get()
{
	return *StyleInstance;
}
