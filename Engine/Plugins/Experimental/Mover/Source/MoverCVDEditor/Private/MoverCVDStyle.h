// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FName;
class FSlateStyleSet;
class ISlateStyle;

class FMoverCVDStyle
{
public:

	static void Initialize();

	static void Shutdown();

	static void ReloadTextures();

	static const ISlateStyle& Get();

	static FName GetStyleSetName();

private:

	static TSharedRef<FSlateStyleSet> Create();

	static TSharedPtr<FSlateStyleSet> StyleInstance;
};