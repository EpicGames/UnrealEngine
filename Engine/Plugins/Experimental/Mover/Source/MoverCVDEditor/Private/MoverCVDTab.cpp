// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoverCVDTab.h"

#include "ChaosLog.h"
#include "ChaosVDModule.h"
#include "ChaosVDScene.h"
#include "Framework/Docking/TabManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SChaosVDMainTab.h"
#include "MoverCVDStyle.h"
#include "MoverSimulationTypes.h"
#include "MoverCVDDataWrappers.h"
#include "MoverCVDSimDataComponent.h"
#include "TEDS/ChaosVDSelectionInterface.h"
#include "ChaosVDSceneParticle.h"

#define LOCTEXT_NAMESPACE "MoverCVDExtension"

FMoverCVDTab::FMoverCVDTab(const FName& InTabID, TSharedPtr<FTabManager> InTabManager, TWeakPtr<SChaosVDMainTab> InOwningTabWidget)
	: FChaosVDObjectDetailsTab(InTabID, InTabManager, InOwningTabWidget)
{
	// Cache a scene pointer for future reference
	if (TSharedPtr<SChaosVDMainTab> OwningTabWidgetSharedPtr = OwningTabWidget.Pin())
	{
		if (TSharedPtr<FChaosVDScene> Scene = OwningTabWidgetSharedPtr->GetScene())
		{
			SceneWeakPtr = Scene;
		}
	}
}

FMoverCVDTab::~FMoverCVDTab()
{
	if (TSharedPtr<FChaosVDScene> Scene = SceneWeakPtr.Pin())
	{
		Scene->OnSceneUpdated().RemoveAll(this);
	}
}

TSharedRef<SDockTab> FMoverCVDTab::HandleTabSpawnRequest(const FSpawnTabArgs& Args)
{
	// I can't add callbacks in the constructor as SharedFromThis would complain
	if (TSharedPtr<FChaosVDScene> Scene = SceneWeakPtr.Pin())
	{
		Scene->OnSceneUpdated().AddSP(this, &FMoverCVDTab::HandleSceneUpdated);
	}

	TSharedRef<SDockTab> Tab = FChaosVDObjectDetailsTab::HandleTabSpawnRequest(Args);

	// Let's rename the tab so its name is unique
	Tab->SetLabel(Args.GetTabId().ToText());

	Tab->SetTabIcon(FMoverCVDStyle::Get().GetBrush("TabIconMoverInfoPanel"));

	return Tab;
}

void FMoverCVDTab::HandleSolverDataSelectionChange(const TSharedPtr<FChaosVDSolverDataSelectionHandle>& SelectionHandle)
{
	// We override the default behavior, otherwise we will display any struct that gets selected and can be displayed
	// while we only want to display mover info
	return;
}

void FMoverCVDTab::RetrieveAllSolversMoverDataComponents()
{
	SolverToSimDataComponentMap.Empty();

	const TSharedPtr<FChaosVDScene> Scene = SceneWeakPtr.Pin();
	if (Scene)
	{
		const FChaosVDSolverInfoByIDMap& SolverInfoByIDMap = Scene->GetSolverInfoActorsMap();
		for (const auto& It : SolverInfoByIDMap)
		{
			int32 SolverID = It.Key;
			// No need to know about AChaosVDSolverInfoActor as we intend to solely find an actor component,
			// but also at the time of writing, AChaosVDSolverInfoActor was no a public include
			const AActor* SolverInfoActor = reinterpret_cast<const AActor*>(It.Value);
			if (SolverInfoActor)
			{
				if (UMoverCVDSimDataComponent* SolverSimDataComponent = SolverInfoActor->FindComponentByClass<UMoverCVDSimDataComponent>())
				{
					SolverToSimDataComponentMap.Add(SolverID, SolverSimDataComponent);
				}
			}
		}
	}
}

TWeakObjectPtr<UMoverCVDSimDataComponent>* FMoverCVDTab::FindMoverDataComponentForSolver(int32 SolverID)
{
	TWeakObjectPtr<UMoverCVDSimDataComponent>* FoundSimDataComponent = SolverToSimDataComponentMap.Find(SolverID);
	if (!FoundSimDataComponent || FoundSimDataComponent->IsStale())
	{
		// Refresh SolverToSimDataComponentMap
		RetrieveAllSolversMoverDataComponents();

		// Try again
		FoundSimDataComponent = SolverToSimDataComponentMap.Find(SolverID);
	}

	return FoundSimDataComponent;
}

void FMoverCVDTab::DisplaySingleParticleInfo(int32 SelectedSolverID, int32 SelectedParticleID)
{
	// At the moment this is really written to support only one particle selected at a time
	// I we were to have multiple selection we would have to cache the whole list of particle IDs selected
	bool IsParticleBeingSelected = (SelectedParticleID != INDEX_NONE && SelectedSolverID != INDEX_NONE);
	bool WasDataFoundForParticle = false;
	if (IsParticleBeingSelected)
	{
		// Find the sim data component weak pointer for the selected solver ID
		TWeakObjectPtr<UMoverCVDSimDataComponent>* FoundSimDataComponent = FindMoverDataComponentForSolver(SelectedSolverID);
		if (FoundSimDataComponent)
		{
			// Pin a strong pointer to it so it doesn't get away while in use
			TStrongObjectPtr<UMoverCVDSimDataComponent> SimDataComponentSharedPtr = FoundSimDataComponent->Pin();
			UMoverCVDSimDataComponent* SimDataComponent = SimDataComponentSharedPtr.Get();
			if (IsValid(SimDataComponent))
			{
				if (SimDataComponent->FindAndUnwrapSimDataForParticle(SelectedParticleID, MoverSimDataWrapper, MoverSyncState, MoverInputCmd, MoverLocalSimDataSections))
				{
					if (MoverSimDataWrapper && MoverInputCmd && MoverSyncState)
					{
						if (DetailsPanelView)
						{
							MultiViewWrapper.Clear();

							static const FName InputCommandCategory("InputCommand");
							static const FName SyncStateCategory("SyncState");

							// This displays the particle ID and solver ID
							MultiViewWrapper.AddData(MoverSimDataWrapper.Get());

							// Input Command Context structs
							for (const TSharedPtr<FMoverDataStructBase>& InputDataStructSharedPtr : MoverInputCmd->InputCollection.GetDataArray())
							{
								if (FMoverDataStructBase* MoverDataStructBase = InputDataStructSharedPtr.Get())
								{
									MultiViewWrapper.AddData(MakeShared<FStructOnScope>(MoverDataStructBase->GetScriptStruct(), reinterpret_cast<uint8*>(MoverDataStructBase)), InputCommandCategory);
								}
							}

							MultiViewWrapper.AddData(MoverSyncState.Get(), SyncStateCategory);
							// Sync State Data Collection structs
							for (const TSharedPtr<FMoverDataStructBase>& SyncStateDataStructSharedPtr : MoverSyncState->SyncStateCollection.GetDataArray())
							{
								if (FMoverDataStructBase* MoverDataStructBase = SyncStateDataStructSharedPtr.Get())
								{
									MultiViewWrapper.AddData(MakeShared<FStructOnScope>(MoverDataStructBase->GetScriptStruct(), reinterpret_cast<uint8*>(MoverDataStructBase)), SyncStateCategory);
								}
							}

							// Local Simulation State — each named section becomes its own category in the details panel
							for (const TPair<FName, TSharedPtr<FMoverDataCollection>>& Section : MoverLocalSimDataSections)
							{
								if (Section.Value)
								{
									for (const TSharedPtr<FMoverDataStructBase>& LocalSimDataStructSharedPtr : Section.Value->GetDataArray())
									{
										if (FMoverDataStructBase* MoverDataStructBase = LocalSimDataStructSharedPtr.Get())
										{
											MultiViewWrapper.AddData(MakeShared<FStructOnScope>(MoverDataStructBase->GetScriptStruct(), reinterpret_cast<uint8*>(MoverDataStructBase)), Section.Key);
										}
									}
								}
							}

							WasDataFoundForParticle = true;

							SetStructToInspect(&MultiViewWrapper);
						}
					}
					// Cache the particle ID so we can display info for that same particle when we scrub to a different frame that also has that particle
					// WARNING: Apparently IDs can be reused across frames for different particles, so this might not be good enough
					CurrentlyDisplayedSolverID = SelectedSolverID;
					CurrentlyDisplayedParticleID = SelectedParticleID;
				}
			}
		}
	}

	if (!WasDataFoundForParticle)
	{
		CurrentlyDisplayedSolverID = INDEX_NONE;
		CurrentlyDisplayedParticleID = INDEX_NONE;
	}
}

void FMoverCVDTab::DisplayMoverInfoForSelectedElements(const TArray<FTypedElementHandle>& SelectedElementHandles)
{
	if (DetailsPanelView)
	{
		DetailsPanelView->SetSelectedStruct(nullptr);

		bool ShouldClearSimDataDetailsPanel = true;
		using namespace Chaos::VD::TypedElementDataUtil;
		for (int32 SelectionIndex = 0; SelectionIndex < SelectedElementHandles.Num(); ++SelectionIndex)
		{
			if (const FChaosVDSceneParticle* Particle = GetStructDataFromTypedElementHandle<FChaosVDSceneParticle>(SelectedElementHandles[SelectionIndex]))
			{
				int32 SelectedParticleID = Particle->GetParticleData()->ParticleIndex; // Replace with ID of selected particle if one is selected
				int32 SelectedSolverID = Particle->GetParticleData()->SolverID;
				ShouldClearSimDataDetailsPanel = false;
				DisplaySingleParticleInfo(SelectedSolverID, SelectedParticleID);

				// Right now, we only handle displaying info for the first particle in the selection, but later we might not break here and display info for all particles
				break;
			}
		}

		// Clear the details panel view if we didn't find particles in the selection
		if (ShouldClearSimDataDetailsPanel)
		{
			DetailsPanelView->SetSelectedStruct(nullptr);
		}
	}
}

void FMoverCVDTab::HandleSceneUpdated()
{
	// This is called when we scrub the timelines for instance
	bool IsParticleSelected = (CurrentlyDisplayedSolverID != INDEX_NONE && CurrentlyDisplayedParticleID != INDEX_NONE);

	if (!IsParticleSelected)
	{
		return;
	}

	const TSharedPtr<FChaosVDScene> Scene = SceneWeakPtr.Pin();
	if (!Scene)
	{
		return;
	}

	bool ShouldClearSimDataDetailsPanel = true;
	using namespace Chaos::VD::TypedElementDataUtil;
	TArray<FTypedElementHandle> SelectedParticlesHandles = Scene->GetSelectedElementHandles();
	DisplayMoverInfoForSelectedElements(SelectedParticlesHandles);
}

void FMoverCVDTab::HandlePostSelectionChange(const UTypedElementSelectionSet* ChangedSelectionSet)
{
	// This is called when the selection changes
	TArray<FTypedElementHandle> SelectedParticlesHandles;
	ChangedSelectionSet->GetSelectedElementHandles(SelectedParticlesHandles, UChaosVDSelectionInterface::StaticClass());
	DisplayMoverInfoForSelectedElements(SelectedParticlesHandles);
}

#undef LOCTEXT_NAMESPACE


