// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "ChaosVDTabSpawnerBase.h"
#include "Widgets/SChaosVDDetailsView.h"
#include "ChaosVDSolverDataSelection.h"
#include "ChaosVDObjectDetailsTab.h"
#include "TEDS/ChaosVDStructTypedElementData.h"

class SOutputLog;
class FOutputLogHistory;
class FName;
struct FMoverSyncState;
struct FMoverInputCmdContext;
struct FMoverCVDSimDataWrapper;
struct FMoverDataCollection;
class UMoverCVDSimDataComponent;

/** This tab is an additional details tab displaying mover info corresponding to the selected particles if they are moved by a Mover component */
class FMoverCVDTab : public FChaosVDObjectDetailsTab
{
public:

	FMoverCVDTab(const FName& InTabID, TSharedPtr<FTabManager> InTabManager, TWeakPtr<SChaosVDMainTab> InOwningTabWidget);
	virtual ~FMoverCVDTab();

	// Implementation of FChaosVDObjectDetailsTab
	virtual TSharedRef<SDockTab> HandleTabSpawnRequest(const FSpawnTabArgs& Args) override;
	virtual void HandlePostSelectionChange(const UTypedElementSelectionSet* ChangedSelectionSet) override;

	virtual void HandleSolverDataSelectionChange(const TSharedPtr<FChaosVDSolverDataSelectionHandle>& SelectionHandle) override;

	// Scene callbacks
	void HandleSceneUpdated();

private:
	void DisplaySingleParticleInfo(int32 SelectedSolverID, int32 SelectedParticleID);
	void DisplayMoverInfoForSelectedElements(const TArray<FTypedElementHandle>& SelectedElementHandles);

	// This function retrieves and caches all the mover data components for all solvers, populating SolverToSimDataComponentMap
	void RetrieveAllSolversMoverDataComponents();

	TWeakObjectPtr<UMoverCVDSimDataComponent>* FindMoverDataComponentForSolver(int32 SolverID);

	TWeakPtr<FChaosVDScene> SceneWeakPtr;

	TMap<int32, TWeakObjectPtr<UMoverCVDSimDataComponent>> SolverToSimDataComponentMap;

	int32 CurrentlyDisplayedParticleID = INDEX_NONE;
	int32 CurrentlyDisplayedSolverID = INDEX_NONE;

	FChaosVDSelectionMultipleView MultiViewWrapper;

	TSharedPtr<FMoverCVDSimDataWrapper> MoverSimDataWrapper;
	TSharedPtr<FMoverSyncState> MoverSyncState;
	TSharedPtr<FMoverInputCmdContext> MoverInputCmd;
	TArray<TPair<FName, TSharedPtr<FMoverDataCollection>>> MoverLocalSimDataSections;
};


