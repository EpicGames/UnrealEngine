// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

class FChaosVDExtension;

/** This module contains the Editor side of the Mover support in Chaos Visual Debugger (CVD)
* Mover is a module responsible for the movement of actors and its replication, for instance the input controlled movement of characters
* This module allows the display and recording specific data in CVD that makes it easier to understand and debug Mover
* MoverCVDTab is the tab in CVD where that information is displayed
* MoverCVDSimDataProcessor is the receiving and processing end of the Mover data trace ("Sim Data") that the game sends to CVD
* MoverSimDataComponent is a component holding Mover data for the current visualized frame
* MoverCVDExtension is where we register MoverCVDTab as a displayable tab, register MoverCVDSimDataProcessor and give access to the MoverSimDataComponent
*/
class FMoverCVDEditorModule : public IModuleInterface
{
public:
    virtual void StartupModule() override;
    virtual void ShutdownModule() override;

private:

	TArray<TWeakPtr<FChaosVDExtension>> AvailableExtensions;
};
