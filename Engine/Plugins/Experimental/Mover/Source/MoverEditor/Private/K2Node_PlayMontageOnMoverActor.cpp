// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_PlayMontageOnMoverActor.h"

#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphPin.h"
#include "MoveLibrary/PlayMoverMontageCallbackProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_PlayMontageOnMoverActor)

#define LOCTEXT_NAMESPACE "Mover_K2Nodes"

UK2Node_PlayMontageOnMoverActor::UK2Node_PlayMontageOnMoverActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(UPlayMoverMontageCallbackProxy, CreateProxyObjectForPlayMoverMontage);
	ProxyFactoryClass = UPlayMoverMontageCallbackProxy::StaticClass();
	ProxyClass = UPlayMoverMontageCallbackProxy::StaticClass();
}

FText UK2Node_PlayMontageOnMoverActor::GetTooltipText() const
{
	return LOCTEXT("K2Node_PlayMontageOnMoverActor_Tooltip", "Plays a Montage on an actor with Mover and SkeletalMesh components. Used for networked animation root motion.\nThe Primary Visual Component will be used if set, or the first skeletal mesh. See @SetPrimaryVisualComponent");
}

FText UK2Node_PlayMontageOnMoverActor::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("PlayMontageOnMoverActor", "Play Montage (Mover Actor)");
}

FText UK2Node_PlayMontageOnMoverActor::GetMenuCategory() const
{
	return LOCTEXT("PlayMontageCategory", "Animation|Montage");
}

static const FName NAME_OnNotifyBegin = "OnNotifyBegin";
static const FName NAME_OnNotifyEnd = "OnNotifyEnd";

void UK2Node_PlayMontageOnMoverActor::GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const
{
	Super::GetPinHoverText(Pin, HoverTextOut);

	if (Pin.PinName == NAME_OnNotifyBegin)
	{
		FText ToolTipText = LOCTEXT("K2Node_PlayMontageOnMoverActor_OnNotifyBegin_Tooltip", "Event called when using a PlayMontageNotify or PlayMontageNotifyWindow Notify in a Montage.");
		HoverTextOut = FString::Printf(TEXT("%s\n%s"), *ToolTipText.ToString(), *HoverTextOut);
	}
	else if (Pin.PinName == NAME_OnNotifyEnd)
	{
		FText ToolTipText = LOCTEXT("K2Node_PlayMontageOnMoverActor_OnNotifyEnd_Tooltip", "Event called when using a PlayMontageNotifyWindow Notify in a Montage.");
		HoverTextOut = FString::Printf(TEXT("%s\n%s"), *ToolTipText.ToString(), *HoverTextOut);
	}
}

#undef LOCTEXT_NAMESPACE
