// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"


class FMoverEditorModule : public IModuleInterface
{
};

IMPLEMENT_MODULE(FMoverEditorModule, MoverEditor)
