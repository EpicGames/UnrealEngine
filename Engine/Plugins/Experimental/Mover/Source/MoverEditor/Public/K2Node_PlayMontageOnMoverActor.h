// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "K2Node_BaseAsyncTask.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_PlayMontageOnMoverActor.generated.h"

class FString;
class UEdGraphPin;
class UObject;

// PlayMontageOnMoverActor: used for playing montages on Mover-based actors. Handles the montage's root motion separately from the animation pose.
UCLASS()
class UK2Node_PlayMontageOnMoverActor : public UK2Node_BaseAsyncTask
{
	GENERATED_UCLASS_BODY()

	//~ Begin UEdGraphNode Interface
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual FText GetMenuCategory() const override;
	//~ End UK2Node Interface
};
